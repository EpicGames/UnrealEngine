{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Audio Insights",
	"Description": "Suite of tools to profile, debug, and monitor aspects of audio in the Unreal Engine.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "AudioInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		},
		{
			"Name": "AudioInsightsEditor",
			"Type": "EditorNoCommandlet",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AudioWidgets",
			"Enabled": true
		}
	]
}