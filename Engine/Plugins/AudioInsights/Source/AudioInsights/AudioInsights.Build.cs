// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioInsights : ModuleRules
{
	public AudioInsights(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"RewindDebuggerRuntimeInterface",
				"TraceInsights",
				"TraceServices",
				"ToolWidgets",
			}
		);	
		
		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioMixerCore",
				"AudioWidgetsCore",
				"BuildSettings",
				"CoreUObject",
				"DeveloperSettings",
				"InputCore",
				"OutputLog",
				"SessionServices",
				"Slate",
				"SlateCore",
				"Sockets",
				"TraceAnalysis",
				"TraceLog",
				"ToolMenus",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
				});
			
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UnrealEd",
			});
		}
	}
}
