// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetEditorCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioInsights.AssetEditorCommands"

namespace UE::Audio::Insights
{
	FAssetEditorCommands::FAssetEditorCommands()
		: TCommands<FAssetEditorCommands>("AssetEditorCommands", LOCTEXT("AssetEditorCommands_ContextDescText", "Asset Editor Commands"), NAME_None, UE::Audio::Insights::FSlateStyle::GetStyleName())
	{
	}

	void FAssetEditorCommands::RegisterCommands()
	{
		UI_COMMAND(Browse, "Browse To Asset", "Browses to the selected node asset in the content browser.", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Control));
		UI_COMMAND(Edit, "Edit", "Opens the selected node for edit.", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Control));
	}

} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
