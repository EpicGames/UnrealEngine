// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioEventLogEditorCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioInsights.AudioEventLogEditorCommands"

namespace UE::Audio::Insights
{
	FAudioEventLogEditorCommands::FAudioEventLogEditorCommands()
		: TCommands<FAudioEventLogEditorCommands>("AudioEventLogEditorCommands", LOCTEXT("AudioEventLogEditorCommands_ContextDescText", "Audio Event Log Editor Commands"), NAME_None, UE::Audio::Insights::FSlateStyle::GetStyleName())
	{
	}

	void FAudioEventLogEditorCommands::RegisterCommands()
	{
		UI_COMMAND(Browse, "Browse To Asset", "Browses to the selected sound asset in the content browser.", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Control));
		UI_COMMAND(Edit, "Edit", "Opens the selected sound for edit.", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Control));
		UI_COMMAND(ResetInspectTimestampEventLog, "Reset Inspect Timestamp Event Log", "Resumes gathering the latest audio data and auto-scrolling in the Event Log if previously paused.", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	}

} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
