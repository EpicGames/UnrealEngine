// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

namespace UE::Audio::Insights
{
	class FAudioEventLogEditorCommands : public TCommands<FAudioEventLogEditorCommands>
	{
	public:
		FAudioEventLogEditorCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<const FUICommandInfo> GetBrowseCommand() const { return Browse; }
		TSharedPtr<const FUICommandInfo> GetEditCommand() const { return Edit; }
		TSharedPtr<const FUICommandInfo> GetResetInspectTimestampCommand() const { return ResetInspectTimestampEventLog; }

	private:
		TSharedPtr<FUICommandInfo> Browse;
		TSharedPtr<FUICommandInfo> Edit;
		TSharedPtr<FUICommandInfo> ResetInspectTimestampEventLog;
	};
} // namespace UE::Audio::Insights
