// Copyright Epic Games, Inc. All Rights Reserved.
#if !WITH_EDITOR
#include "AudioInsightsComponent.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsTraceModule.h"
#include "AudioInsightsStyle.h"
#include "Insights/IInsightsManager.h"
#include "Modules/ModuleManager.h"
#include "Trace/StoreClient.h"
#include "TraceServices/Model/Diagnostics.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "AudioInsightsComponent"

namespace UE::Audio::Insights
{
	namespace ComponentPrivate
	{
		static const FName TabName = "Audio Insights";
	}

	FAudioInsightsComponent::~FAudioInsightsComponent()
	{
		ensure(!bIsInitialized);
	}

	TSharedPtr<FAudioInsightsComponent> FAudioInsightsComponent::CreateInstance()
	{
		ensure(!Instance.IsValid());

		if (Instance.IsValid())
		{
			Instance.Reset();
		}

		Instance = MakeShared<FAudioInsightsComponent>();

		return Instance;
	}

	void FAudioInsightsComponent::Initialize(IUnrealInsightsModule& InsightsModule)
	{
		ensure(!bIsInitialized);
	
		if (!bIsInitialized)
		{
			bIsInitialized = true;

			OnTick = FTickerDelegate::CreateSP(this, &FAudioInsightsComponent::Tick);

			constexpr float TickDelay = 0.5f; // 500 ms. delay between ticks
			OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, TickDelay);

			IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

			TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
			if (InsightsManager.IsValid())
			{
				InsightsManager->GetSessionAnalysisCompletedEvent().AddSP(this, &FAudioInsightsComponent::OnSessionAnalysisCompletedEvent);
			}
		}
	}

	void FAudioInsightsComponent::Shutdown()
	{
		if (!bIsInitialized)
		{
			return;
		}
	
		bIsInitialized = false;

		FTSTicker::RemoveTicker(OnTickHandle);

		IAudioInsightsTraceModule& TraceModule = FAudioInsightsModule::GetChecked().GetTraceModule();
		TraceModule.ResetTicker();

		Instance.Reset();
	}

	void FAudioInsightsComponent::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
	{
		using namespace ComponentPrivate;

		const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(TabName);

		if (Config.bIsAvailable)
		{
			// Register tab spawner for the Audio Insights.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName,
				FOnSpawnTab::CreateRaw(this,  &FAudioInsightsComponent::SpawnTab), 
				FCanSpawnTab::CreateRaw(this, &FAudioInsightsComponent::CanSpawnTab))
				.SetDisplayName(Config.TabLabel.IsSet()   ? Config.TabLabel.GetValue()   : LOCTEXT("AudioInsights_TabTitle", "Audio Insights"))
				.SetTooltipText(Config.TabTooltip.IsSet() ? Config.TabTooltip.GetValue() : LOCTEXT("AudioInsights_TooltipText", "Open the Audio Insights tab."))
				.SetIcon(Config.TabIcon.IsSet() ? Config.TabIcon.GetValue() : FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Submix"));

			const TSharedRef<FWorkspaceItem>* FoundWorkspace = FGlobalTabmanager::Get()->GetLocalWorkspaceMenuRoot()->GetChildItems().FindByPredicate(
				[](const TSharedRef<FWorkspaceItem>& WorkspaceItem)
				{
					return WorkspaceItem->GetDisplayName().ToString() == "Insights Tools";
				});

			if (FoundWorkspace)
			{
				TabSpawnerEntry.SetGroup(*FoundWorkspace);
			}
		}
	}

	void FAudioInsightsComponent::UnregisterMajorTabs()
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ComponentPrivate::TabName);
	}

	bool FAudioInsightsComponent::CanSpawnTab(const FSpawnTabArgs& Args) const
	{
		return bCanSpawnTab;
	}

	TSharedRef<SDockTab> FAudioInsightsComponent::SpawnTab(const FSpawnTabArgs& Args)
	{
		IAudioInsightsTraceModule& TraceModule = FAudioInsightsModule::GetChecked().GetTraceModule();
		TraceModule.RequestChannelUpdate();

		const TSharedRef<SDockTab> DockTab = FAudioInsightsModule::GetChecked().CreateDashboardTabWidget(Args);

		OnTabSpawn.Broadcast();

		return DockTab;
	}

	void FAudioInsightsComponent::OnSessionAnalysisCompletedEvent()
	{
		OnSessionAnalysisCompleted.Broadcast();
	}

	bool FAudioInsightsComponent::GetIsLiveSession() const
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> Session = UnrealInsightsModule.GetAnalysisSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const UE::Trace::FStoreClient* StoreClient = UnrealInsightsModule.GetStoreClient();
			if (StoreClient)
			{
				const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByTraceId(Session->GetTraceId());
				return !Session->IsAnalysisComplete() && SessionInfo != nullptr;
			}
			// Direct traces have a trace ID of zero and a null StoreClient - check IDiagnosticsProvider to see if we are running a direct trace
			else if (Session->GetTraceId() == 0)
			{
				const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*Session.Get());
				if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
				{
					return !Session->IsAnalysisComplete();
				}
			}
		}

		return false;
	}

	bool FAudioInsightsComponent::IsSessionAnalysisComplete() const
	{
		const IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> Session = UnrealInsightsModule.GetAnalysisSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			return Session->IsAnalysisComplete();
		}

		return false;
	}

	bool FAudioInsightsComponent::Tick(float DeltaTime)
	{
		if (!bCanSpawnTab)
		{
			const IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

			TSharedPtr<const TraceServices::IAnalysisSession> Session = UnrealInsightsModule.GetAnalysisSession();
			if (Session.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

				const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*Session.Get());
				if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
				{
					const TraceServices::FSessionInfo& TraceServicesSessionInfo = DiagnosticsProvider->GetSessionInfo();

					bIsEditorTrace = TraceServicesSessionInfo.TargetType == EBuildTargetType::Editor;

					IAudioInsightsTraceModule& TraceModule = FAudioInsightsModule::GetChecked().GetTraceModule();
					TraceModule.InitializeSessionInfo(TraceServicesSessionInfo);
				}

				bCanSpawnTab = true;
			}
		}

		return true;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE

#endif // !WITH_EDITOR
