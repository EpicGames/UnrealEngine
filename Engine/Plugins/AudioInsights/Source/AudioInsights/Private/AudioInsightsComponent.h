// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !WITH_EDITOR
#include "Containers/Ticker.h"
#include "Insights/IUnrealInsightsModule.h"

namespace UE::Audio::Insights
{
	/**
	 * The component that makes Audio Insights available inside Unreal Insights.
	 */
	class FAudioInsightsComponent : public IInsightsComponent, public TSharedFromThis<FAudioInsightsComponent>
	{
	public:
		FAudioInsightsComponent() = default;
		virtual ~FAudioInsightsComponent();

		static TSharedPtr<FAudioInsightsComponent> CreateInstance();

		// IInsightsComponent overrides
		virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
		virtual void Shutdown() override;
		virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
		virtual void UnregisterMajorTabs() override;

		bool GetIsLiveSession() const;
		bool IsSessionAnalysisComplete() const;
		bool GetIsEditorTrace() const { return bIsEditorTrace; }

		DECLARE_MULTICAST_DELEGATE(FOnTabSpawn);
		FOnTabSpawn OnTabSpawn;

		DECLARE_MULTICAST_DELEGATE(FOnSessionAnalysisCompleted);
		FOnSessionAnalysisCompleted OnSessionAnalysisCompleted;

	private:
		bool CanSpawnTab(const FSpawnTabArgs& Args) const;
		TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

		void OnSessionAnalysisCompletedEvent();

		bool Tick(float DeltaTime);

		bool bIsInitialized = false;
		bool bIsEditorTrace = false;
		bool bCanSpawnTab   = false;

		FTickerDelegate OnTick;
		FTSTicker::FDelegateHandle OnTickHandle;

		inline static TSharedPtr<FAudioInsightsComponent> Instance;
	};
} // namespace UE::Audio::Insights
#endif // !WITH_EDITOR