// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsDashboardAssetCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Misc/Attribute.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

UE_DEFINE_TCOMMANDS(UE::Audio::Insights::FDashboardAssetCommands)

namespace UE::Audio::Insights
{
	FDashboardAssetCommands::FDashboardAssetCommands()
		: TCommands<FDashboardAssetCommands>("AudioInsightsDashboardAssetCommands", LOCTEXT("AudioInsightsDashboardAssetCommands", "Dashboard Asset Commands"), NAME_None, FSlateStyle::GetStyleName())
	{
	}

	void FDashboardAssetCommands::RegisterCommands()
	{
		UI_COMMAND(BrowserSync, "Browse", "Browses to the selected asset in the content browser.", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Control));
		UI_COMMAND(Open, "Open", "Opens the selected asset(s) in respective editor(s).", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Control));
		UI_COMMAND(Save, "Save", "Saves the selected asset.", EUserInterfaceActionType::Button, FInputChord(EKeys::S, EModifierKey::Control));
		UI_COMMAND(SaveAll, "Save All", "Saves all modified assets.", EUserInterfaceActionType::Button, FInputChord(EKeys::S, EModifierKey::Control | EModifierKey::Shift));
		UI_COMMAND(CreateTraceBookmark, "Create Trace Bookmark", "Places a bookmark inside a trace file when tracing with Audio Insights.", EUserInterfaceActionType::Button, FInputChord(EKeys::M, EModifierKey::Control));
	}

	void FDashboardAssetCommands::AddAssetCommands(FToolBarBuilder& OutToolbarBuilder) const
	{
		OutToolbarBuilder.AddToolBarButton(
			BrowserSync,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			TAttribute<FSlateIcon>::Create([]() { return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.ContentBrowser"); }),
			"BrowserSync"
		);

		OutToolbarBuilder.AddToolBarButton(
			Open,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			TAttribute<FSlateIcon>::Create([]() { return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Open"); }),
			"Open"
		);
	}

	TSharedPtr<const FUICommandInfo> FDashboardAssetCommands::GetBrowserSyncCommand() const
	{
		return BrowserSync;
	};

	TSharedPtr<const FUICommandInfo> FDashboardAssetCommands::GetOpenCommand() const
	{
		return Open;
	}

	TSharedPtr<const FUICommandInfo> FDashboardAssetCommands::GetSaveCommand() const
	{
		return Save;
	}

	TSharedPtr<const FUICommandInfo> FDashboardAssetCommands::GetSaveAllCommand() const
	{
		return SaveAll;
	}

	TSharedPtr<const FUICommandInfo> FDashboardAssetCommands::GetTraceBookmarkCommand() const
	{
		return CreateTraceBookmark;
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
