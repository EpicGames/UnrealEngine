// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsDashboardFactory.h"

#include "AudioInsightsComponent.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAudioInsightsModuleInterface.h"
#include "Internationalization/Text.h"
#include "Messages/BookmarkTraceMessages.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "AudioInsights"


namespace UE::Audio::Insights
{
	namespace DashboardFactoryPrivate
	{
		static const FText ToolName = LOCTEXT("AudioDashboard_ToolName", "Audio Insights");

		static const FText EnableTracesButtonText = LOCTEXT("AudioDashboard_AutomaticallyEnableTracesTitle", "Enable audio traces");
		static const FText EnableTracesDescription = LOCTEXT("AudioDashboard_AutomaticallyEnableTracesDescription", "Enables the audio and audio mixer trace channels. Audio Insights will not function without these channels enabled.");
		static const FText NoTracesEnabledWarning = LOCTEXT("AudioDashboard_NoTracesEnabledWarning", "Audio Insights requires the audio and audio mixer trace channels to be enabled to function.");
		static const FText EnableThemNowText = LOCTEXT("AudioDashboard_EnableNowText", "Enable them now?");
		static const FText TraceControllerUnavailabledWarning = LOCTEXT("AudioDashboard_TraceControllerUnavailableWarning", "The Trace Controller API is currently unavailable.");
		static const FText TryEnablingMessagingText = LOCTEXT("AudioDashboard_TryEnablingMessagingText", "Make sure you have launched this package with the -Messaging command line argument.");
		static const FText SaveCacheSnapshotText = LOCTEXT("AudioDashboard_SaveCacheSnapshotButton_Text", "Save Cache Snapshot");
		static const FText SaveCacheSnapshotTooltipText = LOCTEXT("AudioDashboard_SaveCacheSnapshotButton_TooltipText", "Saves a snapshot of the Audio Insights cached data to a .utrace file containing only audio trace events.");
		static const FText SaveBookmarkText = LOCTEXT("AudioDashboard_BookmarkButton_Text", "Save Bookmark");
		static const FText SaveBookmarkTooltipText = LOCTEXT("AudioDashboard_BookmarkButton_TooltipText", "Creates an 'Audio Insights' bookmark at the current cache timestamp.");
		static const FText BookmarkLabelText = LOCTEXT("AudioDashboard_BookmarkLabelTitle_Text", "Audio Insights Bookmark");

		static const FText DocumentationTooltipText = LOCTEXT("AudioDashboard_Documentation_TooltipText", "Opens Audio Insights documentation.");
	}

	::Audio::FDeviceId FDashboardFactory::GetDeviceId() const
	{
		return ActiveDeviceId;
	}

	TSharedRef<SDockTab> FDashboardFactory::MakeDockTabWidget(const FSpawnTabArgs& Args)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(DashboardFactoryPrivate::ToolName)
			.Clipping(EWidgetClipping::ClipToBounds)
			.TabRole(ETabRole::NomadTab);

		DashboardTabManager = FGlobalTabmanager::Get()->NewTabManager(DockTab);

		ToolbarWidgets = MakeShared<FToolbarWidgets>();

		TabLayout = GetDefaultTabLayout();

		RegisterTabSpawners();

		const TSharedRef<SWidget> TabContent =

#if !WITH_EDITOR
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
#endif	// !WITH_EDITOR

				SNew(SVerticalBox)
				// Menu bar (File | View | etc.)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					MakeMenuBarWidget()
				]
				// Main Toolbar (for general Audio Insights buttons)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					MakeMainToolbarWidget()
				]
				// Spacing
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(4.0f)
				]
				// Dashboards area
				+ SVerticalBox::Slot()
				[
					DashboardTabManager->RestoreFrom(TabLayout->AsShared(), Args.GetOwnerWindow()).ToSharedRef()
#if WITH_EDITOR
				];
#else
				]
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				MakeEnableTracesOverlay()
			];
#endif	// WITH_EDITOR

		DockTab->SetContent(TabContent);

		DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab> TabClosed)
		{
			UnregisterTabSpawners();
			ToolbarWidgets.Reset();
		}));

		return DockTab;
	}

	TSharedRef<SWidget> FDashboardFactory::MakeMenuBarWidget()
	{
		FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

		MenuBarBuilder.AddPullDownMenu(
			LOCTEXT("File_MenuLabel", "File"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("Close_MenuLabel", "Close"),
					LOCTEXT("Close_MenuLabel_Tooltip", "Closes the Audio Insights dashboard."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this]()
					{
						if (DashboardTabManager.IsValid())
						{
							if (TSharedPtr<SDockTab> OwnerTab = DashboardTabManager->GetOwnerTab())
							{
								OwnerTab->RequestCloseTab();
							}
						}
					}))
				);
			}),
			"File"
		);

		MenuBarBuilder.AddPullDownMenu(
			LOCTEXT("ViewMenuLabel", "View"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
			{
				for (const auto& KVP : DashboardViewFactories)
				{
					const FName& FactoryName = KVP.Key;
					const TSharedPtr<IDashboardViewFactory>& Factory = KVP.Value;

					MenuBuilder.AddMenuEntry(
						Factory->GetDisplayName(),
						FText::GetEmpty(),
						Factory->GetIcon(),
						FUIAction(FExecuteAction::CreateLambda([this, FactoryName]()
						{
							if (DashboardTabManager.IsValid())
							{
								const TSharedPtr<SDockTab> ViewportTab = DashboardTabManager->FindExistingLiveTab(FactoryName);

								if (!ViewportTab.IsValid())
								{
									const TSharedPtr<SDockTab> InvokedTab = DashboardTabManager->TryInvokeTab(FactoryName);

									if (InvokedTab.IsValid() && DashboardViewFactories[FactoryName].IsValid())
									{
										const EDefaultDashboardTabStack DefaultTabStack = DashboardViewFactories[FactoryName]->GetDefaultTabStack();

										if (DefaultTabStack == EDefaultDashboardTabStack::OutputMetering)
										{
											InvokedTab->SetParentDockTabStackTabWellHidden(true);
										}
									}
								}
								else
								{
									ViewportTab->RequestCloseTab();
								}
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([&DashboardTabManager = DashboardTabManager, FactoryName]()
						{
							return DashboardTabManager.IsValid() ? DashboardTabManager->FindExistingLiveTab(FactoryName).IsValid() : false;
						})),
						NAME_None,
						EUserInterfaceActionType::Check
					);

					if (const EDefaultDashboardTabStack DefaultTabStack = DashboardViewFactories[FactoryName]->GetDefaultTabStack();
						DefaultTabStack == EDefaultDashboardTabStack::Log 
						|| DefaultTabStack == EDefaultDashboardTabStack::AudioMeters
						|| DefaultTabStack == EDefaultDashboardTabStack::Plots)
					{
						MenuBuilder.AddMenuSeparator();
					}
				}
			}),
			"View"
		);

		return MenuBarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FDashboardFactory::MakeMainToolbarWidget()
	{
		return SNew(SVerticalBox)
			// Top row: Snapshot, bookmark
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				// Save cache snapshot button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(12.0f, 0.0f, 0.0f, 0.0f)
				[
					MakeCacheSnapshotButtonWidget()
				]
				// Bookmark button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					MakeBookmarkButtonWidget()
				]

				// Documentation
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(12.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Right)
				[
					CreateDocumentationButtonWidget()
				]
			]
			// Bottom row: Cache controls
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				// Cache label
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(28.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheLabelWidget()
				]
				// Cache control: Pause button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(30.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCachePauseButtonWidget()
				]
				// Stop cache button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheStopButtonWidget()
				]
				// Resume cache button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(1.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheResumeButtonWidget()
				]
				// Current timestamp
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheCurrentTimestampWidget()
				]
				// Follow button
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheFollowButtonWidget()
				]
				// Begin timestamp
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheBeginTimestampWidget()
				]
				// Nudge back
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheNudgeBackButtonWidget()
				]
				// Timeline ruler
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheTimelineRulerWidget()
				]
				// Nudge forward
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheNudgeForwardButtonWidget()
				]
				// End timestamp
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheEndTimestampWidget()
				]
				// Separator
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SBox)
					.HeightOverride(22.0f)
					[
						SNew(SSeparator)
						.Orientation(Orient_Vertical)
						.Thickness(3.0f)
					]
				]
				// Cache size and duration
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					ToolbarWidgets->MakeCacheSizeAndDurationWidget()
				]
				// Cache settings
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					ToolbarWidgets->MakeCacheSettingsButtonWidget()
				]
			];
	}

	TSharedRef<SWidget> FDashboardFactory::CreateDocumentationButtonWidget()
	{
		using namespace DashboardFactoryPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(DocumentationTooltipText)
			.OnClicked_Lambda([]()
			{
				// Note: Intentionally using this URL to resolve to the latest Audio Insights documentation
				// Found that using the Documentation module not only is not available in standalone, but was also resolving to the incorrect engine version
				FPlatformProcess::LaunchURL(TEXT("https://dev.epicgames.com/documentation/unreal-engine/audio-insights-in-unreal-engine"), nullptr, nullptr);
				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Documentation"))
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				]
			];
	}

	TSharedRef<SWidget> FDashboardFactory::MakeCacheSnapshotButtonWidget()
	{
		using namespace DashboardFactoryPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(SaveCacheSnapshotTooltipText)
#if !WITH_EDITOR
			.IsEnabled_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession();
			})
#endif // !WITH_EDITOR
			.OnClicked_Lambda([]()
			{
				FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
				AudioInsightsTraceModule.SaveCacheSnapshot();

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					// Save cache snapshot icon
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Trace.CacheSnapshot"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					// Save cache snapshot text
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(6.0f, 1.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(SaveCacheSnapshotText)
					]
				]
			];
	}

	TSharedRef<SWidget> FDashboardFactory::MakeBookmarkButtonWidget()
	{
		using namespace DashboardFactoryPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(SaveBookmarkTooltipText)
#if !WITH_EDITOR
			.IsEnabled_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession();
			})
#endif // !WITH_EDITOR
			.OnClicked_Lambda([this]()
			{
				CreateTraceBookmark();

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Trace.Bookmark"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					// Save bookmark text
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(6.0f, 1.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(SaveBookmarkText)
					]
				]
			];
	}

	TSharedPtr<FTabManager::FLayout> FDashboardFactory::GetDefaultTabLayout()
	{
		using namespace DashboardFactoryPrivate;

		TSharedRef<FTabManager::FStack> LogTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> AnalysisTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> AudioMetersTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> OutputMeteringTabStack = FTabManager::NewStack()->SetHideTabWell(true);
		TSharedRef<FTabManager::FStack> PlotsTabStack = FTabManager::NewStack();

		for (const auto& [FactoryName, Factory] : DashboardViewFactories)
		{
			const EDefaultDashboardTabStack DefaultTabStack = Factory->GetDefaultTabStack();

			switch (DefaultTabStack)
			{
				case EDefaultDashboardTabStack::Log:
				{
					LogTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::Analysis:
				{
					AnalysisTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::AudioMeters:
				{
					AudioMetersTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::OutputMetering:
				{
					OutputMeteringTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::Plots:
				{
					PlotsTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;
				
				default:
					break;
			}
		}

		AnalysisTabStack->SetForegroundTab(FName("Sounds"));

		return FTabManager::NewLayout("AudioDashboard_Layout_v3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				// Left column
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					LogTabStack
					->SetSizeCoefficient(0.25f)
				)

				// Middle column
				->Split
				(
					// Top
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.6f) // Column width
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.6f) // Row width
						->Split
						(
							AnalysisTabStack
							->SetSizeCoefficient(0.58f)
						)
						// Bottom
						->Split
						(
							// Top
							FTabManager::NewSplitter()
							->SetOrientation(Orient_Horizontal)
							->SetSizeCoefficient(0.42f) // Row width
							->Split
							(
								PlotsTabStack
								->SetSizeCoefficient(0.5f)
							)
							// Bottom
							->Split
							(
								AudioMetersTabStack
								->SetSizeCoefficient(0.5f)
							)
						)
					)
				)

				// Right column
				->Split
				(
					OutputMeteringTabStack
					->SetSizeCoefficient(0.25f)
				)
			)
		);
	}

	void FDashboardFactory::RegisterTabSpawners()
	{
		using namespace DashboardFactoryPrivate;

		DashboardWorkspace = DashboardTabManager->AddLocalWorkspaceMenuCategory(ToolName);

		for (const auto& KVP : DashboardViewFactories)
		{
			const FName& FactoryName = KVP.Key;
			const TSharedPtr<IDashboardViewFactory>& Factory = KVP.Value;

			DashboardTabManager->RegisterTabSpawner(FactoryName, FOnSpawnTab::CreateLambda([this, Factory](const FSpawnTabArgs& Args)
			{
				TSharedRef<SDockTab> DockTab = SNew(SDockTab)
					.Clipping(EWidgetClipping::ClipToBounds)
					.Label(Factory->GetDisplayName());

				TSharedRef<SWidget> DashboardView = Factory->MakeWidget(DockTab, Args);
				DockTab->SetContent(DashboardView);

				return DockTab;
			}))
			.SetDisplayName(Factory->GetDisplayName())
			.SetGroup(DashboardWorkspace->AsShared())
			.SetIcon(Factory->GetIcon());
		}
	}

	void FDashboardFactory::RegisterViewFactory(TSharedRef<IDashboardViewFactory> InFactory)
	{
		if (const FName Name = InFactory->GetName(); 
			ensureAlwaysMsgf(!DashboardViewFactories.Contains(Name), TEXT("Failed to register Audio Insights Dashboard '%s': Dashboard with name already registered"), *Name.ToString()))
		{
			DashboardViewFactories.Add(Name, InFactory);
		}
	}

	void FDashboardFactory::UnregisterTabSpawners()
	{
		if (DashboardTabManager.IsValid())
		{
			for (const auto& KVP : DashboardViewFactories)
			{
				const FName& FactoryName = KVP.Key;
				DashboardTabManager->UnregisterTabSpawner(FactoryName);
			}

			DashboardTabManager.Reset();
		}

		DashboardWorkspace.Reset();
	}

	void FDashboardFactory::UnregisterViewFactory(FName InName)
	{
		DashboardViewFactories.Remove(InName);
	}

#if !WITH_EDITOR
	TSharedRef<SWidget> FDashboardFactory::MakeEnableTracesOverlay()
	{
		using namespace DashboardFactoryPrivate;

		return SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.8f))
			.Visibility_Lambda([]() -> EVisibility
			{
				FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr();
				if (AudioInsightsModulePtr == nullptr)
				{
					return EVisibility::Hidden;
				}

				IAudioInsightsTraceModule& TraceModule = AudioInsightsModulePtr->GetTraceModule();
				return TraceModule.AudioChannelsCanBeManuallyEnabled() ? EVisibility::Visible : EVisibility::Hidden;
			})
			[
				SNew(SVerticalBox)
					
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				[
					SNullWidget::NullWidget
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([]()
					{
						FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr();
						if (AudioInsightsModulePtr == nullptr)
						{
							return FText::GetEmpty();
						}

						IAudioInsightsTraceModule& TraceModule = AudioInsightsModulePtr->GetTraceModule();
						return TraceModule.TraceControllerIsAvailable() ? NoTracesEnabledWarning : TraceControllerUnavailabledWarning;
					})
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([]()
					{
						FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr();
						if (AudioInsightsModulePtr == nullptr)
						{
							return FText::GetEmpty();
						}

						IAudioInsightsTraceModule& TraceModule = AudioInsightsModulePtr->GetTraceModule();
						return TraceModule.TraceControllerIsAvailable() ? EnableThemNowText : TryEnablingMessagingText;
					})
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 10.0f)
				[
					MakeEnableTracesButton()
				]

				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				[
					SNullWidget::NullWidget
				]
			];
	}

	TSharedRef<SWidget> FDashboardFactory::MakeEnableTracesButton()
	{
		using namespace DashboardFactoryPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.OnClicked(this, &FDashboardFactory::ToggleAutoEnableAudioTraces)
			.ToolTipText(EnableTracesDescription)
			.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
			.Visibility_Lambda([]() -> EVisibility
			{
				FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr();
				if (AudioInsightsModulePtr == nullptr)
				{
					return EVisibility::Hidden;
				}

				IAudioInsightsTraceModule& TraceModule = AudioInsightsModulePtr->GetTraceModule();

				return TraceModule.TraceControllerIsAvailable() ? EVisibility::Visible : EVisibility::Hidden;
			})
			.Content()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
					.Justification(ETextJustify::Center)
					.Text(EnableTracesButtonText)
				]
			];
	}

	FReply FDashboardFactory::ToggleAutoEnableAudioTraces()
	{
		IAudioInsightsTraceModule& TraceModule = FAudioInsightsModule::GetChecked().GetTraceModule();
		TraceModule.StartTraceAnalysis(false, ETraceMode::Recording);

		return FReply::Handled();
	}
#endif // !WITH_EDITOR

	void FDashboardFactory::CreateTraceBookmark()
	{
		using namespace DashboardFactoryPrivate;

		const FString BookmarkLabel = FString::Printf(TEXT("%s %d"), *BookmarkLabelText.ToString(), BookmarkIndex);

		TRACE_BOOKMARK(TEXT("%s"), *BookmarkLabel);

		FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		const double CurrentTimestamp = CacheManager.GetCacheEndTimeStamp();
		CacheManager.AddMessageToCache(MakeShared<FBookmarkTraceMessage>(CurrentTimestamp, FString(BookmarkLabel)));

		BookmarkIndex++;
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
