// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsDetailsSelectionManager.h"

#if WITH_EDITOR

namespace UE::Audio::Insights
{
	void FAudioInsightsDetailsSelectionManager::SetSelectedAsset(TObjectPtr<UObject> InAsset)
	{
		if (SelectedAsset.Get() != InAsset)
		{
			SelectedAsset = InAsset;
			OnAssetSelectionChanged.Broadcast(InAsset);
		}
	}

	void FAudioInsightsDetailsSelectionManager::ClearSelection()
	{
		SetSelectedAsset(nullptr);
	}

	TWeakObjectPtr<UObject> FAudioInsightsDetailsSelectionManager::GetSelectedAsset() const
	{
		return SelectedAsset;
	}
} // namespace UE::Audio::Insights

#endif // WITH_EDITOR
