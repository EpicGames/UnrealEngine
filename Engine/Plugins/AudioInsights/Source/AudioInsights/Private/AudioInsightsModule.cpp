// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsModule.h"

#include "AudioInsightsDashboardAssetCommands.h"
#include "AudioInsightsDashboardFactory.h"
#include "AudioInsightsLog.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceModule.h"
#include "Features/IModularFeatures.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "TraceServices/ModuleService.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "AssetEditorCommands.h"
#include "AudioInsightsSettings.h"
#endif // WITH_EDITOR

#if !WITH_EDITOR
#include "AudioInsightsComponent.h"
#include "Views/AudioBusDashboardViewFactory.h"
#include "Views/AudioEventLogDashboardViewFactory.h"
#include "Views/AudioMetersPanelDashboardViewFactory.h"
#include "Views/OutputMeterDashboardViewFactory.h"
#include "Views/SignalFlowDashboardViewFactory.h"
#include "Views/SoundDashboardViewFactory.h"
#include "Views/SoundPlotsDashboardViewFactory.h"
#include "Views/SubmixDashboardViewFactory.h"
#include "Views/VirtualLoopDashboardViewFactory.h"
#endif // !WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"
DEFINE_LOG_CATEGORY(LogAudioInsights);


namespace UE::Audio::Insights
{
	void FAudioInsightsModule::StartupModule()
	{
		// Don't run providers in any commandlet to avoid additional, unnecessary overhead as audio insights is dormant.
		if (!IsRunningCommandlet())
		{
#if WITH_EDITOR
			// Re-load settings from engine-wide config so they persist across projects
			GetMutableDefault<UAudioInsightsSettings>()->LoadConfig(nullptr, *UAudioInsightsSettings::GetAudioInsightsConfigFilename());
#endif

			TraceModule = MakeUnique<FTraceModule>();
			RewindDebuggerExtension = MakeUnique<FRewindDebuggerAudioInsightsRuntime>();

			IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
			IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerExtension.Get());

			AudioInsightsTimingViewExtender.BindToTraceModule(*TraceModule);

#if !WITH_EDITOR
			AudioInsightsComponent = FAudioInsightsComponent::CreateInstance();

			IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
			UnrealInsightsModule.RegisterComponent(AudioInsightsComponent);
#endif // !WITH_EDITOR

			DashboardFactory = MakeShared<FDashboardFactory>();
			FDashboardAssetCommands::Register();

#if WITH_EDITOR
			FAssetEditorCommands::Register();
#endif // WITH_EDITOR

			CacheManager = MakeUnique<FAudioInsightsCacheManager>();
			AudioInsightsTimingViewExtender.BindToCacheManager(*CacheManager);

#if !WITH_EDITOR
			IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &AudioInsightsTimingViewExtender);

			TSharedRef<FSoundDashboardViewFactory> SoundsDashboard = MakeShared<FSoundDashboardViewFactory>();
			TSharedRef<FSoundPlotsDashboardViewFactory> PlotsDashboard = MakeShared<FSoundPlotsDashboardViewFactory>();
			TSharedRef<FSubmixDashboardViewFactory> SubmixDashboard = MakeShared<FSubmixDashboardViewFactory>();
			EventLogViewFactory = MakeShared<FAudioEventLogDashboardViewFactory>();

			DashboardFactory->RegisterViewFactory(EventLogViewFactory.ToSharedRef());
			DashboardFactory->RegisterViewFactory(SoundsDashboard);
			DashboardFactory->RegisterViewFactory(MakeShared<FVirtualLoopDashboardViewFactory>());
			DashboardFactory->RegisterViewFactory(PlotsDashboard);
			DashboardFactory->RegisterViewFactory(SubmixDashboard);
			DashboardFactory->RegisterViewFactory(MakeShared<FAudioBusDashboardViewFactory>());
			DashboardFactory->RegisterViewFactory(MakeShared<FAudioMetersPanelDashboardViewFactory>());
			DashboardFactory->RegisterViewFactory(MakeShared<FOutputMeterDashboardViewFactory>(SubmixDashboard));
			DashboardFactory->RegisterViewFactory(MakeShared<FSignalFlowDashboardViewFactory>());

			PlotsDashboard->InitPlots(SoundsDashboard);
#endif // !WITH_EDITOR

			FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this]
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/AudioInsights"));
				IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
				if (!UnrealInsightsModule.GetStoreClient())
				{
					UE_LOGF(LogCore, Display, "AudioInsights module auto-connecting to local trace server...");
					UnrealInsightsModule.ConnectToStore(TEXT("127.0.0.1"));
					UnrealInsightsModule.CreateSessionViewer(false);
				}
			});
		}
	}

	void FAudioInsightsModule::ShutdownModule()
	{
		if (!IsRunningCommandlet())
		{
#if !WITH_EDITOR
			IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
			UnrealInsightsModule.UnregisterComponent(AudioInsightsComponent);

			AudioInsightsComponent.Reset();

			EventLogViewFactory.Reset();

			IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &AudioInsightsTimingViewExtender);
#endif // !WITH_EDITOR

			FDashboardAssetCommands::Unregister();

#if WITH_EDITOR
			FAssetEditorCommands::Unregister();
#endif // WITH_EDITOR

			DashboardFactory.Reset();

			AudioInsightsTimingViewExtender.UnbindFromTraceModule(*TraceModule);

			IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
			TraceModule.Reset();

			IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerExtension.Get());
			RewindDebuggerExtension.Reset();

			AudioInsightsTimingViewExtender.UnbindFromCacheManager(*CacheManager);
			CacheManager.Reset();
		}
	}

	void FAudioInsightsModule::RegisterDashboardViewFactory(TSharedRef<IDashboardViewFactory> InDashboardFactory)
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		if (DashboardFactory.IsValid())
		{
			DashboardFactory->RegisterViewFactory(InDashboardFactory);
		}
	}

	void FAudioInsightsModule::UnregisterDashboardViewFactory(FName InName)
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		if (DashboardFactory.IsValid())
		{
			DashboardFactory->UnregisterViewFactory(InName);
		}
	}

	void FAudioInsightsModule::RegisterEventLogCategories(const TMap<FString, TSet<FString>>& InCategoriesToEvents)
	{
#if !WITH_EDITOR
		if (EventLogViewFactory.IsValid())
		{
			EventLogViewFactory->RegisterExternalEventTypes(InCategoriesToEvents);
		}
#endif // !WITH_EDITOR
	}

	void FAudioInsightsModule::RegisterEventLogDisplayNames(const TMap<FString, FText>& InDisplayNames)
	{
#if !WITH_EDITOR
		if (EventLogViewFactory.IsValid())
		{
			EventLogViewFactory->RegisterExternalDisplayNames(InDisplayNames);
		}
#endif // !WITH_EDITOR
	}

	::Audio::FDeviceId FAudioInsightsModule::GetDeviceId() const
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		return DashboardFactory->GetDeviceId();
	}

	bool FAudioInsightsModule::IsModuleLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded("AudioInsights");
	}

	FAudioInsightsModule& FAudioInsightsModule::GetChecked()
	{
		ensureMsgf(IsInGameThread(), TEXT("Do not call this from outside of the Game Thread, if the module is not loaded this will fail - use FAudioInsightsModule::GetModulePtr() instead."));

		return static_cast<FAudioInsightsModule&>(FModuleManager::LoadModuleChecked<IAudioInsightsModule>("AudioInsights"));
	}

	FAudioInsightsModule* FAudioInsightsModule::GetModulePtr()
	{
		if (IAudioInsightsModule* AudioInsightsModulePtr = FModuleManager::LoadModulePtr<IAudioInsightsModule>("AudioInsights"))
		{
			return static_cast<FAudioInsightsModule*>(AudioInsightsModulePtr);
		}

		return nullptr;
	}

	TSharedRef<FDashboardFactory> FAudioInsightsModule::GetDashboardFactory()
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		return DashboardFactory->AsShared();
	}

	const TSharedRef<FDashboardFactory> FAudioInsightsModule::GetDashboardFactory() const
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		return DashboardFactory->AsShared();
	}

	IAudioInsightsTraceModule& FAudioInsightsModule::GetTraceModule()
	{
		check(!IsRunningCommandlet() && TraceModule.IsValid());

		return *TraceModule;
	}

	FAudioInsightsCacheManager& FAudioInsightsModule::GetCacheManager()
	{
		check(!IsRunningCommandlet() && CacheManager.IsValid());

		return *CacheManager;
	}

	TSharedRef<SDockTab> FAudioInsightsModule::CreateDashboardTabWidget(const FSpawnTabArgs& Args)
	{
		check(!IsRunningCommandlet() && DashboardFactory.IsValid());

		return DashboardFactory->MakeDockTabWidget(Args);
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE // AudioInsights

IMPLEMENT_MODULE(UE::Audio::Insights::FAudioInsightsModule, AudioInsights)
