// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "AudioInsightsDashboardFactory.h"
#include "AudioInsightsDetailsSelectionManager.h"
#include "AudioInsightsTimingViewExtender.h"
#include "AudioInsightsTraceModule.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsModule.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/Docking/SDockTab.h"


namespace UE::Audio::Insights
{
	class FAudioEventLogDashboardViewFactory;

#if !WITH_EDITOR
	class FAudioInsightsComponent;
#endif // !WITH_EDITOR

	class FAudioInsightsModule final : public IAudioInsightsModule
	{
	public:
		FAudioInsightsModule() = default;
		virtual ~FAudioInsightsModule() = default;

		FAudioInsightsModule(const FAudioInsightsModule&) = delete;
		FAudioInsightsModule& operator=(const FAudioInsightsModule&) = delete;
		FAudioInsightsModule(FAudioInsightsModule&&) = delete;
		FAudioInsightsModule& operator=(FAudioInsightsModule&&) = delete;

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		virtual void RegisterDashboardViewFactory(TSharedRef<IDashboardViewFactory> InDashboardFactory) override;
		virtual void UnregisterDashboardViewFactory(FName InName) override;
		virtual void RegisterEventLogCategories(const TMap<FString, TSet<FString>>& InCategoriesToEvents) override;
		virtual void RegisterEventLogDisplayNames(const TMap<FString, FText>& InDisplayNames) override;

		virtual ::Audio::FDeviceId GetDeviceId() const override;

		static bool IsModuleLoaded();
		static FAudioInsightsModule& GetChecked();
		static FAudioInsightsModule* GetModulePtr();
		virtual IAudioInsightsTraceModule& GetTraceModule() override;
		virtual class FAudioInsightsCacheManager& GetCacheManager() override;

#if !WITH_EDITOR
		TSharedPtr<FAudioInsightsComponent> GetAudioInsightsComponent() { return AudioInsightsComponent; };
#endif // !WITH_EDITOR

		FAudioInsightsTimingViewExtender& GetTimingViewExtender() { return AudioInsightsTimingViewExtender; };
		const FAudioInsightsTimingViewExtender& GetTimingViewExtender() const { return AudioInsightsTimingViewExtender; };

#if WITH_EDITOR
		FAudioInsightsDetailsSelectionManager& GetDetailsSelectionManager() override { return DetailsSelectionManager; };
#endif // WITH_EDITOR

		TSharedRef<FDashboardFactory> GetDashboardFactory();
		const TSharedRef<FDashboardFactory> GetDashboardFactory() const;

		virtual TSharedRef<SDockTab> CreateDashboardTabWidget(const FSpawnTabArgs& Args) override;

	private:
		TSharedPtr<FDashboardFactory> DashboardFactory;
#if !WITH_EDITOR
		TSharedPtr<FAudioEventLogDashboardViewFactory> EventLogViewFactory;
#endif // !WITH_EDITOR
		TUniquePtr<FTraceModule> TraceModule;
		TUniquePtr<FRewindDebuggerAudioInsightsRuntime> RewindDebuggerExtension;
		TUniquePtr<class FAudioInsightsCacheManager> CacheManager;

#if !WITH_EDITOR
		TSharedPtr<FAudioInsightsComponent> AudioInsightsComponent;
#endif // !WITH_EDITOR

		FAudioInsightsTimingViewExtender AudioInsightsTimingViewExtender;

#if WITH_EDITOR
		FAudioInsightsDetailsSelectionManager DetailsSelectionManager;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights
