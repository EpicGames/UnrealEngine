// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsSettings.h"

#if WITH_EDITOR
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#endif // WITH_EDITOR

FString UAudioInsightsSettings::GetAudioInsightsConfigFilename()
{
#if WITH_EDITOR
	const FName ConfigName = UAudioInsightsSettings::StaticClass()->ClassConfigName;
	return FConfigCacheIni::GetDestIniFilename(*ConfigName.ToString(), nullptr, *FPaths::EngineEditorSettingsDir());
#else
	return GetConfigFilename(GetMutableDefault<UAudioInsightsSettings>());
#endif
}

#if WITH_EDITOR
FText UAudioInsightsSettings::GetSectionText() const
{
	return NSLOCTEXT("UAudioInsightsSettings", "SectionText", "Audio Insights");
}

void UAudioInsightsSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* const ActiveMemberNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	if (ActiveMemberNode == nullptr || ActiveMemberNode->GetValue() == nullptr)
	{
		return;
	}

	SavePropertyToConfigFile(ActiveMemberNode->GetValue());
}

void UAudioInsightsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// MemberProperty is null when triggered via Reset to Defaults whilst ChangeType is unspecified, so check for both cases
	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.ChangeType == EPropertyChangeType::ResetToDefault ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAudioInsightsSettings, CacheSettings))
	{
		SaveCacheSettings();
	}
}
#endif

void UAudioInsightsSettings::SaveCacheSettings()
{
	const FProperty* const CacheSettingProperty = UAudioInsightsSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAudioInsightsSettings, CacheSettings));
	ensure(CacheSettingProperty);

	SavePropertyToConfigFile(CacheSettingProperty);

	FCacheSettings::OnCacheSizeSettingsChanged.Broadcast();
}

void UAudioInsightsSettings::SavePropertyToConfigFile(const FProperty* const PropertyToSave)
{
	if (PropertyToSave == nullptr)
	{
		return;
	}
	
	// Save setting to config file
	UpdateSinglePropertyInConfigFile(PropertyToSave, GetAudioInsightsConfigFilename());
}
