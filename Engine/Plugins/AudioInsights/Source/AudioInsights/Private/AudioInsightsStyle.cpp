// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInsightsStyle.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Audio::Insights
{
	FSlateStyle& FSlateStyle::Get()
	{
		static FSlateStyle InsightsStyle;
		return InsightsStyle;
	}

	FText FSlateStyle::FormatTimestamp(const double InSeconds) const
	{
		const double AbsSeconds = FMath::Abs(InSeconds);
		const int32 TotalSeconds = FMath::FloorToInt32(AbsSeconds);
		const int32 Hours = TotalSeconds / 3600;
		const int32 Minutes = (TotalSeconds / 60) % 60;
		const int32 Seconds = TotalSeconds % 60;
		const int32 Millis = FMath::FloorToInt32((AbsSeconds - static_cast<double>(TotalSeconds)) * 1000.0);
		const TCHAR* const Sign = (InSeconds < 0.0) ? TEXT("-") : TEXT("");
		return FText::FromString(FString::Printf(TEXT("%s%02d:%02d:%02d.%03d"), Sign, Hours, Minutes, Seconds, Millis));
	}

	TSharedRef<SBox> FSlateStyle::CreateButtonContentWidget(const FName& IconName /*= FName()*/, const FText& Label /*= FText::GetEmpty()*/, const FName& TextStyle /*= TEXT("ButtonText")*/, const float HeightOverride /*= 16.0f*/) const
	{
		TSharedRef<SHorizontalBox> ButtonContainerWidget = SNew(SHorizontalBox);

		// Button icon (optional)
		if (!IconName.IsNone())
		{
			ButtonContainerWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FSlateStyle::Get().GetBrush(IconName))
			];
		}

		// Button text (optional)
		if (!Label.IsEmpty())
		{
			const float LeftPadding = IconName.IsNone() ? 0.0f : 4.0f;

			ButtonContainerWidget->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(LeftPadding, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(&FSlateStyle::Get().GetWidgetStyle<FTextBlockStyle>(TextStyle))
					.Justification(ETextJustify::Center)
					.Text(Label)
				];
		}

		return SNew(SBox)
		.HeightOverride(HeightOverride)
		[
			ButtonContainerWidget
		];
	};

	TSharedRef<SBox> FSlateStyle::CreateToggleButtonContent(const FName& IconName, const FName& TextStyle, TFunction<FSlateColor()> ColorFunction, TFunction<FText()> TextFunction, const float HeightOverride /*= 16.0f*/) const
	{
		TSharedRef<SHorizontalBox> ButtonContainerWidget = SNew(SHorizontalBox);

		if (!IconName.IsNone())
		{
			ButtonContainerWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity_Lambda([ColorFunction]()
				{ 
					return ColorFunction();
				})
				.Image(FSlateStyle::Get().GetBrush(IconName))
			];
		}

		const float LeftPadding = IconName.IsNone() ? 0.0f : 4.0f;

		ButtonContainerWidget->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(LeftPadding, 0.0f, 0.0f, 0.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(&FSlateStyle::Get().GetWidgetStyle<FTextBlockStyle>(TextStyle))
			.Justification(ETextJustify::Center)
			.Text_Lambda([TextFunction]()
			{ 
				return TextFunction();
			})
		];

		return SNew(SBox)
		.HeightOverride(HeightOverride)
		[
			ButtonContainerWidget
		];
	};
}
