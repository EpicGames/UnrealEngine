// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsTimingViewExtender.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsSettings.h"
#include "AudioInsightsTraceModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Insights/ITimingViewSession.h"
#include "TraceServices/Model/AnalysisSession.h"

#if !WITH_EDITOR
#include "AudioInsightsComponent.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	void FAudioInsightsTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		if (TimingView == nullptr)
		{
			TimingView = &InSession;
		}

		SystemControllingTime.Reset();

		InSession.OnTimeMarkerChanged().AddRaw(this, &FAudioInsightsTimingViewExtender::OnTimeMarkerChanged);
	}

	void FAudioInsightsTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		InSession.OnTimeMarkerChanged().RemoveAll(this);

		TimingView = nullptr;
	}

	void FAudioInsightsTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		AnalysisSession = &InAnalysisSession;

		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);
			TraceDurationSeconds = InAnalysisSession.GetDurationSeconds();
		}
	}

	void FAudioInsightsTimingViewExtender::ResumeTimeMarker()
	{
		SystemControllingTime.Reset();
		PausedTimeMarker = 0.0;

		// Prevent re-triggering broadcast when we're already resumed
		if (CacheAndProcessMessageStatus == ECacheAndProcess::Latest)
		{
			return;
		}

		CacheAndProcessMessageStatus = ECacheAndProcess::Latest;

		OnTimeControlMethodReset.Broadcast();
	}

	void FAudioInsightsTimingViewExtender::StopProcessingNewMessages()
	{
		CacheAndProcessMessageStatus = ECacheAndProcess::CacheLatestNoProcess;
	}

	void FAudioInsightsTimingViewExtender::StopCachingAndProcessingNewMessages()
	{
		CacheAndProcessMessageStatus = ECacheAndProcess::None;
	}

	TRange<double> FAudioInsightsTimingViewExtender::GetPlottingRange() const 
	{
		return PlottingRange;
	}

	double FAudioInsightsTimingViewExtender::GetCurrentTraceDurationSeconds() const
	{
		return TraceDurationSeconds;
	}

	void FAudioInsightsTimingViewExtender::SetPausedTimeMarker(const double InTimeMarker)
	{
		PausedTimeMarker = InTimeMarker;
		ApplyStopCacheWhenPausedBehaviour();
	}

	void FAudioInsightsTimingViewExtender::PauseTimeMarker(const double InTimestamp, const ESystemControllingTimeMarker InControllingSystem, TOptional<TRange<double>> InPlottingRange /*= TOptional<TRange<double>>()*/)
	{
		SystemControllingTime = InControllingSystem;
		bUserInputDetected = true;

		SetPausedTimeMarker(InTimestamp);

		if (InPlottingRange.IsSet())
		{
			PlottingRange = InPlottingRange.GetValue();
		}
		else
		{
			PlottingRange = TRange<double>(InTimestamp - MaxPlottingHistorySeconds, InTimestamp);
		}

		// If you set the time marker on the TimingView, it will Broadcast OnTimingViewTimeMarkerChanged. Otherwise we gotta do it ourselves.
		if (TimingView != nullptr)
		{
			TimingView->SetAndCenterOnTimeMarker(InTimestamp);
		}
		else
		{
			OnTimingViewTimeMarkerChanged.Broadcast(InTimestamp);
		}
	}

	void FAudioInsightsTimingViewExtender::BindToTraceModule(FTraceModule& InTraceModule)
	{
		OnAnalysisStartingHandle = InTraceModule.OnAnalysisStarting.AddRaw(this, &FAudioInsightsTimingViewExtender::OnAnalysisStarting);
	}

	void FAudioInsightsTimingViewExtender::UnbindFromTraceModule(FTraceModule& InTraceModule)
	{
		if (OnAnalysisStartingHandle.IsValid())
		{
			InTraceModule.OnAnalysisStarting.Remove(OnAnalysisStartingHandle);
			OnAnalysisStartingHandle.Reset();
		}
	}

	void FAudioInsightsTimingViewExtender::BindToCacheManager(FAudioInsightsCacheManager& InCacheManager)
	{
		OnCacheChunkOverwrittenHandle = InCacheManager.OnChunkOverwritten.AddRaw(this, &FAudioInsightsTimingViewExtender::OnCacheChunkOverwritten);
	}

	void FAudioInsightsTimingViewExtender::UnbindFromCacheManager(FAudioInsightsCacheManager& InCacheManager)
	{
		if (OnCacheChunkOverwrittenHandle.IsValid())
		{
			InCacheManager.OnChunkOverwritten.Remove(OnCacheChunkOverwrittenHandle);
			OnCacheChunkOverwrittenHandle.Reset();
		}
	}

	void FAudioInsightsTimingViewExtender::OnAnalysisStarting(const double Timestamp)
	{
		ResumeTimeMarker();
	}

	void FAudioInsightsTimingViewExtender::OnTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
	{
		if (AnalysisSession == nullptr)
		{
			bUserInputDetected = false;
			return;
		}
		
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		const bool bIsValidTime = InTimeMarker >= 0.0 && InTimeMarker <= AnalysisSession->GetDurationSeconds();
		if (bIsValidTime)
		{
			if (!bUserInputDetected)
			{
				SystemControllingTime = ESystemControllingTimeMarker::External;
				PlottingRange = TRange<double>(InTimeMarker - MaxPlottingHistorySeconds, InTimeMarker);
			}

			SetPausedTimeMarker(InTimeMarker);

			OnTimingViewTimeMarkerChanged.Broadcast(InTimeMarker);
		}
		else
		{
			SystemControllingTime.Reset();
		}

		bUserInputDetected = false;
	}

	void FAudioInsightsTimingViewExtender::ApplyStopCacheWhenPausedBehaviour()
	{
#if !WITH_EDITOR
		if (!IAudioInsightsModule::IsLiveSession())
		{
			return;
		}
#endif // !WITH_EDITOR

		const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>();

		const EStopCacheWhenPausedBehaviour StopCacheWhenPausedBehaviour = Settings
			? Settings->CacheSettings.StopCacheWhenPausedBehaviour
			: FCacheSettings::DefaultStopCacheWhenPausedBehaviour;

		switch (StopCacheWhenPausedBehaviour)
		{
			case EStopCacheWhenPausedBehaviour::WhenMarkedForDeletion:
			{
				FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

				const bool bPausedTimeAlreadyOverwritten = PausedTimeMarker < CacheManager.GetCacheStartTimeStamp();
				if (bPausedTimeAlreadyOverwritten || CacheManager.IsTimestampMarkedForDeletion(PausedTimeMarker))
				{
					StopCachingAndProcessingNewMessages();
				}
				else if (CacheAndProcessMessageStatus != ECacheAndProcess::None)
				{
					StopProcessingNewMessages();
				}
				break;
			}

			case EStopCacheWhenPausedBehaviour::Always:
			{
				StopCachingAndProcessingNewMessages();
				break;
			}

			case EStopCacheWhenPausedBehaviour::Never:
			{
				StopProcessingNewMessages();
				break;
			}
		}
	}

	void FAudioInsightsTimingViewExtender::OnCacheChunkOverwritten(const double /*NewCacheStartTimestamp*/)
	{
		auto ReapplyCacheBehaviour = [this]()
		{
			const bool bIsPaused = ((SystemControllingTime.IsSet()) && (CacheAndProcessMessageStatus == ECacheAndProcess::CacheLatestNoProcess));
			if (bIsPaused)
			{
				ApplyStopCacheWhenPausedBehaviour();
			}
		};

		if (IsInGameThread())
		{
			ReapplyCacheBehaviour();
		}
		else
		{
			ExecuteOnGameThread(TEXT("FAudioInsightsTimingViewExtender::OnCacheChunkOverwritten"), ReapplyCacheBehaviour);
		}
	}
} // namespace UE::Audio::Insights
