// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsToolbarWidgets.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsSettings.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTimingViewExtender.h"
#include "AudioInsightsTraceModule.h"
#include "Brushes/SlateColorBrush.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAudioInsightsModule.h"
#include "Internationalization/Text.h"
#include "Settings/CacheSettings.h"
#include "SSimpleTimeSlider.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace ToolbarWidgetsPrivate
	{
		static const FText CachePauseTooltipText = LOCTEXT("AudioInsightsToolbar_CachePause_TooltipText", "Click to pause monitoring.\n\nDepending on your cache settings, this may also stop the cache collecting data from the current session.");
		static const FText CacheResumeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheResume_TooltipText", "Click to resume monitoring.");

		static const FText FollowEnabledTooltipText = LOCTEXT("AudioInsightsToolbar_FollowEnabled_TooltipText", "Follow Cache Playhead is enabled.\n\nClick to disable. The timeline will stop following the current time.");
		static const FText FollowDisabledTooltipText = LOCTEXT("AudioInsightsToolbar_FollowDisabled_TooltipText", "Follow Cache Playhead is disabled.\n\nClick to enable. The timeline will auto-scroll to follow the current time when monitoring.");

		static const FText StopCacheTooltipText = LOCTEXT("AudioInsightsToolbar_StopCache_TooltipText", "Click to stop caching and processing new messages.");
		static const FText StopCacheActiveTooltipText = LOCTEXT("AudioInsightsToolbar_StopCacheActive_TooltipText", "Cache is stopped. No new messages are being cached or processed.");
		static const FText ResumeCacheTooltipText = LOCTEXT("AudioInsightsToolbar_ResumeCache_TooltipText", "Click to resume caching and processing new messages.");

		static const FText CacheLabelText = LOCTEXT("AudioInsightsToolbar_CacheLabel", "Cache Controls:");

		static const FText CacheSizeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheStats_CacheSizeTooltip", "Cache Size");
		static const FText CacheStatsMemoryText = LOCTEXT("AudioInsightsToolbar_CacheStats_Memory", "{0} / {1}");
		static const FText CacheStatsDurationText = LOCTEXT("AudioInsightsToolbar_CacheStats_Duration", " : Duration: ");
		static const FText CacheStatsDurationValueText = LOCTEXT("AudioInsightsToolbar_CacheStats_DurationValue", "{0}s");

		static const FText CacheStatsTimestampValueText = LOCTEXT("AudioInsightsToolbar_CacheStats_TimestampValue", "{0}");
		static const FText CacheStatsCurrentTimeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheStats_CurrentTimeTooltip", "Cache Playhead Time");
		static const FText CacheStatsCacheBeginTimeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheStats_CacheBeginTimeTooltip", "Cache Begin Time");
		static const FText CacheStatsCacheEndTimeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheStats_CacheEndTimeTooltip", "Cache End Time");

		static const FText CacheSettingsHeaderText = LOCTEXT("AudioInsightsToolbar_CacheSettings_HeaderText", "Cache Settings");
		static const FText CacheSettingsShowStatsText = LOCTEXT("AudioInsightsToolbar_CacheSettings_ShowCacheStats", "Show Cache Stats");
		static const FText CacheSettingsShowStatsTooltipText = LOCTEXT("AudioInsightsToolbar_CacheSettings_ShowCacheStatsTooltip", "Shows/Hides Cache Stats");
		static const FText CacheSettingsCacheSizeText = LOCTEXT("AudioInsightsToolbar_CacheSettings_CacheSize", "Cache Size (MB)");
		static const FText CacheSettingsCacheSizeTooltipText = LOCTEXT("AudioInsightsToolbar_CacheSettings_CacheSizeTooltip", "Maximum cache size in megabytes (default: 32). Changing this value will reset the current cache.");

		static const FText CacheSettingsNudgeStepText = LOCTEXT("AudioInsightsToolbar_CacheSettings_NudgeStep", "Cache Nudge Step (s)");
		static const FText CacheSettingsNudgeStepTooltipText = LOCTEXT("AudioInsightsToolbar_CacheSettings_NudgeStepTooltip", "Amount in seconds the cache playhead moves when nudged back or forward (default: 0.5).");

		static const FText CacheSettingsStopCacheWhenPausedText = LOCTEXT("AudioInsightsToolbar_CacheSettings_StopCacheWhenPaused", "Stop Cache When Paused");
		static const FText CacheSettingsStopCacheWhenPausedTooltipText = LOCTEXT("AudioInsightsToolbar_CacheSettings_StopCacheWhenPausedTooltip", "Controls whether the cache stops collecting new data when paused.");

		static const FText StopCacheWhenPaused_WhenMarkedForDeletionText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_WhenMarkedForDeletion", "When marked for deletion");
		static const FText StopCacheWhenPaused_WhenMarkedForDeletionTooltipText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_WhenMarkedForDeletionTooltip", "Stops caching new data when the cache playhead is paused in the cache region marked for deletion.");

		static const FText StopCacheWhenPaused_AlwaysText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_Always", "Always");
		static const FText StopCacheWhenPaused_AlwaysTooltipText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_AlwaysTooltip", "Always stops caching new data when paused.");

		static const FText StopCacheWhenPaused_NeverText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_Never", "Never");
		static const FText StopCacheWhenPaused_NeverTooltipText = LOCTEXT("AudioInsightsToolbar_StopCacheWhenPaused_NeverTooltip", "Never stops caching new data. When the cache playhead is paused in the cache region being deleted, the cache will resume.");

		static const FText NudgeBackTooltipText = LOCTEXT("AudioInsightsToolbar_NudgeBack_TooltipText", "Nudge cache playhead back");
		static const FText NudgeForwardTooltipText = LOCTEXT("AudioInsightsToolbar_NudgeForward_TooltipText", "Nudge cache playhead forward");

		static const FText TimelineRulerTooltipText = LOCTEXT("AudioInsightsToolbar_TimelineRuler_TooltipText", "Timeline ruler. Click-drag handle to scrub time. Ctrl+Scroll to zoom. Right-click drag to pan.");

		static const FText CacheSettingsTooltipText = LOCTEXT("AudioInsightsToolbar_CacheSettings_TooltipText", "Cache Settings");

		double GetNudgeStepSeconds()
		{
			const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>();
			const float RawValue = Settings ? Settings->CacheSettings.NudgeStepSeconds : FCacheSettings::DefaultNudgeStepSeconds;

			return FMath::Clamp(RawValue, FCacheSettings::MinNudgeStepSeconds, FCacheSettings::MaxNudgeStepSeconds);
		}

		// Intentionally override SSimpleTimeSlider, as the internal Background and ScrubHandle colours/brushes are 'Editor-only'
		// Overriding here, allows our cache controls within Standalone to be rendered correctly without any missing textures
		class SAudioInsightsTimeSlider final : public SSimpleTimeSlider
		{
		public:
			void Construct(const FArguments& InArgs)
			{
				SSimpleTimeSlider::Construct(InArgs);

				// Using White base, allows for TimeSlider Background and ScrubHandle tint colour to stay visible
				static const TUniquePtr<FSlateColorBrush> BackgroundOverride = MakeUnique<FSlateColorBrush>(FStyleColors::White);
				static const TUniquePtr<FSlateColorBrush> ScrubHandleOverride = MakeUnique<FSlateColorBrush>(FLinearColor::White);

				CursorBackground = BackgroundOverride.Get();
				ScrubHandleUp    = ScrubHandleOverride.Get();
				ScrubHandleDown  = ScrubHandleOverride.Get();
			}
		};

	} // namespace ToolbarWidgetsPrivate

	FToolbarWidgets::FToolbarWidgets()
	{
		InitializeDelegates();
	}

	FToolbarWidgets::~FToolbarWidgets()
	{
		DeinitializeDelegates();
	}

	void FToolbarWidgets::InitializeDelegates()
	{
		FTraceModule& TraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
		OnAnalysisStartingHandle = TraceModule.OnAnalysisStarting.AddRaw(this, &FToolbarWidgets::OnAnalysisStarting);

		FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();
		OnTimingViewTimeMarkerChangedHandle = TimingViewExtender.OnTimingViewTimeMarkerChanged.AddRaw(this, &FToolbarWidgets::OnTimingViewTimeMarkerChanged);
		OnTimeControlMethodResetHandle = TimingViewExtender.OnTimeControlMethodReset.AddRaw(this, &FToolbarWidgets::OnTimeControlMethodReset);
	}

	void FToolbarWidgets::DeinitializeDelegates()
	{
		FTraceModule& TraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());

		if (OnAnalysisStartingHandle.IsValid())
		{
			TraceModule.OnAnalysisStarting.Remove(OnAnalysisStartingHandle);
			OnAnalysisStartingHandle.Reset();
		}

		FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

		if (OnTimingViewTimeMarkerChangedHandle.IsValid())
		{
			TimingViewExtender.OnTimingViewTimeMarkerChanged.Remove(OnTimingViewTimeMarkerChangedHandle);
			OnTimingViewTimeMarkerChangedHandle.Reset();
		}

		if (OnTimeControlMethodResetHandle.IsValid())
		{
			TimingViewExtender.OnTimeControlMethodReset.Remove(OnTimeControlMethodResetHandle);
			OnTimeControlMethodResetHandle.Reset();
		}
	}

	double FToolbarWidgets::GetCurrentRelativeTime() const
	{
		if (bIsPaused)
		{
			return PausedTimeMarker - BeginTimestamp;
		}

		if (!IAudioInsightsModule::IsLiveSession())
		{
			return 0.0;
		}

		const FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
		return FMath::Max(0.0, CacheManager.GetCacheEndTimeStamp() - BeginTimestamp);
	}

	void FToolbarWidgets::OnAnalysisStarting(const double Timestamp)
	{
#if WITH_EDITOR
		BeginTimestamp = Timestamp - GStartTime;
#else
		BeginTimestamp = 0.0;
#endif // WITH_EDITOR

		TimelineViewRange = TRange<double>(0.0, 10.0);
		bTimelineViewRangeSetByUser = false;
	}

	void FToolbarWidgets::OnTimingViewTimeMarkerChanged(double InTimeMarker)
	{
		// Only center the view on the first pause, not during scrubbing
		if (!bIsPaused)
		{
			const double RelativeTime = InTimeMarker - BeginTimestamp;
			const double HalfViewSize = TimelineViewRange.Size<double>() * 0.5;
			TimelineViewRange = TRange<double>(RelativeTime - HalfViewSize, RelativeTime + HalfViewSize);
			bTimelineViewRangeSetByUser = true;
		}

		bIsPaused = true;
		PausedTimeMarker = InTimeMarker;
	}

	void FToolbarWidgets::OnTimeControlMethodReset()
	{
		bIsPaused = false;
		bTimelineAutoScroll = true;
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheLabelWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(STextBlock)
			.Justification(ETextJustify::Center)
			.Text(CacheLabelText);
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCachePauseButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.IsEnabled_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession();
			})
			.ToolTipText_Lambda([this]()
			{
				return bIsPaused ? CacheResumeTooltipText : CachePauseTooltipText;
			})
			.OnClicked_Lambda([this]()
			{
				FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

				if (bIsPaused)
				{
					TimingViewExtender.ResumeTimeMarker();
				}
				else
				{
					const double PauseTime = IAudioInsightsModule::GetChecked().GetCacheManager().GetCacheEndTimeStamp();
					TimingViewExtender.PauseTimeMarker(PauseTime, ESystemControllingTimeMarker::External);
				}

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CachePause"))
						.ColorAndOpacity_Lambda([this]()
						{
							return bIsPaused ? FStyleColors::AccentBlue : FSlateColor::UseForeground();
						})
					]
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheStopButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.IsEnabled_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession();
			})
			.ToolTipText_Lambda([this]()
			{
				const ECacheAndProcess Status = IAudioInsightsModule::GetChecked().GetTimingViewExtender().GetMessageCacheAndProcessingStatus();
				return Status == ECacheAndProcess::None ? StopCacheActiveTooltipText : StopCacheTooltipText;
			})
			.OnClicked_Lambda([]()
			{
				FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

				const double PauseTime = IAudioInsightsModule::GetChecked().GetCacheManager().GetCacheEndTimeStamp();
				TimingViewExtender.PauseTimeMarker(PauseTime, ESystemControllingTimeMarker::External);
				TimingViewExtender.StopCachingAndProcessingNewMessages();

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CacheStop"))
						.ColorAndOpacity_Lambda([]()
						{
							const ECacheAndProcess Status = IAudioInsightsModule::GetChecked().GetTimingViewExtender().GetMessageCacheAndProcessingStatus();
							return Status == ECacheAndProcess::None ? FStyleColors::AccentBlue : FSlateColor::UseForeground();
						})
					]
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheResumeButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.IsEnabled_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession();
			})
			.ToolTipText_Lambda([this]()
			{
				return bIsPaused ? ResumeCacheTooltipText : CachePauseTooltipText;
			})
			.OnClicked_Lambda([this]()
			{
				FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

				if (bIsPaused)
				{
					TimingViewExtender.ResumeTimeMarker();
				}
				else
				{
					const double PauseTime = IAudioInsightsModule::GetChecked().GetCacheManager().GetCacheEndTimeStamp();
					TimingViewExtender.PauseTimeMarker(PauseTime, ESystemControllingTimeMarker::External);
				}

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CacheResume"))
						.ColorAndOpacity_Lambda([]()
						{
							const ECacheAndProcess Status = IAudioInsightsModule::GetChecked().GetTimingViewExtender().GetMessageCacheAndProcessingStatus();
							return Status == ECacheAndProcess::Latest ? FStyleColors::AccentBlue : FSlateColor::UseForeground();
						})
					]
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheFollowButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.Visibility_Lambda([]()
			{
				return IAudioInsightsModule::IsLiveSession() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.ToolTipText_Lambda([this]()
			{
				return bTimelineAutoScroll ? FollowEnabledTooltipText : FollowDisabledTooltipText;
			})
			.OnClicked_Lambda([this]()
			{
				bTimelineAutoScroll = !bTimelineAutoScroll;

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CacheFollowPlayhead"))
						.ColorAndOpacity_Lambda([this]()
						{
							return bTimelineAutoScroll ? FStyleColors::AccentBlue : FSlateColor::UseForeground();
						})
					]
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheCurrentTimestampWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SBox)
			.MinDesiredWidth(80.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.ColorAndOpacity(FStyleColors::AccentBlue)
					.ToolTipText(CacheStatsCurrentTimeTooltipText)
					.Text_Lambda([this]()
					{
						const double CurrentTime = GetCurrentRelativeTime();

						return FText::Format(CacheStatsTimestampValueText, FSlateStyle::Get().FormatTimestamp(CurrentTime));
					})
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheBeginTimestampWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SBox)
			.MinDesiredWidth(80.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.ToolTipText(CacheStatsCacheBeginTimeTooltipText)
					.Text_Lambda([this]()
					{
						FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
						const double BeginTime = CacheManager.GetCacheStartTimeStamp() - BeginTimestamp;

						return FText::Format(CacheStatsTimestampValueText, FSlateStyle::Get().FormatTimestamp(BeginTime));
					})
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheEndTimestampWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SBox)
			.MinDesiredWidth(80.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.ToolTipText(CacheStatsCacheEndTimeTooltipText)
					.Text_Lambda([this]()
					{
						FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
						const double EndTime = CacheManager.GetCacheEndTimeStamp() - BeginTimestamp;

						return FText::Format(CacheStatsTimestampValueText, FSlateStyle::Get().FormatTimestamp(EndTime));
					})
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheSizeAndDurationWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		constexpr FLinearColor CacheStatsColor(1.0f, 1.0f, 1.0f, 0.5f);

		const auto CacheStatsVisibilityLambda = []()
		{
			if (!IAudioInsightsModule::IsLiveSession())
			{
				return EVisibility::Collapsed;
			}

			if (const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>())
			{
				const FCacheSettings& CacheSettings = Settings->CacheSettings;
				return CacheSettings.bShowCacheStats ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		};

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.ToolTipText(CacheSizeTooltipText)
				.Text_Lambda([]()
				{
					FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
					const float CacheSizeMb = static_cast<float>(CacheManager.GetUsedCacheSize()) / static_cast<float>(1 << 20);
					const float MaxCacheSizeMb = static_cast<float>(CacheManager.GetMaxCacheSize()) / static_cast<float>(1 << 20);

					return FText::Format(CacheStatsMemoryText
										, FText::AsNumber(CacheSizeMb, FSlateStyle::Get().GetMemoryFormat())
										, FSlateStyle::Get().FormatMemoryAsMegabytes(MaxCacheSizeMb));
				})
				.Visibility_Lambda(CacheStatsVisibilityLambda)
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.ColorAndOpacity(CacheStatsColor)
				.Text(CacheStatsDurationText)
				.Visibility_Lambda(CacheStatsVisibilityLambda)
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text_Lambda([]()
				{
					FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
					const float CacheDuration = CacheManager.GetCacheDuration();

					return FText::Format(CacheStatsDurationValueText
										, FText::AsNumber(CacheDuration, FSlateStyle::Get().GetShortTimeFormat()));
				})
				.Visibility_Lambda(CacheStatsVisibilityLambda)
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheNudgeBackButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(NudgeBackTooltipText)
			.IsFocusable(false)
			.OnClicked_Lambda([this]()
			{
				FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
				FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

				const double ClampMin = CacheManager.GetCacheStartTimeStamp();
				const double CurrentTime = bIsPaused ? PausedTimeMarker : CacheManager.GetCacheEndTimeStamp();
				const double NewTime = FMath::Max(CurrentTime - GetNudgeStepSeconds(), ClampMin);

				TimingViewExtender.PauseTimeMarker(NewTime, ESystemControllingTimeMarker::External);

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CacheNudgeBackward"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheNudgeForwardButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(NudgeForwardTooltipText)
			.IsFocusable(false)
			.OnClicked_Lambda([this]()
			{
				FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
				FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();

				const double ClampMax = CacheManager.GetCacheEndTimeStamp();
				const double CurrentTime = bIsPaused ? PausedTimeMarker : ClampMax;
				const double NewTime = FMath::Min(CurrentTime + GetNudgeStepSeconds(), ClampMax);

				TimingViewExtender.PauseTimeMarker(NewTime, ESystemControllingTimeMarker::External);

				return FReply::Handled();
			})
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.CacheControls.CacheNudgeForward"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheTimelineRulerWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.ToolTipText(TimelineRulerTooltipText)
			[
				SNew(SBox)
				.HeightOverride(22.0f)
				[
					SNew(SAudioInsightsTimeSlider)
					.DesiredSize(FVector2f(200.0f, 22.0f))
					.OnFormatTickLabel_Lambda([](double InSeconds) { return FSlateStyle::Get().FormatTimestamp(InSeconds).ToString(); })
					.MinPixelsPerDisplayTick(10)
					.ScrubPosition_Lambda([this]() -> double
					{
						return GetCurrentRelativeTime();
					})
					.ViewRange_Lambda([this]() -> TRange<double>
					{
						if (bTimelineAutoScroll && !bIsPaused)
						{
							if (!IAudioInsightsModule::IsLiveSession())
							{
								const double TraceDuration = IAudioInsightsModule::GetChecked().GetTimingViewExtender().GetCurrentTraceDurationSeconds();
								return TRange<double>(0.0, FMath::Max(TraceDuration, 1.0));
							}

							constexpr double DefaultViewWindow = 50.0;
							const double ViewWindow = bTimelineViewRangeSetByUser
								? TimelineViewRange.Size<double>()
								: DefaultViewWindow;
							const double CurrentTime = GetCurrentRelativeTime();
							return TRange<double>(CurrentTime - ViewWindow, CurrentTime);
						}

						return TimelineViewRange;
					})
					.ClampRange_Lambda([this]() -> TRange<double>
					{
						if (!IAudioInsightsModule::IsLiveSession())
						{
							const double TraceDuration = IAudioInsightsModule::GetChecked().GetTimingViewExtender().GetCurrentTraceDurationSeconds();
							return TRange<double>(0.0, FMath::Max(TraceDuration, 1.0));
						}

						FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
						const double ClampMin = CacheManager.GetCacheStartTimeStamp() - BeginTimestamp;
						const double ClampMax = CacheManager.GetCacheEndTimeStamp() - BeginTimestamp;

						if (ClampMax <= ClampMin)
						{
							return TRange<double>(0.0, 1.0);
						}

						return TRange<double>(ClampMin, ClampMax);
					})
					.AllowZoom(true)
					.AllowPan(true)
					.ClampRangeHighlightSize(1.0f)
					.ClampRangeHighlightColor(FLinearColor(0.07f, 0.50f, 0.85f, 0.3f)) // cache region highlight color
					.OnScrubPositionChanged_Lambda([this](double NewValue, bool /*bIsScrubbing*/)
					{
						const double AbsoluteTime = NewValue + BeginTimestamp;
						FAudioInsightsTimingViewExtender& TimingViewExtender = IAudioInsightsModule::GetChecked().GetTimingViewExtender();
						TimingViewExtender.PauseTimeMarker(AbsoluteTime, ESystemControllingTimeMarker::External);
					})
					.OnViewRangeChanged_Lambda([this](TRange<double> NewRange)
					{
						TimelineViewRange = NewRange;
						bTimelineViewRangeSetByUser = true;
						bTimelineAutoScroll = false;
					})
				]
			];
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheSettingsMenuContent()
	{
		using namespace ToolbarWidgetsPrivate;

		FMenuBuilder MenuBuilder(false /*bShouldCloseWindowAfterMenuSelection*/, TSharedPtr<FUICommandList>());

		if (IAudioInsightsModule::IsLiveSession())
		{
			MenuBuilder.BeginSection("AudioInsightsToolbarCacheSettingsActions", CacheSettingsHeaderText);
			{
				MenuBuilder.AddMenuEntry(
					CacheSettingsShowStatsText,
					CacheSettingsShowStatsTooltipText,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([]()
						{
							if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
							{
								FCacheSettings& CacheSettings = Settings->CacheSettings;
								CacheSettings.bShowCacheStats = !CacheSettings.bShowCacheStats;
								Settings->SaveCacheSettings();
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([]()
						{
							if (const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>())
							{
								const FCacheSettings& CacheSettings = Settings->CacheSettings;
								return CacheSettings.bShowCacheStats;
							}

							return false;
						})
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddWidget(
					SNew(SBox)
					.WidthOverride(75.0f)
					[
						SNew(SSpinBox<uint32>)
						.MinValue(8u)
						.MaxValue(512u)
						.BroadcastValueChangesPerKey(true)
						.Value_Lambda([]() -> uint32
						{
							if (const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>())
							{
								return Settings->CacheSettings.CacheSizeMB;
							}

							return FCacheSettings::DefaultCacheSizeMB;
						})
						.OnValueChanged_Lambda([](uint32 NewValue)
						{
							if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
							{
								Settings->CacheSettings.CacheSizeMB = NewValue;
							}
						})
						.OnValueCommitted_Lambda([](uint32 /*NewValue*/, ETextCommit::Type)
						{
							// In-memory value is already updated per keystroke; persist to .ini on commit.
							if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
							{
								Settings->SaveCacheSettings();
							}
						})
						.ToolTipText(CacheSettingsCacheSizeTooltipText)
					],
					CacheSettingsCacheSizeText
				);

				MenuBuilder.AddWidget(
					SNew(SBox)
					.WidthOverride(75.0f)
					[
						SNew(SSpinBox<float>)
						.MinValue(FCacheSettings::MinNudgeStepSeconds)
						.MaxValue(FCacheSettings::MaxNudgeStepSeconds)
						.Delta(0.1f)
						.MinFractionalDigits(1)
						.MaxFractionalDigits(2)
						.Value_Lambda([]() -> float
						{
							if (const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>())
							{
								return FMath::Clamp(Settings->CacheSettings.NudgeStepSeconds, FCacheSettings::MinNudgeStepSeconds, FCacheSettings::MaxNudgeStepSeconds);
							}

							return FCacheSettings::DefaultNudgeStepSeconds;
						})
						.OnValueCommitted_Lambda([](float NewValue, ETextCommit::Type)
						{
							if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
							{
								Settings->CacheSettings.NudgeStepSeconds = FMath::Clamp(NewValue, FCacheSettings::MinNudgeStepSeconds, FCacheSettings::MaxNudgeStepSeconds);
								Settings->SaveCacheSettings();
							}
						})
						.ToolTipText(CacheSettingsNudgeStepTooltipText)
					],
					CacheSettingsNudgeStepText
				);

				MenuBuilder.AddSubMenu(
					CacheSettingsStopCacheWhenPausedText,
					CacheSettingsStopCacheWhenPausedTooltipText,
					FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
					{
						auto AddBehaviourEntry = [&SubMenuBuilder](const EStopCacheWhenPausedBehaviour Behaviour, const FText& Label, const FText& Tooltip)
						{
							SubMenuBuilder.AddMenuEntry(
								Label,
								Tooltip,
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([Behaviour]()
									{
										if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
										{
											Settings->CacheSettings.StopCacheWhenPausedBehaviour = Behaviour;
											Settings->SaveCacheSettings();
										}
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([Behaviour]()
									{
										if (const TObjectPtr<const UAudioInsightsSettings> Settings = GetDefault<UAudioInsightsSettings>())
										{
											return Settings->CacheSettings.StopCacheWhenPausedBehaviour == Behaviour;
										}

										return false;
									})
								),
								NAME_None,
								EUserInterfaceActionType::RadioButton
							);
						};

						AddBehaviourEntry(EStopCacheWhenPausedBehaviour::WhenMarkedForDeletion, StopCacheWhenPaused_WhenMarkedForDeletionText, StopCacheWhenPaused_WhenMarkedForDeletionTooltipText);
						AddBehaviourEntry(EStopCacheWhenPausedBehaviour::Always, StopCacheWhenPaused_AlwaysText, StopCacheWhenPaused_AlwaysTooltipText);
						AddBehaviourEntry(EStopCacheWhenPausedBehaviour::Never, StopCacheWhenPaused_NeverText, StopCacheWhenPaused_NeverTooltipText);
					})
				);
			}

			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FToolbarWidgets::MakeCacheSettingsButtonWidget()
	{
		using namespace ToolbarWidgetsPrivate;

		return SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent_Lambda([this]()
			{
				return MakeCacheSettingsMenuContent();
			})
			.OnMenuOpenChanged_Lambda([](bool bIsOpen)
			{
				// Flush in-memory changes to .ini when the menu closes (covers click-outside where commit may not fire).
				if (!bIsOpen)
				{
					if (TObjectPtr<UAudioInsightsSettings> Settings = GetMutableDefault<UAudioInsightsSettings>())
					{
						Settings->SaveCacheSettings();
					}
				}
			})
			.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
			.HasDownArrow(false)
			.ToolTipText(CacheSettingsTooltipText)
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Settings"))
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				]
			];
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
