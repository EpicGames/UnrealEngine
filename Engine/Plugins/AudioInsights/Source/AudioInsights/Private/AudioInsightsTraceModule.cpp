// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsTraceModule.h"

#include "AudioInsightsLog.h"
#include "AudioInsightsModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "CoreGlobals.h"
#include "Insights/IInsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Providers/VirtualLoopTraceProvider.h"
#include "SocketSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Trace/Trace.h"
#include "TraceServices/Model/Channel.h"
#include "TraceServices/Model/Diagnostics.h"
#include "TraceWriter/AudioInsightsCacheTraceWriter.h"

namespace UE::Audio::Insights
{
	namespace FTraceModulePrivate
	{
		static const TCHAR* AudioChannelName = TEXT("Audio");
		static const TCHAR* AudioMixerChannelName = TEXT("AudioMixer");
		static const TCHAR* BookmarkChannelName = TEXT("Bookmark");

		FString GetTraceStoreFilePath(const FString& FileName)
		{
			auto GetTraceHostAddress = []() -> FString
			{
				TSharedPtr<FInternetAddr> RecorderAddress;

				if (ISocketSubsystem* const Sockets = ISocketSubsystem::Get())
				{
					bool bCanBindAll = false;
					RecorderAddress = Sockets->GetLocalHostAddr(*GLog, bCanBindAll);
				}

				if (RecorderAddress.IsValid())
				{
					constexpr bool bAppendPort = false;
					return RecorderAddress->ToString(bAppendPort);
				}

				return "127.0.0.1";
			};

			const TUniquePtr<UE::Trace::FStoreClient> StoreClient = UE::Trace::FStoreClient::Create(*GetTraceHostAddress());

			if (StoreClient.IsValid())
			{
				if (const UE::Trace::FStoreClient::FStatus* StoreClientStatus = StoreClient->GetStatus())
				{
					const FString TraceStorePath = FString(StoreClientStatus->GetStoreDir());

					FString FilePath = FPaths::Combine(TraceStorePath, FileName);
					FPaths::NormalizeFilename(FilePath);

					return FilePath;
				}
			}

			UE_LOGF(LogAudioInsights, Warning, "Failed to connect to the Trace Store Client, returning an empty path.");

			return FString();
		}
	}

	void FRewindDebuggerAudioInsightsRuntime::RecordingStarted()
	{
		using namespace FTraceModulePrivate;

		UE::Trace::ToggleChannel(AudioChannelName, true);
		UE::Trace::ToggleChannel(AudioMixerChannelName, true);
		UE::Trace::ToggleChannel(BookmarkChannelName, true);
	}

	FTraceModule::FTraceModule()
#if !WITH_EDITOR
		: AudioChannels({ FTraceModulePrivate::AudioChannelName, FTraceModulePrivate::AudioMixerChannelName, FTraceModulePrivate::BookmarkChannelName } )
		, EmptyArray()
#endif
	{
#if WITH_EDITOR
		FTraceAuxiliary::OnTraceStopped.AddRaw(this, &FTraceModule::OnTraceStopped);
#else
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");

		const TSharedPtr<ISessionManager> SessionManager = SessionServicesModule.GetSessionManager();
		if (SessionManager.IsValid())
		{
			SessionManager->OnSessionsUpdated().AddRaw(this, &FTraceModule::OnSessionsUpdated);
		}
#endif // WITH_EDITOR
	}

	FTraceModule::~FTraceModule()
	{
#if WITH_EDITOR
		FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
#else
		if (FModuleManager::Get().IsModuleLoaded("SessionServices"))
		{
			ISessionServicesModule& SessionServicesModule = FModuleManager::GetModuleChecked<ISessionServicesModule>("SessionServices");

			const TSharedPtr<ISessionManager> SessionManager = SessionServicesModule.GetSessionManager();
			if (SessionManager.IsValid())
			{
				SessionManager->OnSessionsUpdated().RemoveAll(this);
			}
		}
#endif // WITH_EDITOR
	}

	void FTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
	{
		OutModuleInfo.Name = GetName();
		OutModuleInfo.DisplayName = TEXT("Audio");
	}

	void FTraceModule::AddTraceProvider(TSharedPtr<FTraceProviderBase> TraceProvider)
	{
		TraceProviders.Add(TraceProvider->GetName(), TraceProvider);
	}

	const FName FTraceModule::GetName()
	{
		const FLazyName TraceName = { "TraceModule_AudioTrace" };
		return TraceName.Resolve();
	}

	void FTraceModule::DisableAllTraceChannels()
	{
		UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& ChannelInfo, void* User)
		{
			// Only disable channels that are not read only (i.e. set from the command line)
			if (!ChannelInfo.bIsReadOnly && ChannelInfo.bIsEnabled)
			{
				UE::Trace::ToggleChannel(ANSI_TO_TCHAR(ChannelInfo.Name), false);
			}
			return true;
		}
		, nullptr);
	}

	bool FTraceModule::EnableAudioInsightsTraceChannels()
	{
		using namespace FTraceModulePrivate;

#if WITH_EDITOR
 		const bool bAudioOn = UE::Trace::ToggleChannel(AudioChannelName, true);
 		const bool bAudioMixerOn = UE::Trace::ToggleChannel(AudioMixerChannelName, true);
		const bool bBookmarkChannelOn = UE::Trace::ToggleChannel(BookmarkChannelName, true);

 		return bAudioOn && bAudioMixerOn && bBookmarkChannelOn;
#else
		if (!InstanceID.IsValid())
		{
			return false;
		}

		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
		if (!TraceController.IsValid())
		{
			return false;
		}

		// Requires ITraceController to have received a response to discovering the session and channel info
		if (TraceController->HasAvailableInstance(InstanceID))
		{
			TraceController->WithInstance(InstanceID, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				if (Status.bIsTracing)
				{
					Commands.SetChannels(AudioChannels, EmptyArray);
				}
			});

			return true;
		}
		return false;
#endif // WITH_EDITOR
	}

	void FTraceModule::DisableAudioInsightsTraceChannels() const
	{
		using namespace FTraceModulePrivate;

		auto DisableIfNotMarkedToRestore = [this](const TCHAR* ChannelName)
		{
			if (!ChannelsToRestore.Contains(ChannelName))
			{
				UE::Trace::ToggleChannel(ChannelName, false);
			}
		};

		DisableIfNotMarkedToRestore(AudioChannelName);
		DisableIfNotMarkedToRestore(AudioMixerChannelName);
		DisableIfNotMarkedToRestore(BookmarkChannelName);
	}

	void FTraceModule::CacheCurrentlyEnabledTraceChannels()
	{	
		ChannelsToRestore.Empty();

		UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& ChannelInfo, void* User)
		{
			TArray<FString>* EnabledChannels = static_cast<TArray<FString>*>(User);
			if (!ChannelInfo.bIsReadOnly && ChannelInfo.bIsEnabled)
			{
				EnabledChannels->Emplace(ANSI_TO_TCHAR(ChannelInfo.Name));
			}

			return true;
		}, &ChannelsToRestore);
	}

	void FTraceModule::RestoreCachedChannels() const
	{
		for (const FString& Channel : ChannelsToRestore)
		{
			UE::Trace::ToggleChannel(Channel.GetCharArray().GetData(), true);
		}
	}

#if !WITH_EDITOR
	bool FTraceModule::IsSessionInfoAvailable() const
	{
		IUnrealInsightsModule& InsightsModule = FModuleManager::GetModuleChecked<IUnrealInsightsModule>("TraceInsights");

		TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession = InsightsModule.GetAnalysisSession();
		if (AnalysisSession.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession.Get());

			UE::Trace::FStoreClient* StoreClient = InsightsModule.GetStoreClient();
			if (StoreClient)
			{
				const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByTraceId(AnalysisSession->GetTraceId());
				return SessionInfo != nullptr;
			}
			// Direct traces have a trace ID of zero and a null StoreClient - check IDiagnosticsProvider to see if we are running a direct trace
			else if (AnalysisSession->GetTraceId() == 0)
			{
				const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*AnalysisSession.Get());
				if (DiagnosticsProvider)
				{
					return DiagnosticsProvider->IsSessionInfoAvailable();
				}
			}
		}

		return false;
	}

	bool FTraceModule::GetAudioTracesAreEnabled() const
	{
		using namespace FTraceModulePrivate;

		const IUnrealInsightsModule& InsightsModule = FModuleManager::GetModuleChecked<IUnrealInsightsModule>("TraceInsights");
		const TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession = InsightsModule.GetAnalysisSession();

		if (!AnalysisSession.IsValid())
		{
			return false;
		}

		const TraceServices::IChannelProvider* ChannelProvider = AnalysisSession->ReadProvider<TraceServices::IChannelProvider>("ChannelProvider");
		if (ChannelProvider == nullptr)
		{
			return false;
		}
		
		bool bAudioChannelIsEnabled = false;
		bool bAudioMixerChannelIsEnabled = false;

		const TArray<TraceServices::FChannelEntry>& Channels = ChannelProvider->GetChannels();
		for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
		{
			if (Channels[ChannelIndex].bIsEnabled)
			{
				if (!bAudioChannelIsEnabled && Channels[ChannelIndex].Name.Compare(AudioChannelName) == 0)
				{
					bAudioChannelIsEnabled = true;
				}
				else if (!bAudioMixerChannelIsEnabled && Channels[ChannelIndex].Name.Compare(AudioMixerChannelName) == 0)
				{
					bAudioMixerChannelIsEnabled = true;
				}

				if (bAudioChannelIsEnabled && bAudioMixerChannelIsEnabled)
				{
					return true;
				}
			}
		}

		return false;
	}

	void FTraceModule::SendDiscoveryRequestToTraceController() const
	{
		// We need to send discovery requests to ITraceController so we can update the active channels later
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");

		TSharedPtr<ISessionManager> SessionManager = SessionServicesModule.GetSessionManager();
		TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
		if (SessionManager.IsValid() && TraceController.IsValid())
		{
			const FGuid SessionId = FApp::GetSessionId();

			if (TraceController.IsValid())
			{
				TraceController->SendDiscoveryRequest(SessionId, InstanceID);
				TraceController->SendStatusUpdateRequest();
			}
		}
	}

	bool FTraceModule::Tick(float DeltaTime)
	{
		if (TraceControllerIsAvailable())
		{
			RequestChannelUpdate();
			ResetTicker();
		}
		else
		{
			SendDiscoveryRequestToTraceController();
		}

		return true;
	}
#endif // !WITH_EDITOR

	void FTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
	{
		for (const auto& [ProviderName, Provider] : TraceProviders)
		{
#if !WITH_EDITOR
			Provider->InitSessionCachedMessages(InSession);
#endif // !WITH_EDITOR

			InSession.AddProvider(ProviderName, Provider, Provider);
			InSession.AddAnalyzer(Provider->ConstructAnalyzer(InSession));
		}

		FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();
		CacheManager.ClearCache();

		OnAnalysisStarting.Broadcast(FPlatformTime::Seconds());
	}

	void FTraceModule::StartTraceAnalysis(const bool bInOnlyTraceAudioChannels, const ETraceMode InTraceMode)
	{
		if (bTraceAnalysisHasStarted)
		{
			return;
		}

#if WITH_EDITOR
		// A trace already being connected means we are latching onto one started elsewhere
		// (e.g. the regular Insights record button). In that case the user picked their own
		// channels and we must not override them - just additively enable the audio channels.
		const bool bIsExternalTrace = FTraceAuxiliary::IsConnected();

		CacheCurrentlyEnabledTraceChannels();

		if ((bInOnlyTraceAudioChannels) && (!bIsExternalTrace))
		{
			DisableAllTraceChannels();
		}

		CurrentTraceMode = InTraceMode;
		bAudioTraceChannelsEnabled = EnableAudioInsightsTraceChannels();

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		if (bIsExternalTrace)
		{
			// Start analysis with already connected live session
			const FString& TraceDestination = FTraceAuxiliary::GetTraceDestinationString();

			if (TraceDestination == "localhost" || TraceDestination == "127.0.0.1")
			{
				UnrealInsightsModule.StartAnalysisForLastLiveSession();
			}
		}
		else
		{
			bIsAudioInsightsTrace = true;

			bool bTraceStarted = false;

			// Clear all buffered data and prevent data from previous recordings from leaking into the new recording
			FTraceAuxiliary::FOptions Options;
			Options.bExcludeTail = true;

			if (CurrentTraceMode == ETraceMode::Recording)
			{
				bTraceStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, TEXT("localhost"), TEXT(""), &Options);

				if (bTraceStarted)
				{
					UnrealInsightsModule.StartAnalysisForLastLiveSession();
				}
			}
			else if (CurrentTraceMode == ETraceMode::Monitoring)
			{
				const uint32 DirectTracePort = UnrealInsightsModule.StartAnalysisWithDirectTrace();

				const FString DirectTraceHost = FString::Printf(TEXT("127.0.0.1:%u"), DirectTracePort);

				bTraceStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, *DirectTraceHost, TEXT(""), &Options);
			}

			if (!bTraceStarted)
			{
				OnTraceStopped(FTraceAuxiliary::EConnectionType::Network, FString());
				return;
			}
		}
#else
		bAudioTraceChannelsEnabled = EnableAudioInsightsTraceChannels();
#endif // WITH_EDITOR

		bTraceAnalysisHasStarted = true;
	}

	void FTraceModule::StopTraceAnalysis()
	{
		if (FTraceAuxiliary::IsConnected())
		{
			FTraceAuxiliary::Stop();
		}
	}

#if WITH_EDITOR
	void FTraceModule::OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
	{
		if (bIsAudioInsightsTrace)
		{
			bIsAudioInsightsTrace = false;
		}

		if (bAudioTraceChannelsEnabled)
		{
			DisableAudioInsightsTraceChannels();
			RestoreCachedChannels();
			ChannelsToRestore.Empty();

			bAudioTraceChannelsEnabled = false;
		}

		bTraceAnalysisHasStarted = false;
		CurrentTraceMode = ETraceMode::None;
	}
#endif // WITH_EDITOR

	void FTraceModule::OnOnlyTraceAudioChannelsStateChanged(const bool bOnlyTraceAudioChannels)
	{
		if (!bTraceAnalysisHasStarted)
		{
			return;
		}

		if (bOnlyTraceAudioChannels)
		{
			// Re-cache the current settings for enabled channels. This may have changed since Audio Insights began.
			CacheCurrentlyEnabledTraceChannels();

			// Don't disable other channels when we're piggybacking on an externally-started trace -
			// the user picked those channels for their own profiling and we'd silently break it.
			if (bIsAudioInsightsTrace)
			{
				DisableAllTraceChannels();
			}

			EnableAudioInsightsTraceChannels();
		}
		else
		{
			RestoreCachedChannels();
		}
	}

	bool FTraceModule::IsTraceAnalysisActive() const
	{
		return FTraceAuxiliary::IsConnected() && bTraceAnalysisHasStarted;
	}

	bool FTraceModule::AudioChannelsCanBeManuallyEnabled() const
	{
#if WITH_EDITOR
		return false;
#else
		// When attaching to a packaged build, we need to check whether we're connected and the audio channels are enabled
		return !bTraceAnalysisHasStarted && !bAudioTraceChannelsEnabled && IsSessionInfoAvailable() && !GetAudioTracesAreEnabled();
#endif // WITH_EDITOR
	}

#if !WITH_EDITOR
	void FTraceModule::InitializeSessionInfo(const TraceServices::FSessionInfo& SessionInfo)
	{
		InstanceID = SessionInfo.InstanceId;

		OnTick = FTickerDelegate::CreateRaw(this, &FTraceModule::Tick);

		constexpr float TickDelay = 0.5f; // 500 ms. delay between ticks
		OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, TickDelay);
	}

	void FTraceModule::RequestChannelUpdate()
	{
		// Request for ITraceController to get the latest active channels list
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
		if (!TraceController.IsValid())
		{
			return;
		}

		TraceController->SendChannelUpdateRequest();
	}

	void FTraceModule::ResetTicker()
	{
		if (OnTickHandle.IsValid())
		{
			FTSTicker::RemoveTicker(OnTickHandle);
		}
	}

	bool FTraceModule::TraceControllerIsAvailable() const
	{
		if (!InstanceID.IsValid())
		{
			return false;
		}

		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
		if (!TraceController.IsValid())
		{
			return false;
		}

		return TraceController->HasAvailableInstance(InstanceID);
	}

	void FTraceModule::OnSessionsUpdated()
	{
		if (!InstanceID.IsValid())
		{
			const IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

			TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession = UnrealInsightsModule.GetAnalysisSession();
			if (AnalysisSession.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession.Get());

				const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*AnalysisSession.Get());
				if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
				{
					InstanceID = DiagnosticsProvider->GetSessionInfo().InstanceId;
				}
			}
		}

		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");

		const TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
		if (TraceController.IsValid() && !TraceController->HasAvailableInstance(InstanceID))
		{
			TraceController->SendDiscoveryRequest();
			TraceController->SendStatusUpdateRequest();
		}
	}
#endif // !WITH_EDITOR

	void FTraceModule::ExecuteConsoleCommand(const FString& InCommandStr)const 
	{
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		TSharedPtr<ISessionManager> SessionManager = SessionServicesModule.GetSessionManager();

		if (SessionManager.IsValid())
		{
			TArray<TSharedPtr<ISessionInfo>> Sessions;
			SessionManager->GetSessions(Sessions);

			for (const TSharedPtr<ISessionInfo>& Session : Sessions)
			{
				if (Session.IsValid())
				{
					TArray<TSharedPtr<ISessionInstanceInfo>> SessionInstances;
					Session->GetInstances(SessionInstances);

					for (const TSharedPtr<ISessionInstanceInfo>& SessionInstance : SessionInstances)
					{
						SessionInstance->ExecuteCommand(InCommandStr);
					}
				}
			}
		}
	}
	
	void FTraceModule::SaveCacheSnapshot()
	{
		using namespace FTraceModulePrivate;

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();

		if (CacheManager.GetNumUsedChunks() == 0)
		{
			UE_LOGF(LogAudioInsights, Warning, "Cache is empty, nothing to export.");
			return;
		}

		const FString FileName = FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S_AudioInsightsCache.utrace"));
		const FString FilePath = GetTraceStoreFilePath(FileName);

		if (FilePath.IsEmpty())
		{
			UE_LOGF(LogAudioInsights, Warning, "Failed to determine trace store path for cache snapshot.");
			return;
		}

		if (!FAudioInsightsCacheTraceWriter::WriteCacheToFile(CacheManager, FilePath))
		{
			UE_LOGF(LogAudioInsights, Warning, "Failed to write cache snapshot to %ls", *FilePath);
		}
	}

	ETraceMode FTraceModule::GetTraceMode() const
	{
		return CurrentTraceMode;
	}

	void FTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
	{
		OutLoggers.Add(TEXT("Audio"));
	}

	void FTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
	{

	}
} // namespace UE::Audio::Insights
