// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInsightsTraceProviderBase.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsTimingViewExtender.h"
#include "Modules/ModuleManager.h"

namespace UE::Audio::Insights
{
	FTraceProviderBase::FTraceProviderBase(FName InName)
		: Name(InName)
	{
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		FAudioInsightsTimingViewExtender& AudioInsightsTimingViewExtender = AudioInsightsModule.GetTimingViewExtender();
		AudioInsightsTimingViewExtender.OnTimingViewTimeMarkerChanged.AddRaw(this, &FTraceProviderBase::OnTimingViewTimeMarkerChanged);
		AudioInsightsTimingViewExtender.OnTimeControlMethodReset.AddRaw(this, &FTraceProviderBase::OnTimeControlMethodReset);
	}

	FTraceProviderBase::~FTraceProviderBase()
	{
		if (FModuleManager::Get().IsModuleLoaded("AudioInsights"))
		{
			FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			FAudioInsightsTimingViewExtender& AudioInsightsTimingViewExtender = AudioInsightsModule.GetTimingViewExtender();
			AudioInsightsTimingViewExtender.OnTimingViewTimeMarkerChanged.RemoveAll(this);
			AudioInsightsTimingViewExtender.OnTimeControlMethodReset.RemoveAll(this);
		}
	}

	FName FTraceProviderBase::GetName() const
	{
		return Name;
	}

	FTraceProviderBase::FTraceAnalyzerBase::FTraceAnalyzerBase(TSharedRef<FTraceProviderBase> InProvider)
		: Provider(InProvider)
		, CacheManager(IAudioInsightsModule::GetChecked().GetCacheManager())
	{
	}

	void FTraceProviderBase::FTraceAnalyzerBase::OnAnalysisBegin(const FOnAnalysisContext& Context)
	{
		Provider->Reset();

		UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
		Builder.RouteEvent(RouteID_ChannelAnnounce, "Trace", "ChannelAnnounce");
		Builder.RouteEvent(RouteID_ChannelToggle, "Trace", "ChannelToggle");
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		if (ShouldIgnoreNewEvents())
		{
			return true;
		}

		switch (RouteId)
		{
			case RouteID_ChannelAnnounce:
			{
				return HandleChannelAnnounceEvent(RouteId, Style, Context);
			}
			case RouteID_ChannelToggle:
				return HandleChannelToggleEvent(RouteId, Style, Context);
			default:
			{
				return OnHandleEvent(RouteId, Style, Context);
			}
		}
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::HandleChannelAnnounceEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		if (AudioChannelID.IsSet() && AudioMixerChannelID.IsSet())
		{
			return OnEventSuccess(RouteId, Style, Context);
		}

		const Trace::IAnalyzer::FEventData& EventData = Context.EventData;

		FString ChannelName;
		EventData.GetString("Name", ChannelName);

		if (!AudioChannelID.IsSet() && ChannelName.Compare("Audio") == 0)
		{
			AudioChannelID = EventData.GetValue<uint32>("Id");
		}
		else if (!AudioMixerChannelID.IsSet() && ChannelName.Compare("AudioMixer") == 0)
		{
			AudioMixerChannelID = EventData.GetValue<uint32>("Id");
		}

		return OnEventSuccess(RouteId, Style, Context);
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::HandleChannelToggleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		if (!AudioChannelID.IsSet() || !AudioMixerChannelID.IsSet())
		{
			return OnEventSuccess(RouteId, Style, Context);
		}

		const bool bPrevAudioChannelIsEnabled = bAudioChannelIsEnabled;
		const bool bPrevAudioMixerChannelIsEnabled = bAudioMixerChannelIsEnabled;

		const Trace::IAnalyzer::FEventData& EventData = Context.EventData;
		const uint32 ToggledChannelID = EventData.GetValue<uint32>("Id");
		if (ToggledChannelID == AudioChannelID.GetValue())
		{
			bAudioChannelIsEnabled = EventData.GetValue<bool>("IsEnabled");
		}
		else if (ToggledChannelID == AudioMixerChannelID.GetValue())
		{
			bAudioMixerChannelIsEnabled = EventData.GetValue<bool>("IsEnabled");
		}

		const bool bChannelStateHasBeenChanged = bAudioChannelIsEnabled != bPrevAudioChannelIsEnabled || bAudioMixerChannelIsEnabled != bPrevAudioMixerChannelIsEnabled;
		const bool bProviderRequiresRefresh = bAudioChannelIsEnabled && bAudioMixerChannelIsEnabled && bChannelStateHasBeenChanged;
												
		if (bProviderRequiresRefresh)
		{
			if (IsInGameThread())
			{
				Provider->OnTraceChannelsEnabled();
			}
			else
			{
				ExecuteOnGameThread(TEXT("FTraceProviderBase::FTraceAnalyzerBase::::OnTraceChannelsEnabled"), [this] { Provider->OnTraceChannelsEnabled(); });
			}
		}

		return OnEventSuccess(RouteId, Style, Context);
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::OnEventSuccess(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		++(Provider->LastMessageId);
		return true;
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::OnEventFailure(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		++(Provider->LastMessageId);

		const FString ProviderName = Provider->GetName().ToString();
		constexpr bool bEventSucceeded = false;
		ensureAlwaysMsgf(bEventSucceeded, TEXT("'%s' TraceProvider's Analyzer message with RouteId '%u' event not handled"), *ProviderName, RouteId);
		return bEventSucceeded;
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::ShouldIgnoreNewEvents() const
	{
		return Provider->GetMessageCacheAndProcessingStatus() == ECacheAndProcess::None;
	}

	bool FTraceProviderBase::FTraceAnalyzerBase::ShouldProcessNewEvents() const
	{
		return Provider->GetMessageCacheAndProcessingStatus() == ECacheAndProcess::Latest;
	}

	bool FTraceProviderBase::IsMessageProcessingPaused() const
	{
		return GetMessageCacheAndProcessingStatus() != ECacheAndProcess::Latest;
	}

	ECacheAndProcess FTraceProviderBase::GetMessageCacheAndProcessingStatus() const
	{
		if (const FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr())
		{
			return AudioInsightsModulePtr->GetTimingViewExtender().GetMessageCacheAndProcessingStatus();
		}
		
		return ECacheAndProcess::None;
	}

	void FTraceProviderBase::ResetMessageProcessType()
	{
		if (FAudioInsightsModule* AudioInsightsModulePtr = FAudioInsightsModule::GetModulePtr())
		{
			AudioInsightsModulePtr->GetTimingViewExtender().ResumeTimeMarker();
		}
	}
} // namespace UE::Audio::Insights
