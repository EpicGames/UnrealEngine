// Copyright Epic Games, Inc. All Rights Reserved.
#include "Cache/AudioCachedMessageChunk.h"

namespace UE::Audio::Insights
{
	void FAudioCachedMessageChunk::AddMessageToChunk(const TSharedPtr<IAudioCachedMessage> Message)
    {
		if (!Message.IsValid())
		{
			return;
		}

		const double MessageTimestamp = Message->Timestamp;
		
		TSharedPtr<TraceServices::TPagedArray<TSharedPtr<IAudioCachedMessage>>>& CachedMessages = Messages.FindOrAdd(Message->GetMessageName(), MakeShared<TraceServices::TPagedArray<TSharedPtr<IAudioCachedMessage>>>(Allocator, 4096));
		if (CachedMessages->Num() == 0 || CachedMessages->Last()->Timestamp <= MessageTimestamp)
		{
			if (RangeStartTimestamp == INVALID_TIMESTAMP)
			{
				RangeStartTimestamp = MessageTimestamp;
			}

			if (RangeEndTimestamp < MessageTimestamp)
			{
				RangeEndTimestamp = MessageTimestamp;
			}

			CurrentChunkSize += Message->GetSizeOf();

			CachedMessages->EmplaceBack(Message);
		}
    }

	void FAudioCachedMessageChunk::ClearChunk()
	{
		Messages.Reset();
		CurrentChunkSize = 0u;

		RangeStartTimestamp = INVALID_TIMESTAMP;
		RangeEndTimestamp = INVALID_TIMESTAMP;
	}

	uint32 FAudioCachedMessageChunk::GetCurrentChunkSize() const
	{
		return CurrentChunkSize;
	}

	bool FAudioCachedMessageChunk::IsChunkFull() const
	{
		return CurrentChunkSize >= MaxChunkSize;
	}

	bool FAudioCachedMessageChunk::HasAnyData() const
	{
		return CurrentChunkSize > 0u;
	}

	const TMap<FName, TSharedPtr<FAudioCachedMessageChunk::TCachePagedArray>>& FAudioCachedMessageChunk::GetAllChunkMessages() const
	{
		return Messages;
	}

	bool FAudioCachedMessageChunk::TimestampIsInChunkRange(const double Timestamp) const
	{
		return FMath::IsWithin(Timestamp, RangeStartTimestamp, RangeEndTimestamp);
	}

	double FAudioCachedMessageChunk::GetChunkTimeRangeStart() const
	{
		return RangeStartTimestamp;
	}

	double FAudioCachedMessageChunk::GetChunkTimeRangeEnd() const
	{
		return RangeEndTimestamp;
	}
} // namespace UE::Audio::Insights
