// Copyright Epic Games, Inc. All Rights Reserved.
#include "Cache/AudioInsightsCacheManager.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsSettings.h"
#include "AudioInsightsTimingViewExtender.h"
#if !WITH_EDITOR
#include "AudioInsightsComponent.h"
#include "Insights/IInsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace CacheManagerPrivate
	{
		constexpr uint32 MaxChunkSize = 1 << 20; // 1 MB per chunk
	}

	FAudioInsightsCacheManager::FAudioInsightsCacheManager()
		: Allocator(8 << 20) // 8mB for storage of pointers to messages
	{
		OnCacheSizeSettingsChanged();

		check(NumChunks > 0u);

		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("FAudioInsightsCacheManager::ProcessPendingMessages"), 0.0f, [this](float DeltaTime)
		{
			ProcessPendingMessages(DeltaTime);
			return true;
		});

		CacheSettingsChangedHandle = FCacheSettings::OnCacheSizeSettingsChanged.AddRaw(this, &FAudioInsightsCacheManager::OnCacheSizeSettingsChanged);
	}

	FAudioInsightsCacheManager::~FAudioInsightsCacheManager()
	{
		FTSTicker::RemoveTicker(TickerHandle);

		if (CacheSettingsChangedHandle.IsValid())
		{
			FCacheSettings::OnCacheSizeSettingsChanged.Remove(CacheSettingsChangedHandle);
			CacheSettingsChangedHandle.Reset();
		}
	}

	void FAudioInsightsCacheManager::AddMessageToCache(TSharedPtr<IAudioCachedMessage> Message)
	{
		if (Message.IsValid())
		{
			NewMessageQueue.Enqueue(MoveTemp(Message));
		}
	}

	void FAudioInsightsCacheManager::ClearCache()
	{
		NewMessageQueue.DequeueAll();
		MessageTypeRegistry.Empty();

		CreateCache();
	}

	// NumChunks = CacheSizeMB since each chunk is 1 MB (MaxChunkSize)
	void FAudioInsightsCacheManager::OnCacheSizeSettingsChanged()
	{
		if (const UAudioInsightsSettings* const Settings = GetDefault<UAudioInsightsSettings>())
		{
			// Clamp at runtime since ClampMin/ClampMax metadata only applies in the editor UI
			const uint32 ClampedCacheSizeMB = FMath::Clamp(Settings->CacheSettings.CacheSizeMB, FCacheSettings::MinCacheSizeMB, FCacheSettings::MaxCacheSizeMB);
			if (NumChunks != ClampedCacheSizeMB)
			{
				NumChunks = ClampedCacheSizeMB;
			}

			// Reset if size has changed
			if (Cache.Capacity() != NumChunks)
			{
				ClearCache();

				// Always restart/resume the cache when we reset it
				if (IAudioInsightsModule::IsModuleLoaded())
				{
					IAudioInsightsModule::GetChecked().GetTimingViewExtender().ResumeTimeMarker();
				}
			}
		}
	}

	float FAudioInsightsCacheManager::GetCacheDuration() const
	{
		return FMath::Max(static_cast<float>(GetCacheEndTimeStamp() - GetCacheStartTimeStamp()), 0.0f);
	}

	double FAudioInsightsCacheManager::GetCacheStartTimeStamp() const
	{
		check(StartIndex < Cache.Capacity());

		return Cache[StartIndex].GetChunkTimeRangeStart();
	}

	double FAudioInsightsCacheManager::GetCacheEndTimeStamp() const
	{
		check(WriteIndex < Cache.Capacity());

		return Cache[WriteIndex].GetChunkTimeRangeEnd();
	}

	uint32 FAudioInsightsCacheManager::GetUsedCacheSize() const
	{
		uint32 UsedSpace = 0u;
		for (uint32 Index = 0u; Index < Cache.Capacity(); ++Index)
		{
			UsedSpace += Cache[Index].GetCurrentChunkSize();
		}

		return UsedSpace;
	}

	uint32 FAudioInsightsCacheManager::GetMaxCacheSize() const
	{
		return CacheManagerPrivate::MaxChunkSize * GetNumChunks();
	}

	uint32 FAudioInsightsCacheManager::GetNumChunks() const
	{
		return Cache.Capacity();
	}

	uint32 FAudioInsightsCacheManager::GetNumUsedChunks() const
	{
		uint32 NumUsedChunks = 0u;
		for (uint32 Index = 0u; Index < Cache.Capacity(); ++Index)
		{
			if (Cache[Index].HasAnyData())
			{
				++NumUsedChunks;
			}
		}

		return NumUsedChunks;
	}

	TOptional<uint32> FAudioInsightsCacheManager::TryGetNumChunksFromStartForTimestamp(const double Timestamp) const
	{
		uint32 ReadIndex = StartIndex;
		for (uint32 Index = 0u; Index < Cache.Capacity(); ++Index)
		{
			if (Cache[ReadIndex].TimestampIsInChunkRange(Timestamp))
			{
				return Index;
			}
			else
			{
				ReadIndex = Cache.GetNextIndex(ReadIndex);
				if (ReadIndex >= Cache.Capacity())
				{
					break;
				}
			}
		}

		return TOptional<uint32>();
	}

	const FAudioCachedMessageChunk* const FAudioInsightsCacheManager::GetChunk(const uint32 NumChunksFromStart) const
	{
		ensure(NumChunksFromStart < Cache.Capacity());
		if (NumChunksFromStart >= Cache.Capacity())
		{
			return nullptr;
		}

		uint32 ReadIndex = StartIndex;
		for (uint32 Index = 0u; Index < NumChunksFromStart; ++Index)
		{
			ReadIndex = Cache.GetNextIndex(ReadIndex);
		}

		if (ReadIndex >= Cache.Capacity())
		{
			return nullptr;
		}

		if (!Cache[ReadIndex].HasAnyData())
		{
			return nullptr;
		}

		return &Cache[ReadIndex];
	}

	bool FAudioInsightsCacheManager::IsTimestampMarkedForDeletion(const double Timestamp) const
	{
		if (GetNumUsedChunks() != GetNumChunks())
		{
			return false;
		}

		const TOptional<uint32> NumChunksFromStart = TryGetNumChunksFromStartForTimestamp(Timestamp);
		return NumChunksFromStart.IsSet() && (NumChunksFromStart.GetValue() == 0u);
	}

	void FAudioInsightsCacheManager::CreateCache()
	{
		Cache.ResetCache(Allocator, NumChunks);

		WriteIndex = 0u;
		StartIndex = 0u;
	}

	void FAudioInsightsCacheManager::ProcessPendingMessages(float DeltaTime)
	{
		check(WriteIndex < Cache.Capacity());

		FAudioCachedMessageChunk* Chunk = &Cache[WriteIndex];

		const TArray<TSharedPtr<IAudioCachedMessage>> NewMessages = NewMessageQueue.DequeueAll();
		for (const TSharedPtr<IAudioCachedMessage>& Message : NewMessages)
		{
			const FName MessageName = Message->GetMessageName();
			if (!MessageTypeRegistry.Contains(MessageName))
			{
				MessageTypeRegistry.Emplace(MessageName, Message);
			}

			if (Chunk->IsChunkFull())
			{
				const uint32 NextWriteIndex = Cache.GetNextIndex(WriteIndex);

				// If our cache is not limited, we may need to reserve more chunks
				if (NextWriteIndex >= Cache.Capacity())
				{
					Cache.ReserveNewChunks(Allocator, NumChunks);
				}

				WriteIndex = NextWriteIndex;

				Chunk = &Cache[WriteIndex];

				// If this new chunk already has data, clear it before we begin writing
				if (Chunk->HasAnyData())
				{
					StartIndex = Cache.GetNextIndex(WriteIndex);
					Chunk->ClearChunk();
				}

				Chunk->AddMessageToChunk(Message);

				OnChunkOverwritten.Broadcast(GetCacheStartTimeStamp());
			}
			else
			{
				Chunk->AddMessageToChunk(Message);
			}
		}
	}

	FAudioInsightsCache::FAudioInsightsCache()
	{
#if WITH_EDITOR
		bLimitCacheSize = true;
#else
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		if (UnrealInsightsModule.GetAnalysisSession().IsValid())
		{
			InitCacheIsLimited();
		}
		else
		{
			// The session isn't ready yet - request a notification when it goes live
			TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
			if (InsightsManager.IsValid())
			{
				InsightsManager->GetSessionChangedEvent().AddRaw(this, &FAudioInsightsCache::OnSessionChangedEvent);
			}
		}
#endif // WITH_EDITOR
	}

	FAudioInsightsCache::~FAudioInsightsCache()
	{
#if !WITH_EDITOR
		if (IUnrealInsightsModule* UnrealInsightsModule = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
		{
			TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule->GetInsightsManager();
			if (InsightsManager.IsValid())
			{
				InsightsManager->GetSessionChangedEvent().RemoveAll(this);
			}
		}
#endif // !WITH_EDITOR
	}

	void FAudioInsightsCache::ResetCache(TraceServices::FSlabAllocator& Allocator, const uint32 InNumChunks)
	{
		Cache.Empty();

		ReserveNewChunks(Allocator, InNumChunks);
	}

	void FAudioInsightsCache::ReserveNewChunks(TraceServices::FSlabAllocator& Allocator, const uint32 NumToReserve)
	{
		Cache.Reserve(Cache.Num() + NumToReserve);

		for (uint32 Index = 0u; Index < NumToReserve; ++Index)
		{
			Cache.Add(MakeUnique<FAudioCachedMessageChunk>(Allocator, CacheManagerPrivate::MaxChunkSize));
		}
	}

	uint32 FAudioInsightsCache::Capacity() const
	{
		return static_cast<uint32>(Cache.Num());
	}

	uint32 FAudioInsightsCache::GetNextIndex(uint32 ReadIndex) const
	{
		ReadIndex++;
		if (bLimitCacheSize && ReadIndex >= Capacity())
		{
			ReadIndex = 0u;
		}

		return ReadIndex;
	}

	uint32 FAudioInsightsCache::GetPreviousIndex(uint32 ReadIndex) const
	{
		if (ReadIndex > 0u)
		{
			ReadIndex--;
		}
		else if (bLimitCacheSize)
		{
			ReadIndex = Capacity() - 1u;
		}
		else
		{
			ReadIndex = INDEX_NONE;
		}

		return ReadIndex;
	}

#if !WITH_EDITOR
	void FAudioInsightsCache::InitCacheIsLimited()
	{
		// Limit cache in standalone if LiveSession or DirectTrace. Note: We don't limit the cache when reading from a trace file.

		if (IAudioInsightsModule::IsLiveSession())
		{
			bLimitCacheSize = true;
			return;
		}

		// Direct traces can be started before going 'Live', but always have a trace ID of zero.
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<const TraceServices::IAnalysisSession> Session = UnrealInsightsModule.GetAnalysisSession();

		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			if (Session->GetTraceId() == 0)
			{
				bLimitCacheSize = true;
			}
		}
	}

	void FAudioInsightsCache::OnSessionChangedEvent()
	{
		InitCacheIsLimited();

		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule.GetInsightsManager();
		if (InsightsManager.IsValid())
		{
			InsightsManager->GetSessionChangedEvent().RemoveAll(this);
		}
	}
#endif // !WITH_EDITOR
} // namespace UE::Audio::Insights
