// Copyright Epic Games, Inc. All Rights Reserved.
#include "IAudioInsightsModule.h"

#include "AudioInsightsComponent.h"
#include "AudioInsightsModule.h"

IAudioInsightsTraceModule& IAudioInsightsModule::GetTraceModule()
{
	IAudioInsightsModule& AudioInsightsModule = static_cast<IAudioInsightsModule&>(UE::Audio::Insights::FAudioInsightsModule::GetChecked());
	return AudioInsightsModule.GetTraceModule();
}

UE::Audio::Insights::FAudioInsightsCacheManager& IAudioInsightsModule::GetCacheManager()
{
	IAudioInsightsModule& AudioInsightsModule = static_cast<IAudioInsightsModule&>(UE::Audio::Insights::FAudioInsightsModule::GetChecked());
	return AudioInsightsModule.GetCacheManager();
}

UE::Audio::Insights::FAudioInsightsTimingViewExtender& IAudioInsightsModule::GetTimingViewExtender()
{
	IAudioInsightsModule& AudioInsightsModule = static_cast<IAudioInsightsModule&>(UE::Audio::Insights::FAudioInsightsModule::GetChecked());
	return AudioInsightsModule.GetTimingViewExtender();
}

#if WITH_EDITOR
UE::Audio::Insights::FAudioInsightsDetailsSelectionManager& IAudioInsightsModule::GetDetailsSelectionManager()
{
	IAudioInsightsModule& AudioInsightsModule = static_cast<IAudioInsightsModule&>(UE::Audio::Insights::FAudioInsightsModule::GetChecked());
	return AudioInsightsModule.GetDetailsSelectionManager();
}
#endif // WITH_EDITOR

bool IAudioInsightsModule::IsModuleLoaded()
{
	return UE::Audio::Insights::FAudioInsightsModule::IsModuleLoaded();
}

bool IAudioInsightsModule::IsLiveSession()
{
	if (UE::Audio::Insights::FAudioInsightsModule* const ModulePtr = UE::Audio::Insights::FAudioInsightsModule::GetModulePtr())
	{
#if WITH_EDITOR
		const IAudioInsightsTraceModule& TraceModule = ModulePtr->GetTraceModule();
		return TraceModule.IsTraceAnalysisActive();
#else
		const TSharedPtr<UE::Audio::Insights::FAudioInsightsComponent> Component = ModulePtr->GetAudioInsightsComponent();
		return Component.IsValid() && Component->GetIsLiveSession();
#endif // WITH_EDITOR
	}

	return false;
}

IAudioInsightsModule& IAudioInsightsModule::GetChecked()
{
	return static_cast<IAudioInsightsModule&>(UE::Audio::Insights::FAudioInsightsModule::GetChecked());
}

#ifdef WITH_EDITOR
IAudioInsightsModule& IAudioInsightsModule::GetEditorChecked()
{
	return static_cast<IAudioInsightsModule&>(FModuleManager::GetModuleChecked<IAudioInsightsModule>("AudioInsightsEditor"));
}
#endif
