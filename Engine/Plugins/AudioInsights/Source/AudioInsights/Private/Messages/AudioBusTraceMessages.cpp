// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/AudioBusTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace AudioBusMessageNames
	{
		const FName Start = "AudioBusStart";
		const FName HasActivity = "AudioBusHasActivity";
		const FName EnvelopeFollowerEnabled = "AudioBusEnvelopeFollowerEnabled";
		const FName EnvelopeValues = "AudioBusEnvelopeValues";
		const FName Stop = "AudioBusStop";
	};

	// FAudioBusMessageBase
	FAudioBusMessageBase::FAudioBusMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
    {
		const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		AudioBusId = EventData.GetValue<uint32>("AudioBusId");
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
    }

	// FAudioBusStartMessage
	FAudioBusStartMessage::FAudioBusStartMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FAudioBusMessageBase(InContext)
	{
		const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		EventData.GetString("Name", Name);
		NumChannels = EventData.GetValue<int32>("NumChannels");

		AudioBusType = FSoftObjectPath(Name).IsAsset() ? EAudioBusType::AssetBased : EAudioBusType::CodeGenerated;
	}

	uint32 FAudioBusStartMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FAudioBusStartMessage);

		// Add any dynamically sized members
		MemorySize += Name.GetAllocatedSize();

		return MemorySize;
	}

	FCacheWriteHandler FAudioBusStartMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("AudioBusStart"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AudioBusId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Name"),        UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("NumChannels"), UE::Trace::ETraceWriterFieldType::Int32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioBusStartMessage& Msg = static_cast<const FAudioBusStartMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("AudioBusId",  Msg.AudioBusId)
					.Field("Timestamp",   TimestampCycles)
					.Field("Name",        FStringView(Msg.Name))
					.Field("NumChannels", Msg.NumChannels)
					.End();
			}
		};
	}

	// FAudioBusHasActivityMessage
	FAudioBusHasActivityMessage::FAudioBusHasActivityMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FAudioBusMessageBase(InContext)
	{
		const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		bHasActivity = EventData.GetValue<bool>("HasActivity");
	}

	FCacheWriteHandler FAudioBusHasActivityMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("AudioBusHasActivity"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AudioBusId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("HasActivity"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioBusHasActivityMessage& Msg = static_cast<const FAudioBusHasActivityMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("AudioBusId",  Msg.AudioBusId)
					.Field("Timestamp",   TimestampCycles)
					.Field("HasActivity", Msg.bHasActivity)
					.End();
			}
		};
	}

	// FAudioBusEnvelopeFollowerEnabledMessage
	FAudioBusEnvelopeFollowerEnabledMessage::FAudioBusEnvelopeFollowerEnabledMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FAudioBusMessageBase(InContext)
	{
		const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		bEnvelopeFollowerEnabled = EventData.GetValue<bool>("EnvelopeFollowerEnabled");
	}

	FCacheWriteHandler FAudioBusEnvelopeFollowerEnabledMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("AudioBusEnvelopeFollowerEnabled"))
					.Field(ANSITEXTVIEW("DeviceId"),                UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AudioBusId"),              UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),               UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("EnvelopeFollowerEnabled"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioBusEnvelopeFollowerEnabledMessage& Msg = static_cast<const FAudioBusEnvelopeFollowerEnabledMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",                Msg.DeviceId)
					.Field("AudioBusId",              Msg.AudioBusId)
					.Field("Timestamp",               TimestampCycles)
					.Field("EnvelopeFollowerEnabled", Msg.bEnvelopeFollowerEnabled)
					.End();
			}
		};
	}

	// FAudioBusEnvelopeValuesMessage
	FAudioBusEnvelopeValuesMessage::FAudioBusEnvelopeValuesMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FAudioBusMessageBase(InContext)
	{
		const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		const TArrayView<const float> EnvelopeValuesView = EventData.GetArrayView<float>("EnvelopeValues");

		for (const auto& EnvelopeValue : EnvelopeValuesView)
		{
			EnvelopeValues.Emplace(EnvelopeValue);
		}
	}

	uint32 FAudioBusEnvelopeValuesMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FAudioBusEnvelopeValuesMessage);

		// Add any dynamically sized members
		if (EnvelopeValues.Num() > 0)
		{
			MemorySize += EnvelopeValues.Num() * sizeof(EnvelopeValues[0]);
		}

		return MemorySize;
	}

	FCacheWriteHandler FAudioBusEnvelopeValuesMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("AudioBusEnvelopeValues"))
					.Field(ANSITEXTVIEW("DeviceId"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AudioBusId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),      UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("EnvelopeValues"), UE::Trace::ETraceWriterFieldType::Float32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioBusEnvelopeValuesMessage& Msg = static_cast<const FAudioBusEnvelopeValuesMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",       Msg.DeviceId)
					.Field("AudioBusId",     Msg.AudioBusId)
					.Field("Timestamp",      TimestampCycles)
					.Field("EnvelopeValues", TConstArrayView<float>(Msg.EnvelopeValues.GetData(), Msg.EnvelopeValues.Num()))
					.End();
			}
		};
	}

	// FAudioBusStopMessage
	FAudioBusStopMessage::FAudioBusStopMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FAudioBusMessageBase(InContext)
	{
	}

	FCacheWriteHandler FAudioBusStopMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("AudioBusStop"))
					.Field(ANSITEXTVIEW("DeviceId"),   UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AudioBusId"), UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),  UE::Trace::ETraceWriterFieldType::Uint64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioBusStopMessage& Msg = static_cast<const FAudioBusStopMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",   Msg.DeviceId)
					.Field("AudioBusId", Msg.AudioBusId)
					.Field("Timestamp",  TimestampCycles)
					.End();
			}
		};
	}

	// FAudioBusDashboardEntry
	FAudioBusDashboardEntry::FAudioBusDashboardEntry()
		: AudioMeterInfo(MakeShared<FAudioMeterInfo>())
	{
	}

	FAudioBusDashboardEntry::FAudioBusDashboardEntry(FAudioBusDashboardEntry& Other)
		: AudioBusId(Other.AudioBusId)
		, Timestamp(Other.Timestamp)
		, AudioMeterInfo(MakeShared<FAudioMeterInfo>(*Other.AudioMeterInfo))
		, Name(Other.Name)
		, bHasActivity(Other.bHasActivity)
		, bEnvelopeFollowerEnabled(Other.bEnvelopeFollowerEnabled)
		, AudioBusType(Other.AudioBusType)
	{
	}

	FAudioBusDashboardEntry& FAudioBusDashboardEntry::operator=(const FAudioBusDashboardEntry& Other)
	{
		if (this == &Other)
		{
			return *this;
		}

		AudioBusId = Other.AudioBusId;
		Timestamp = Other.Timestamp;
		AudioMeterInfo = MakeShared<FAudioMeterInfo>(*Other.AudioMeterInfo);
		Name = Other.Name;
		bHasActivity = Other.bHasActivity;
		bEnvelopeFollowerEnabled = Other.bEnvelopeFollowerEnabled;
		AudioBusType = Other.AudioBusType;

		return *this;
	}
} // namespace UE::Audio::Insights
