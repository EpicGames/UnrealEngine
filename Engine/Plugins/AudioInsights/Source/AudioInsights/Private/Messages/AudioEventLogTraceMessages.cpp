// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/AudioEventLogTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
    FAudioEventLogMessage::FAudioEventLogMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
    {
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		static uint32 MessageIDTracker = 0u;
		MessageID = MessageIDTracker++;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
		PlayOrder = EventData.GetValue<uint32>("PlayOrder");

		EventData.GetString("EventLogName", EventName);
		EventData.GetString("AssetPath", AssetPath);
		EventData.GetString("ActorLabel", ActorLabel);
		EventData.GetString("ActorIconName", ActorIconName);
		EventData.GetString("AudioComponentName", AudioComponentName);
		EventData.GetString("SoundClassName", CategoryName);

		if (CategoryName == SoundClassNames::SoundCue)
		{
			CategoryType = EAudioEventLogSoundCategory::SoundCue;
		}
		else if (CategoryName == SoundClassNames::SoundWave)
		{
			CategoryType = EAudioEventLogSoundCategory::SoundWave;
		}
		else if (CategoryName == SoundClassNames::MetaSoundSource)
		{
			CategoryType = EAudioEventLogSoundCategory::MetaSound;
		}
		else if (CategoryName == SoundClassNames::SoundWaveProcedural)
		{
			CategoryType = EAudioEventLogSoundCategory::ProceduralSource;
		}
		else if (CategoryName == SoundClassNames::SoundCueTemplate)
		{
			CategoryType = EAudioEventLogSoundCategory::SoundCueTemplate;
		}
		else
		{
			CategoryType = EAudioEventLogSoundCategory::None;
		}
    }

	uint32 FAudioEventLogMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FAudioEventLogMessage);

		// Add any dynamically sized members
		MemorySize += EventName.GetAllocatedSize();
		MemorySize += AssetPath.GetAllocatedSize();
		MemorySize += ActorLabel.GetAllocatedSize();
		MemorySize += ActorIconName.GetAllocatedSize();
		MemorySize += AudioComponentName.GetAllocatedSize();
		MemorySize += CategoryName.GetAllocatedSize();

		return MemorySize;
	}

	const FName FAudioEventLogMessage::GetMessageName() const
	{
		static const FLazyName EventLogName = "EventLog";
		return EventLogName;
	}

	FCacheWriteHandler FAudioEventLogMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("EventLog"))
					.Field(ANSITEXTVIEW("DeviceId"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),      UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),      UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),      UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("EventLogName"),   UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),     UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorIconName"),  UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"), UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("AudioComponentName"), UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FAudioEventLogMessage& Msg = static_cast<const FAudioEventLogMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",       Msg.DeviceId)
					.Field("Timestamp",      TimestampCycles)
					.Field("PlayOrder",      Msg.PlayOrder)
					.Field("AssetPath",      FStringView(Msg.AssetPath))
					.Field("EventLogName",   FStringView(Msg.EventName))
					.Field("ActorLabel",     FStringView(Msg.ActorLabel))
					.Field("ActorIconName",  FStringView(Msg.ActorIconName))
					.Field("SoundClassName", FStringView(Msg.CategoryName))
					.Field("AudioComponentName", FStringView(Msg.AudioComponentName))
					.End();
			}
		};
	}
}