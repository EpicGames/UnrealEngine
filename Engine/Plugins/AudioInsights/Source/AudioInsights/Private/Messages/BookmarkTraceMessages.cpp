// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/BookmarkTraceMessages.h"

#include "ProfilingDebugging/FormatArgsTrace.h"
#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace BookmarkMessageNames
	{
		const FName Bookmark = "AudioInsightsBookmark";
	};

	FBookmarkTraceMessage::FBookmarkTraceMessage(double InTimestamp, const FString& InLabel)
		: Label(InLabel)
	{
		Timestamp = InTimestamp;
	}

	uint32 FBookmarkTraceMessage::GetSizeOf() const
	{
		uint32 MemorySize = sizeof(FBookmarkTraceMessage);

		MemorySize += Label.GetAllocatedSize();

		return MemorySize;
	}

	FCacheWriteHandler FBookmarkTraceMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				// Declare and write BookmarkSpec (once for all bookmark messages)
				const uint32 SpecEventId = Writer.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("BookmarkSpec"), UE::Trace::ETraceWriterEventFlags::ImportantNoSync)
					.Field(ANSITEXTVIEW("BookmarkPoint"), UE::Trace::ETraceWriterFieldType::Pointer)
					.Field(ANSITEXTVIEW("Line"),          UE::Trace::ETraceWriterFieldType::Int32)
					.Field(ANSITEXTVIEW("FormatString"),  UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("FileName"),      UE::Trace::ETraceWriterFieldType::AnsiString)
					.End();

				constexpr uint64 SyntheticBookmarkPoint = 1;

				Writer.WriteEvent(SpecEventId)
					.Field(ANSITEXTVIEW("BookmarkPoint"), SyntheticBookmarkPoint)
					.Field(ANSITEXTVIEW("Line"),          0)
					.Field(ANSITEXTVIEW("FormatString"),  TEXTVIEW("%s"))
					.Field(ANSITEXTVIEW("FileName"),      ANSITEXTVIEW("AudioInsightsBookmark"))
					.End();

				// Declare Bookmark event (written per cached bookmark message)
				return Writer.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("Bookmark"))
					.Field(ANSITEXTVIEW("Cycle"),         UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("BookmarkPoint"), UE::Trace::ETraceWriterFieldType::Pointer)
					.Field(ANSITEXTVIEW("FormatArgs"),    UE::Trace::ETraceWriterFieldType::Uint8 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
					.Field(ANSITEXTVIEW("CallstackId"),   UE::Trace::ETraceWriterFieldType::Uint32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FBookmarkTraceMessage& Msg = static_cast<const FBookmarkTraceMessage&>(BaseMsg);

				uint8 FormatArgsBuffer[4096];
				const uint16 FormatArgsSize = FFormatArgsTrace::EncodeArguments(FormatArgsBuffer, *Msg.Label);

				constexpr uint64 SyntheticBookmarkPoint = 1;

				Writer.WriteEvent(EventId)
					.Field(ANSITEXTVIEW("Cycle"),         TimestampCycles)
					.Field(ANSITEXTVIEW("BookmarkPoint"), SyntheticBookmarkPoint)
					.Field(ANSITEXTVIEW("FormatArgs"),    TConstArrayView<uint8>(FormatArgsBuffer, FormatArgsSize))
					.Field(ANSITEXTVIEW("CallstackId"),   0u)
					.End();
			}
		};
	}
} // namespace UE::Audio::Insights
