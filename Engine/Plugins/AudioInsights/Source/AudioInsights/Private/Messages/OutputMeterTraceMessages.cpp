// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/OutputMeterTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace OutputMeterMessageNames
	{
		const FName MainSubmixLoaded = "OutputMeterMainSubmixLoaded";
		const FName MainSubmixAlivePing = "OutputMeterMainSubmixAlivePing";
		const FName TruePeak = "OutputMeterTruePeak";
		const FName LKFSValues = "OutputMeterLKFSValues";
	};

	// FOutputMeterMessageBase
	FOutputMeterMessageBase::FOutputMeterMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId  = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		SubmixId  = EventData.GetValue<uint32>("SubmixId");
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
	}

	// FOutputMeterMainSubmixLoadedMessage
	FOutputMeterMainSubmixLoadedMessage::FOutputMeterMainSubmixLoadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FOutputMeterMessageBase(InContext)
	{
	}

	uint32 FOutputMeterMainSubmixLoadedMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FOutputMeterMainSubmixLoadedMessage);
	}

	FCacheWriteHandler FOutputMeterMainSubmixLoadedMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("MainSubmixLoaded"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FOutputMeterMainSubmixLoadedMessage& Msg = static_cast<const FOutputMeterMainSubmixLoadedMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("SubmixId",    Msg.SubmixId)
					.Field("Timestamp",   TimestampCycles)
					.End();
			}
		};
	}

	// FOutputMeterMainSubmixAlivePingMessage
	FOutputMeterMainSubmixAlivePingMessage::FOutputMeterMainSubmixAlivePingMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FOutputMeterMainSubmixLoadedMessage(InContext)
	{
	}

	uint32 FOutputMeterMainSubmixAlivePingMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FOutputMeterMainSubmixAlivePingMessage);
	}

	FCacheWriteHandler FOutputMeterMainSubmixAlivePingMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("MainSubmixAlivePing"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FOutputMeterMainSubmixAlivePingMessage& Msg = static_cast<const FOutputMeterMainSubmixAlivePingMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("SubmixId",  Msg.SubmixId)
					.Field("Timestamp", TimestampCycles)
					.End();
			}
		};
	}

	// FOutputMeterTruePeakMessage
	FOutputMeterTruePeakMessage::FOutputMeterTruePeakMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FOutputMeterMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		TruePeakMaxValueDb = EventData.GetValue<float>("TruePeakMaxValueDb");
	}
	
	uint32 FOutputMeterTruePeakMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FOutputMeterTruePeakMessage);
	}

	FCacheWriteHandler FOutputMeterTruePeakMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("MainSubmixTruePeakMaxValue"))
					.Field(ANSITEXTVIEW("DeviceId"),          UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),          UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),         UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("TruePeakMaxValueDb"),UE::Trace::ETraceWriterFieldType::Float32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FOutputMeterTruePeakMessage& Msg = static_cast<const FOutputMeterTruePeakMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",          Msg.DeviceId)
					.Field("SubmixId",          Msg.SubmixId)
					.Field("Timestamp",         TimestampCycles)
					.Field("TruePeakMaxValueDb",Msg.TruePeakMaxValueDb)
					.End();
			}
		};
	}

	// FOutputMeterLKFSValuesMessage
	FOutputMeterLKFSValuesMessage::FOutputMeterLKFSValuesMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FOutputMeterMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		LongTermLoudness  = EventData.GetValue<float>("LongTermLoudness");
		ShortTermLoudness = EventData.GetValue<float>("ShortTermLoudness");
		MomentaryLoudness = EventData.GetValue<float>("MomentaryLoudness");
		LoudnessRangeLowerBound = EventData.GetValue<float>("LoudnessRangeLowerBound");
		LoudnessRangeUpperBound = EventData.GetValue<float>("LoudnessRangeUpperBound");
	}

	uint32 FOutputMeterLKFSValuesMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FOutputMeterLKFSValuesMessage);
	}

	FCacheWriteHandler FOutputMeterLKFSValuesMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("MainSubmixLKFSValues"))
					.Field(ANSITEXTVIEW("DeviceId"),                UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),                UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),               UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("LongTermLoudness"),        UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("ShortTermLoudness"),       UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("MomentaryLoudness"),       UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("LoudnessRangeLowerBound"), UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("LoudnessRangeUpperBound"), UE::Trace::ETraceWriterFieldType::Float32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FOutputMeterLKFSValuesMessage& Msg = static_cast<const FOutputMeterLKFSValuesMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",                Msg.DeviceId)
					.Field("SubmixId",                Msg.SubmixId)
					.Field("Timestamp",               TimestampCycles)
					.Field("LongTermLoudness",        Msg.LongTermLoudness)
					.Field("ShortTermLoudness",       Msg.ShortTermLoudness)
					.Field("MomentaryLoudness",       Msg.MomentaryLoudness)
					.Field("LoudnessRangeLowerBound", Msg.LoudnessRangeLowerBound)
					.Field("LoudnessRangeUpperBound", Msg.LoudnessRangeUpperBound)
					.End();
			}
		};
	}
} // namespace UE::Audio::Insights
