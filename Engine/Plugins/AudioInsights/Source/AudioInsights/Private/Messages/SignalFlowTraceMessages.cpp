// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/SignalFlowTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace SignalFlowMessageNames
	{
		const FName SoundWaveToBusSend = "SoundToBusSend";
		const FName SoundWaveToSubmixSend = "SoundToSubmixSend";
		const FName SubmixToSubmixSend = "SubmixToSubmixSend";
		const FName BusPatchWriterConnected = "BusPatchWriterConnected";
		const FName BusPatchReaderConnected = "BusPatchReaderConnected";
	};

	namespace FSignalFlowDashboardEntryPrivate
	{
		constexpr uint32 DataPointsCapacity = 64u;
	}

	namespace FSignalFlowMessagePrivate
	{
		FCacheWriteHandler MakeSendWriteHandler(FAnsiStringView InEventName)
		{
			FAnsiString EventNameStr(InEventName);
			return {
				[EventNameStr](UE::Trace::FTraceWriter& Writer) -> uint32
				{
					return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), FAnsiStringView(*EventNameStr))
						.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
						.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
						.Field(ANSITEXTVIEW("SenderID"),    UE::Trace::ETraceWriterFieldType::Uint32)
						.Field(ANSITEXTVIEW("ReceiverIDs"), UE::Trace::ETraceWriterFieldType::Uint32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
						.Field(ANSITEXTVIEW("SendLevels"),  UE::Trace::ETraceWriterFieldType::Float32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
						.End();
				},
				[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
				{
					const FSignalFlowSendMessageBase& Msg = static_cast<const FSignalFlowSendMessageBase&>(BaseMsg);

					TArray<uint32> ReceiverIDs;
					TArray<float> SendLevels;

					ReceiverIDs.Reserve(Msg.SendInfos.Num());
					SendLevels.Reserve(Msg.SendInfos.Num());

					for (const FSignalFlowSendMessageBase::FSendInfoPair& Info : Msg.SendInfos)
					{
						ReceiverIDs.Add(Info.ReceiverID);
						SendLevels.Add(Info.SendLevel);
					}

					Writer.WriteEvent(EventId)
						.Field("DeviceId",    Msg.DeviceId)
						.Field("Timestamp",   TimestampCycles)
						.Field("SenderID",    Msg.SenderID)
						.Field("ReceiverIDs", TConstArrayView<uint32>(ReceiverIDs.GetData(), ReceiverIDs.Num()))
						.Field("SendLevels",  TConstArrayView<float>(SendLevels.GetData(), SendLevels.Num()))
						.End();
				}
			};
		}
	}

	FSignalFlowMessageBase::FSignalFlowMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
	}

	FSignalFlowSendMessageBase::FSignalFlowSendMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSignalFlowMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		SenderID = EventData.GetValue<uint32>("SenderID");

		const TArrayView<const uint32> ReceiverIDs = EventData.GetArrayView<uint32>("ReceiverIDs");
		const TArrayView<const float> SendLevels = EventData.GetArrayView<float>("SendLevels");
		if (ensure(ReceiverIDs.Num() == SendLevels.Num()))
		{
			const int32 NumSends = ReceiverIDs.Num();
			SendInfos.Reserve(NumSends);
			for (int32 SendIndex = 0; SendIndex < NumSends; ++SendIndex)
			{
				SendInfos.Add(FSendInfoPair{ ReceiverIDs[SendIndex], SendLevels[SendIndex] });
			}
		}
	}

	uint32 FSignalFlowSendMessageBase::GetSizeOf() const
	{
		return sizeof(FSignalFlowSendMessageBase) + SendInfos.GetAllocatedSize();;
	}

	FSubmixToSubmixSendMessage::FSubmixToSubmixSendMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSignalFlowMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		SenderID = EventData.GetValue<uint32>("SenderID");
		ReceiverID = EventData.GetValue<uint32>("ReceiverID");
		SendLevel = EventData.GetValue<float>("SendLevel");
		bIsFinalStage = EventData.GetValue<bool>("IsFinalStage");
	}

	uint32 FSubmixToSubmixSendMessage::GetSizeOf() const
	{
		return sizeof(FSubmixToSubmixSendMessage);
	}

	FCacheWriteHandler FSoundToBusSendMessage::GetCacheWriteHandler() const
	{
		return FSignalFlowMessagePrivate::MakeSendWriteHandler("SoundToBusSend");
	}

	FCacheWriteHandler FBusPatchWriterConnectedMessage::GetCacheWriteHandler() const
	{
		return FSignalFlowMessagePrivate::MakeSendWriteHandler("BusPatchWriterConnected");
	}

	FCacheWriteHandler FBusPatchReaderConnectedMessage::GetCacheWriteHandler() const
	{
		return FSignalFlowMessagePrivate::MakeSendWriteHandler("BusPatchReaderConnected");
	}

	FCacheWriteHandler FSoundToSubmixSendMessage::GetCacheWriteHandler() const
	{
		return FSignalFlowMessagePrivate::MakeSendWriteHandler("SoundToSubmixSend");
	}

	FCacheWriteHandler FSubmixToSubmixSendMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixToSubmixSend"))
					.Field(ANSITEXTVIEW("DeviceId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),    UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("SenderID"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ReceiverID"),   UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SendLevel"),    UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("IsFinalStage"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixToSubmixSendMessage& Msg = static_cast<const FSubmixToSubmixSendMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",     Msg.DeviceId)
					.Field("Timestamp",    TimestampCycles)
					.Field("SenderID",     Msg.SenderID)
					.Field("ReceiverID",   Msg.ReceiverID)
					.Field("SendLevel",    Msg.SendLevel)
					.Field("IsFinalStage", Msg.bIsFinalStage)
					.End();
			}
		};
	}

	FSignalFlowDashboardEntry::FSignalFlowDashboardEntry()
		: AmplitudeDataRange(FSignalFlowDashboardEntryPrivate::DataPointsCapacity, 0.0f)
		, NumConsecutiveSilentAmplitudeValues(FSignalFlowDashboardEntryPrivate::DataPointsCapacity)
	{
	}

	void FSignalFlowDashboardEntry::WriteNewAmplitudeValue(const float Value)
	{
		AmplitudeDataRange[AmplitudeWriteIndex] = Value;
		AmplitudeWriteIndex = AmplitudeDataRange.GetNextIndex(AmplitudeWriteIndex);

		constexpr float SilentAmplitudeThreshold = UE_KINDA_SMALL_NUMBER;
		if (FMath::IsNearlyZero(Value, SilentAmplitudeThreshold))
		{
			NumConsecutiveSilentAmplitudeValues = FMath::Min(NumConsecutiveSilentAmplitudeValues + 1u, FSignalFlowDashboardEntryPrivate::DataPointsCapacity);
		}
		else
		{
			NumConsecutiveSilentAmplitudeValues = 0u;
		}
	}

	void FSignalFlowDashboardEntry::ResetDataBuffers()
	{
		for (uint32 Index = 0u; Index < AmplitudeDataRange.Capacity(); ++Index)
		{
			AmplitudeDataRange[Index] = 0.0f;
		}

		AmplitudeWriteIndex = 0;
		NumConsecutiveSilentAmplitudeValues = FSignalFlowDashboardEntryPrivate::DataPointsCapacity;
	}

	bool FSignalFlowDashboardEntry::IsEntryAmplitudeWindowSilent() const
	{
		return NumConsecutiveSilentAmplitudeValues == FSignalFlowDashboardEntryPrivate::DataPointsCapacity;
	}
} // namespace UE::Audio::Insights
