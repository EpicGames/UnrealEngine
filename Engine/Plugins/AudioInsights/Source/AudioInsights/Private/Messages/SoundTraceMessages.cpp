// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/SoundTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace SoundMessageNames
	{
		const FName SoundStart = "SoundStart";
		const FName SoundIsAlivePing = "SoundIsAlivePing";
		const FName SoundWaveStart = "SoundWaveStart";
		const FName SoundWaveIsAlivePing = "SoundWaveIsAlivePing";
		const FName SoundStop = "SoundStop";
		const FName SoundWaveStop = "SoundWaveStop";

		const FName PriorityParam = "Priority";
		const FName DistanceParam = "Distance";
		const FName DistanceAttenuationParam = "DistanceAttenuation";
		const FName HPFFreqParam = "HPFFreq";
		const FName LPFFreqParam = "LPFFreq";
		const FName EnvelopeParam = "Envelope";
		const FName PitchParam = "Pitch";
		const FName VolumeParam = "Volume";
		const FName RelativeRenderCostParam = "RelativeRenderCost";
	};

	namespace SoundMessageUtils
	{
		uint64 GeneratePlayOrderUniqueID(const uint32 ActiveSoundPlayOrder, const uint32 WaveInstancePlayOrder)
		{
			return (static_cast<uint64>(ActiveSoundPlayOrder) << 32) + static_cast<uint64>(WaveInstancePlayOrder);
		}

		bool IsPriorityValueSetToMax(const float PriorityValue)
		{
			// Max priority as defined in SoundWave.cpp
			static constexpr float VolumeWeightedMaxPriority = TNumericLimits<float>::Max() / MAX_VOLUME;

			return PriorityValue >= VolumeWeightedMaxPriority;
		}
	};

	namespace FSoundMessagePrivate
	{
		FStringView GetSoundClassNameFromEntryType(ESoundDashboardEntryType InEntryType)
		{
			switch (InEntryType)
			{
				case ESoundDashboardEntryType::SoundCue:
					return SoundClassNames::SoundCue;
				case ESoundDashboardEntryType::SoundWave:
					return SoundClassNames::SoundWave;
				case ESoundDashboardEntryType::MetaSound:
					return SoundClassNames::MetaSoundSource;
				case ESoundDashboardEntryType::ProceduralSource:
					return SoundClassNames::SoundWaveProcedural;
				case ESoundDashboardEntryType::SoundCueTemplate:
					return SoundClassNames::SoundCueTemplate;
				default:
					break;
			}

			return TEXTVIEW("");
		}

		uint32 DeclareParamEvent(UE::Trace::FTraceWriter& Writer, FAnsiStringView InEventName, FAnsiStringView InParamFieldName)
		{
			return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), FAnsiStringView(InEventName))
				.Field(ANSITEXTVIEW("DeviceId"),             UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(ANSITEXTVIEW("Timestamp"),            UE::Trace::ETraceWriterFieldType::Uint64)
				.Field(ANSITEXTVIEW("PlayOrder"),            UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(ANSITEXTVIEW("ActiveSoundPlayOrder"), UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(FAnsiStringView(InParamFieldName),    UE::Trace::ETraceWriterFieldType::Float32)
				.End();
		}

		uint32 DeclareSoundSourceFiltersEvent(UE::Trace::FTraceWriter& Writer)
		{
			return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundSourceFilters"))
				.Field(ANSITEXTVIEW("DeviceId"),             UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(ANSITEXTVIEW("Timestamp"),            UE::Trace::ETraceWriterFieldType::Uint64)
				.Field(ANSITEXTVIEW("PlayOrder"),            UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(ANSITEXTVIEW("ActiveSoundPlayOrder"), UE::Trace::ETraceWriterFieldType::Uint32)
				.Field(ANSITEXTVIEW("LPFFrequency"),         UE::Trace::ETraceWriterFieldType::Float32)
				.Field(ANSITEXTVIEW("HPFFrequency"),         UE::Trace::ETraceWriterFieldType::Float32)
				.End();
		}
	};

	FSoundMessageBase::FSoundMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
	}

	FSoundStartMessage::FSoundStartMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSoundMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;
		EventData.GetString("AssetPath", Name);
		ActiveSoundPlayOrder = EventData.GetValue<uint32>("PlayOrder");

		FString SoundClassName;

		EventData.GetString("SoundClassName", SoundClassName);

		if (SoundClassName == SoundClassNames::SoundCue)
		{
			EntryType = ESoundDashboardEntryType::SoundCue;
		}
		else if (SoundClassName == SoundClassNames::SoundWave)
		{
			EntryType = ESoundDashboardEntryType::SoundWave;
		}
		else if (SoundClassName == SoundClassNames::MetaSoundSource)
		{
			EntryType = ESoundDashboardEntryType::MetaSound;
		}
		else if (SoundClassName == SoundClassNames::SoundWaveProcedural)
		{
			EntryType = ESoundDashboardEntryType::ProceduralSource;
		}
		else if (SoundClassName == SoundClassNames::SoundCueTemplate)
		{
			EntryType = ESoundDashboardEntryType::SoundCueTemplate;
		}
		else
		{
			EntryType = ESoundDashboardEntryType::None;
		}

		EventData.GetString("ActorLabel", ActorLabel);
		EventData.GetString("AudioComponentName", AudioComponentName);
	}

	uint32 FSoundStartMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FSoundStartMessage);

		// Add any dynamically sized members
		MemorySize += Name.GetAllocatedSize();
		MemorySize += ActorLabel.GetAllocatedSize();
		MemorySize += AudioComponentName.GetAllocatedSize();

		return MemorySize;
	}

	uint32 FSoundWaveStartMessage::GetSizeOf() const
	{
		uint32 MemorySize = FSoundStartMessage::GetSizeOf();
		MemorySize += sizeof(FSoundWaveStartMessage) - sizeof(FSoundStartMessage);
		MemorySize += ActorIconName.GetAllocatedSize();

		return MemorySize;
	}

	uint32 FSoundStopMessage::GetSizeOf() const
	{
		return sizeof(FSoundStopMessage);
	}

	uint32 FSoundWaveStopMessage::GetSizeOf() const
	{
		return sizeof(FSoundWaveStopMessage);
	}

	FCacheWriteHandler FSoundStartMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundStart"))
					.Field(ANSITEXTVIEW("DeviceId"),              UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),             UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),             UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),             UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("EventLogName"),          UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"),        UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),            UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorIconName"),         UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("AudioComponentName"),    UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundStartMessage& Msg = static_cast<const FSoundStartMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",              Msg.DeviceId)
					.Field("Timestamp",             TimestampCycles)
					.Field("PlayOrder",             Msg.ActiveSoundPlayOrder)
					.Field("AssetPath",             FStringView(Msg.Name))
					.Field("EventLogName",          TEXTVIEW(""))
					.Field("SoundClassName",        FSoundMessagePrivate::GetSoundClassNameFromEntryType(Msg.EntryType))
					.Field("ActorLabel",            FStringView(Msg.ActorLabel))
					.Field("ActorIconName",         TEXTVIEW(""))
					.Field("AudioComponentName",    FStringView(Msg.AudioComponentName))
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundIsAlivePingMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundIsAlivePing"))
					.Field(ANSITEXTVIEW("DeviceId"),              UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),             UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),             UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),             UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"),        UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),            UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("AudioComponentName"),    UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundIsAlivePingMessage& Msg = static_cast<const FSoundIsAlivePingMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",              Msg.DeviceId)
					.Field("Timestamp",             TimestampCycles)
					.Field("PlayOrder",             Msg.ActiveSoundPlayOrder)
					.Field("AssetPath",             FStringView(Msg.Name))
					.Field("SoundClassName",        FSoundMessagePrivate::GetSoundClassNameFromEntryType(Msg.EntryType))
					.Field("ActorLabel",            FStringView(Msg.ActorLabel))
					.Field("AudioComponentName",    FStringView(Msg.AudioComponentName))
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundWaveStartMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundWaveStart"))
					.Field(ANSITEXTVIEW("DeviceId"),              UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),             UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),             UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ActiveSoundPlayOrder"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),             UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"),        UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),            UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorIconName"),         UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SourceBusID"),           UE::Trace::ETraceWriterFieldType::Int32)
					.Field(ANSITEXTVIEW("AudioComponentName"),    UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundWaveStartMessage& Msg = static_cast<const FSoundWaveStartMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",              Msg.DeviceId)
					.Field("Timestamp",             TimestampCycles)
					.Field("PlayOrder",             Msg.WaveInstancePlayOrder)
					.Field("ActiveSoundPlayOrder",  Msg.ActiveSoundPlayOrder)
					.Field("AssetPath",             FStringView(Msg.Name))
					.Field("SoundClassName",        FSoundMessagePrivate::GetSoundClassNameFromEntryType(Msg.EntryType))
					.Field("ActorLabel",            FStringView(Msg.ActorLabel))
					.Field("ActorIconName",         FStringView(Msg.ActorIconName))
					.Field("SourceBusID",           Msg.SourceBusID)
					.Field("AudioComponentName",    FStringView(Msg.AudioComponentName))
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundWaveIsAlivePingMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundWaveIsAlivePing"))
					.Field(ANSITEXTVIEW("DeviceId"),              UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),             UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),             UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ActiveSoundPlayOrder"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),             UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"),        UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),            UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorIconName"),         UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SourceBusID"),           UE::Trace::ETraceWriterFieldType::Int32)
					.Field(ANSITEXTVIEW("AudioComponentName"),    UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundWaveIsAlivePingMessage& Msg = static_cast<const FSoundWaveIsAlivePingMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",              Msg.DeviceId)
					.Field("Timestamp",             TimestampCycles)
					.Field("PlayOrder",             Msg.WaveInstancePlayOrder)
					.Field("ActiveSoundPlayOrder",  Msg.ActiveSoundPlayOrder)
					.Field("AssetPath",             FStringView(Msg.Name))
					.Field("SoundClassName",        FSoundMessagePrivate::GetSoundClassNameFromEntryType(Msg.EntryType))
					.Field("ActorLabel",            FStringView(Msg.ActorLabel))
					.Field("ActorIconName",         FStringView(Msg.ActorIconName))
					.Field("SourceBusID",           Msg.SourceBusID)
					.Field("AudioComponentName",    FStringView(Msg.AudioComponentName))
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundStopMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundStop"))
					.Field(ANSITEXTVIEW("DeviceId"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),      UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),      UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("AssetPath"),      UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("EventLogName"),   UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("SoundClassName"), UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorLabel"),     UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("ActorIconName"),  UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundStopMessage& Msg = static_cast<const FSoundStopMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",       Msg.DeviceId)
					.Field("Timestamp",      TimestampCycles)
					.Field("PlayOrder",      Msg.ActiveSoundPlayOrder)
					.Field("AssetPath",      TEXTVIEW(""))
					.Field("EventLogName",   TEXTVIEW(""))
					.Field("SoundClassName", TEXTVIEW(""))
					.Field("ActorLabel",     TEXTVIEW(""))
					.Field("ActorIconName",  TEXTVIEW(""))
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundWaveStopMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SoundWaveStop"))
					.Field(ANSITEXTVIEW("DeviceId"),             UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),            UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),            UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ActiveSoundPlayOrder"), UE::Trace::ETraceWriterFieldType::Uint32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundWaveStopMessage& Msg = static_cast<const FSoundWaveStopMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",             Msg.DeviceId)
					.Field("Timestamp",            TimestampCycles)
					.Field("PlayOrder",            Msg.WaveInstancePlayOrder)
					.Field("ActiveSoundPlayOrder", Msg.ActiveSoundPlayOrder)
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundLPFFreqMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32 { return FSoundMessagePrivate::DeclareSoundSourceFiltersEvent(Writer); },
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundLPFFreqMessage& Msg = static_cast<const FSoundLPFFreqMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",             Msg.DeviceId)
					.Field("Timestamp",            TimestampCycles)
					.Field("PlayOrder",            Msg.WaveInstancePlayOrder)
					.Field("ActiveSoundPlayOrder", Msg.ActiveSoundPlayOrder)
					.Field("LPFFrequency",         Msg.LPFFrequency)
					.Field("HPFFrequency",         MIN_FILTER_FREQUENCY)
					.End();
			}
		};
	}

	FCacheWriteHandler FSoundHPFFreqMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32 { return FSoundMessagePrivate::DeclareSoundSourceFiltersEvent(Writer); },
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSoundHPFFreqMessage& Msg = static_cast<const FSoundHPFFreqMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",             Msg.DeviceId)
					.Field("Timestamp",            TimestampCycles)
					.Field("PlayOrder",            Msg.WaveInstancePlayOrder)
					.Field("ActiveSoundPlayOrder", Msg.ActiveSoundPlayOrder)
					.Field("LPFFrequency",         MAX_FILTER_FREQUENCY)
					.Field("HPFFrequency",         Msg.HPFFrequency)
					.End();
			}
		};
	}

	// Macro-generated param message implementations use a shared helper
	// DEFINE_SOUND_PARAM_MESSAGE generates the struct but GetCacheWriteHandler must be defined per-type
#define IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(ClassName, TraceEventName, ParamFieldName, ParamMember)						\
	FCacheWriteHandler ClassName::GetCacheWriteHandler() const																\
	{																														\
		return {																											\
			[](UE::Trace::FTraceWriter& Writer) -> uint32																	\
			{																												\
				FAnsiString EventNameStr(TraceEventName);																	\
				FAnsiString FieldNameStr(ParamFieldName);																	\
				return FSoundMessagePrivate::DeclareParamEvent(Writer, EventNameStr, FieldNameStr);							\
			},																												\
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)							\
			{																												\
				const FSoundParameterMessage& Msg = static_cast<const FSoundParameterMessage&>(BaseMsg);					\
				Writer.WriteEvent(EventId)																					\
					.Field("DeviceId",             Msg.DeviceId)															\
					.Field("Timestamp",            TimestampCycles)			\
					.Field("PlayOrder",            Msg.WaveInstancePlayOrder)												\
					.Field("ActiveSoundPlayOrder", Msg.ActiveSoundPlayOrder)												\
					.Field(4, static_cast<const ClassName&>(BaseMsg).ParamMember)											\
					.End();																									\
			}																												\
		};																													\
	}

	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundPriorityMessage, "SoundPriority", "Priority", Priority)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundDistanceMessage, "SoundDistance", "Distance", Distance)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundDistanceAttenuationMessage, "SoundDistanceAttenuation", "DistanceAttenuation", DistanceAttenuation)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundVolumeMessage, "SoundVolume", "Volume", Volume)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundPitchMessage, "SoundPitch", "Pitch", Pitch)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundEnvelopeMessage, "SoundSourceEnvelope", "Envelope", Envelope)
	IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER(FSoundRelativeRenderCostMessage, "SoundRelativeRenderCost", "RelativeRenderCost", RelativeRenderCost)
#undef IMPLEMENT_PARAM_GET_CACHE_WRITE_HANDLER

	FSoundDashboardEntry::FSoundDashboardEntry()
	{
		constexpr uint32 DataPointsCapacity = 256u;

		PriorityDataRange.SetCapacity(DataPointsCapacity);
		DistanceDataRange.SetCapacity(DataPointsCapacity);
		DistanceAttenuationDataRange.SetCapacity(DataPointsCapacity);
		LPFFreqDataRange.SetCapacity(DataPointsCapacity);
		HPFFreqDataRange.SetCapacity(DataPointsCapacity);
		AmplitudeDataRange.SetCapacity(DataPointsCapacity);
		VolumeDataRange.SetCapacity(DataPointsCapacity);
		PitchDataRange.SetCapacity(DataPointsCapacity);
		RelativeRenderCostDataRange.SetCapacity(DataPointsCapacity);
	}

	void FSoundDashboardEntry::ResetDataBuffers(const uint32 DataPointsCapacity)
	{
		PriorityDataRange.Reset(DataPointsCapacity);
		DistanceDataRange.Reset(DataPointsCapacity);
		DistanceAttenuationDataRange.Reset(DataPointsCapacity);
		LPFFreqDataRange.Reset(DataPointsCapacity);
		HPFFreqDataRange.Reset(DataPointsCapacity);
		AmplitudeDataRange.Reset(DataPointsCapacity);
		VolumeDataRange.Reset(DataPointsCapacity);
		PitchDataRange.Reset(DataPointsCapacity);
		RelativeRenderCostDataRange.Reset(DataPointsCapacity);
	}
}
