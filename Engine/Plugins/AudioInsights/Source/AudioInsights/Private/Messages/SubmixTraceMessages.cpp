// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/SubmixTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace SubmixMessageNames
	{
		const FName Loaded = "SubmixLoaded";
		const FName IsAlivePing = "SubmixAlivePing";
		const FName HasActivity = "SubmixHasActivity";
		const FName EnvelopeFollowerEnabled = "SubmixEnvelopeFollowerEnabled";
		const FName EnvelopeValues = "SubmixEnvelopeValues";
		const FName Unloaded = "SubmixUnloaded";
	}

	// FSubmixMessageBase
	FSubmixMessageBase::FSubmixMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		SubmixId = EventData.GetValue<uint32>("SubmixId");
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
	}

	// FSubmixLoadedMessage
	FSubmixLoadedMessage::FSubmixLoadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSubmixMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		EventData.GetString("Name", Name);
		NumChannels = EventData.GetValue<int32>("NumChannels");
		bIsMainSubmix = EventData.GetValue<bool>("IsMainSubmix");
	}

	uint32 FSubmixLoadedMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FSubmixLoadedMessage);

		// Add any dynamically sized members
		MemorySize += Name.GetAllocatedSize();

		return MemorySize;
	};

	FCacheWriteHandler FSubmixLoadedMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixLoaded"))
					.Field(ANSITEXTVIEW("DeviceId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),    UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Name"),         UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("NumChannels"),  UE::Trace::ETraceWriterFieldType::Int32)
					.Field(ANSITEXTVIEW("IsMainSubmix"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixLoadedMessage& Msg = static_cast<const FSubmixLoadedMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",     Msg.DeviceId)
					.Field("SubmixId",     Msg.SubmixId)
					.Field("Timestamp",    TimestampCycles)
					.Field("Name",         FStringView(Msg.Name))
					.Field("NumChannels",  Msg.NumChannels)
					.Field("IsMainSubmix", Msg.bIsMainSubmix)
					.End();
			}
		};
	}

	FCacheWriteHandler FSubmixAlivePingMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixAlivePing"))
					.Field(ANSITEXTVIEW("DeviceId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),     UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),    UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Name"),         UE::Trace::ETraceWriterFieldType::WideString)
					.Field(ANSITEXTVIEW("NumChannels"),  UE::Trace::ETraceWriterFieldType::Int32)
					.Field(ANSITEXTVIEW("IsMainSubmix"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixAlivePingMessage& Msg = static_cast<const FSubmixAlivePingMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",     Msg.DeviceId)
					.Field("SubmixId",     Msg.SubmixId)
					.Field("Timestamp",    TimestampCycles)
					.Field("Name",         FStringView(Msg.Name))
					.Field("NumChannels",  Msg.NumChannels)
					.Field("IsMainSubmix", Msg.bIsMainSubmix)
					.End();
			}
		};
	}

	// FSubmixHasActivityMessage
	FSubmixHasActivityMessage::FSubmixHasActivityMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSubmixMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		bHasActivity = EventData.GetValue<bool>("HasActivity");
	}

	FCacheWriteHandler FSubmixHasActivityMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixHasActivity"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("HasActivity"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixHasActivityMessage& Msg = static_cast<const FSubmixHasActivityMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("SubmixId",    Msg.SubmixId)
					.Field("Timestamp",   TimestampCycles)
					.Field("HasActivity", Msg.bHasActivity)
					.End();
			}
		};
	}

	// FSubmixEnvelopeFollowerEnabledMessage
	FSubmixEnvelopeFollowerEnabledMessage::FSubmixEnvelopeFollowerEnabledMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSubmixMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		bEnvelopeFollowerEnabled = EventData.GetValue<bool>("EnvelopeFollowerEnabled");
	}

	FCacheWriteHandler FSubmixEnvelopeFollowerEnabledMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixEnvelopeFollowerEnabled"))
					.Field(ANSITEXTVIEW("DeviceId"),                UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),                UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),               UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("EnvelopeFollowerEnabled"), UE::Trace::ETraceWriterFieldType::Bool)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixEnvelopeFollowerEnabledMessage& Msg = static_cast<const FSubmixEnvelopeFollowerEnabledMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",                Msg.DeviceId)
					.Field("SubmixId",                Msg.SubmixId)
					.Field("Timestamp",               TimestampCycles)
					.Field("EnvelopeFollowerEnabled", Msg.bEnvelopeFollowerEnabled)
					.End();
			}
		};
	}

	// FSubmixEnvelopeValuesMessage
	FSubmixEnvelopeValuesMessage::FSubmixEnvelopeValuesMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSubmixMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		const TArrayView<const float> EnvelopeValuesView = EventData.GetArrayView<float>("EnvelopeValues");

		for (const auto& EnvelopeValue : EnvelopeValuesView)
		{
			EnvelopeValues.Emplace(EnvelopeValue);
		}
	}

	uint32 FSubmixEnvelopeValuesMessage::GetSizeOf() const
	{
		// Start with the fixed size of the struct
		uint32 MemorySize = sizeof(FSubmixEnvelopeValuesMessage);

		// Add any dynamically sized members
		if (EnvelopeValues.Num() > 0)
		{
			MemorySize += EnvelopeValues.Num() * sizeof(EnvelopeValues[0]);
		}

		return MemorySize;
	};

	FCacheWriteHandler FSubmixEnvelopeValuesMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixEnvelopeValues"))
					.Field(ANSITEXTVIEW("DeviceId"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),      UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("EnvelopeValues"), UE::Trace::ETraceWriterFieldType::Float32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixEnvelopeValuesMessage& Msg = static_cast<const FSubmixEnvelopeValuesMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",       Msg.DeviceId)
					.Field("SubmixId",       Msg.SubmixId)
					.Field("Timestamp",      TimestampCycles)
					.Field("EnvelopeValues", TConstArrayView<float>(Msg.EnvelopeValues.GetData(), Msg.EnvelopeValues.Num()))
					.End();
			}
		};
	}

	// FSubmixUnloadedMessage
	FSubmixUnloadedMessage::FSubmixUnloadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FSubmixMessageBase(InContext)
	{

	};

	FCacheWriteHandler FSubmixUnloadedMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("SubmixUnloaded"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("SubmixId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FSubmixUnloadedMessage& Msg = static_cast<const FSubmixUnloadedMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("SubmixId",  Msg.SubmixId)
					.Field("Timestamp", TimestampCycles)
					.End();
			}
		};
	}

	// FSubmixDashboardEntry
	FSubmixDashboardEntry::FSubmixDashboardEntry()
		: AudioMeterInfo(MakeShared<FAudioMeterInfo>())
	{
	}

	FSubmixDashboardEntry::FSubmixDashboardEntry(FSubmixDashboardEntry& Other)
		: SubmixId(Other.SubmixId)
		, Timestamp(Other.Timestamp)
		, AudioMeterInfo(MakeShared<FAudioMeterInfo>(*Other.AudioMeterInfo))
		, Name(Other.Name)
		, bHasActivity(Other.bHasActivity)
		, bEnvelopeFollowerEnabled(Other.bEnvelopeFollowerEnabled)
		, bIsMainSubmix(Other.bIsMainSubmix)
	{
	}

	FSubmixDashboardEntry& FSubmixDashboardEntry::operator=(const FSubmixDashboardEntry& Other)
	{
		if (this == &Other)
		{
			return *this;
		}

		SubmixId = Other.SubmixId;
		Timestamp = Other.Timestamp;
		AudioMeterInfo = MakeShared<FAudioMeterInfo>(*Other.AudioMeterInfo);
		Name = Other.Name;
		bHasActivity = Other.bHasActivity;
		bEnvelopeFollowerEnabled = Other.bEnvelopeFollowerEnabled;
		bIsMainSubmix = Other.bIsMainSubmix;

		return *this;
	}
} // namespace UE::Audio::Insights
