// Copyright Epic Games, Inc. All Rights Reserved.
#include "Messages/VirtualLoopTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace VirtualLoopMessageNames
	{
		extern const FName Virtualize    = "VirtualLoopVirtualize";
		extern const FName StopOrRealize = "VirtualLoopStopOrRealize";
		extern const FName Update        = "VirtualLoopUpdate";
	};

	// FVirtualLoopMessageBase
	FVirtualLoopMessageBase::FVirtualLoopMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
		PlayOrder = EventData.GetValue<uint32>("PlayOrder");
	}

	// FVirtualLoopVirtualizeMessage
	FVirtualLoopVirtualizeMessage::FVirtualLoopVirtualizeMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FVirtualLoopMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		EventData.GetString("Name", Name);
		ComponentId = EventData.GetValue<uint64>("ComponentId");
	}
	
	uint32 FVirtualLoopVirtualizeMessage::GetSizeOf() const
	{
		// Fixed size of the struct
		uint32 MemorySize = sizeof(FVirtualLoopVirtualizeMessage);

		// Adde dynamically sized members
		MemorySize += Name.GetAllocatedSize();

		return MemorySize;
	}

	FCacheWriteHandler FVirtualLoopVirtualizeMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("VirtualLoopVirtualize"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),   UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ComponentId"), UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Name"),        UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FVirtualLoopVirtualizeMessage& Msg = static_cast<const FVirtualLoopVirtualizeMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("Timestamp",   TimestampCycles)
					.Field("PlayOrder",   Msg.PlayOrder)
					.Field("ComponentId", Msg.ComponentId)
					.Field("Name",        FStringView(Msg.Name))
					.End();
			}
		};
	}

	// FVirtualLoopStopOrRealizeMessage
	FVirtualLoopStopOrRealizeMessage::FVirtualLoopStopOrRealizeMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FVirtualLoopMessageBase(InContext)
	{
	}

	uint32 FVirtualLoopStopOrRealizeMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FVirtualLoopStopOrRealizeMessage);
	}

	FCacheWriteHandler FVirtualLoopStopOrRealizeMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("VirtualLoopStopOrRealize"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"), UE::Trace::ETraceWriterFieldType::Uint32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FVirtualLoopStopOrRealizeMessage& Msg = static_cast<const FVirtualLoopStopOrRealizeMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("Timestamp", TimestampCycles)
					.Field("PlayOrder", Msg.PlayOrder)
					.End();
			}
		};
	}

	// FVirtualLoopUpdateMessage
	FVirtualLoopUpdateMessage::FVirtualLoopUpdateMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
		: FVirtualLoopMessageBase(InContext)
	{
		const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

		TimeVirtualized = EventData.GetValue<float>("TimeVirtualized");
		PlaybackTime = EventData.GetValue<float>("PlaybackTime");
		UpdateInterval = EventData.GetValue<float>("UpdateInterval");

		LocationX = EventData.GetValue<double>("LocationX");
		LocationY = EventData.GetValue<double>("LocationY");
		LocationZ = EventData.GetValue<double>("LocationZ");

		RotatorPitch = EventData.GetValue<double>("RotatorPitch");
		RotatorYaw = EventData.GetValue<double>("RotatorYaw");
		RotatorRoll = EventData.GetValue<double>("RotatorRoll");
	}

	uint32 FVirtualLoopUpdateMessage::GetSizeOf() const
	{
		// Returning fixed size of the struct (no dynamically sized members in it)
		return sizeof(FVirtualLoopUpdateMessage);
	}

	FCacheWriteHandler FVirtualLoopUpdateMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("VirtualLoopUpdate"))
					.Field(ANSITEXTVIEW("DeviceId"),        UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),       UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("PlayOrder"),       UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("TimeVirtualized"), UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("PlaybackTime"),    UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("UpdateInterval"),  UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("LocationX"),       UE::Trace::ETraceWriterFieldType::Float64)
					.Field(ANSITEXTVIEW("LocationY"),       UE::Trace::ETraceWriterFieldType::Float64)
					.Field(ANSITEXTVIEW("LocationZ"),       UE::Trace::ETraceWriterFieldType::Float64)
					.Field(ANSITEXTVIEW("RotatorPitch"),    UE::Trace::ETraceWriterFieldType::Float64)
					.Field(ANSITEXTVIEW("RotatorYaw"),      UE::Trace::ETraceWriterFieldType::Float64)
					.Field(ANSITEXTVIEW("RotatorRoll"),     UE::Trace::ETraceWriterFieldType::Float64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FVirtualLoopUpdateMessage& Msg = static_cast<const FVirtualLoopUpdateMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",        Msg.DeviceId)
					.Field("Timestamp",       TimestampCycles)
					.Field("PlayOrder",       Msg.PlayOrder)
					.Field("TimeVirtualized", Msg.TimeVirtualized)
					.Field("PlaybackTime",    Msg.PlaybackTime)
					.Field("UpdateInterval",  Msg.UpdateInterval)
					.Field("LocationX",       Msg.LocationX)
					.Field("LocationY",       Msg.LocationY)
					.Field("LocationZ",       Msg.LocationZ)
					.Field("RotatorPitch",    Msg.RotatorPitch)
					.Field("RotatorYaw",      Msg.RotatorYaw)
					.Field("RotatorRoll",     Msg.RotatorRoll)
					.End();
			}
		};
	}
} // namespace UE::Audio::Insights
