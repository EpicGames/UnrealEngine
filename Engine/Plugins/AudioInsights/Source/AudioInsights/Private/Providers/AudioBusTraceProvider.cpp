// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/AudioBusTraceProvider.h"

#include "Async/ParallelFor.h"
#include "AudioInsightsLog.h"
#include "AudioInsightsModule.h"
#include "Cache/AudioInsightsCacheManager.h"

#if WITH_EDITOR
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#else
#include "Common/PagedArray.h"
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace FAudioBusTraceProviderPrivate
	{
#if WITH_EDITOR
		TObjectPtr<UAudioBus> ResolveAudioBusFromPath(const FString& InAssetPath)
		{
			const FSoftObjectPath SoftPath(InAssetPath);

			TObjectPtr<UAudioBus> AudioBus = Cast<UAudioBus>(SoftPath.ResolveObject());
			if (AudioBus == nullptr && !IsGarbageCollecting())
			{
				AudioBus = Cast<UAudioBus>(SoftPath.TryLoad());
			}

			return AudioBus;
		}

		TSharedPtr<FAudioBusDashboardEntry> CreateEntryFromAsset(const FString& InAssetPath, const TObjectPtr<UAudioBus> InAudioBus)
		{
			TSharedPtr<FAudioBusDashboardEntry> Entry = MakeShared<FAudioBusDashboardEntry>();

			Entry->Timestamp = FPlatformTime::Seconds() - GStartTime;
			Entry->AudioBusId = InAudioBus ? InAudioBus->GetUniqueID() : INDEX_NONE;

			Entry->Name = InAssetPath;

			Entry->AudioMeterInfo->NumChannels = InAudioBus ? InAudioBus->GetNumChannels() : 0;
			Entry->AudioBusType = EAudioBusType::AssetBased;

			return Entry;
		}

		// Returns true when an existing entry's Name was upgraded to InAssetPath. False when an entry was newly created or already matched.
		bool ApplyAssetToEntry(TSharedPtr<FAudioBusDashboardEntry>& InOutEntry, const FString& InAssetPath, const TObjectPtr<UAudioBus> InAudioBus)
		{
			if (!InOutEntry.IsValid())
			{
				InOutEntry = CreateEntryFromAsset(InAssetPath, InAudioBus);
				return false;
			}

			if (InOutEntry->Name == InAssetPath)
			{
				return false;
			}

			InOutEntry->Name = InAssetPath;
			InOutEntry->AudioBusType = EAudioBusType::AssetBased;

			if (InOutEntry->AudioMeterInfo->NumChannels == 0 && InAudioBus)
			{
				InOutEntry->AudioMeterInfo->NumChannels = InAudioBus->GetNumChannels();
			}

			return true;
		}
#endif // WITH_EDITOR
	}

	FAudioBusTraceProvider::FAudioBusTraceProvider()
		: TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioBusDashboardEntry>>(GetName_Static())
	{
#if WITH_EDITOR
		AudioBusAssetProvider.OnAssetAdded.BindRaw(this, &FAudioBusTraceProvider::HandleOnAudioBusAssetAdded);
		AudioBusAssetProvider.OnAssetRemoved.BindRaw(this, &FAudioBusTraceProvider::HandleOnAudioBusAssetRemoved);
		AudioBusAssetProvider.OnAssetListUpdated.BindRaw(this, &FAudioBusTraceProvider::HandleOnAudioBusAssetListUpdated);
#endif // WITH_EDITOR
	}

	FAudioBusTraceProvider::~FAudioBusTraceProvider()
	{
#if WITH_EDITOR
		AudioBusAssetProvider.OnAssetAdded.Unbind();
		AudioBusAssetProvider.OnAssetRemoved.Unbind();
		AudioBusAssetProvider.OnAssetListUpdated.Unbind();
#endif // WITH_EDITOR
	}
	
	FName FAudioBusTraceProvider::GetName_Static()
	{
		static const FLazyName AudioBusTraceProviderName = "AudioBusProvider";
		return AudioBusTraceProviderName;
	}

	UE::Trace::IAnalyzer* FAudioBusTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FAudioBusTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FAudioBusTraceAnalyzer(TSharedRef<FAudioBusTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

				Builder.RouteEvent(RouteId_Start,                   "Audio", "AudioBusStart");
				Builder.RouteEvent(RouteId_HasActivity,             "Audio", "AudioBusHasActivity");
				Builder.RouteEvent(RouteId_EnvelopeFollowerEnabled, "Audio", "AudioBusEnvelopeFollowerEnabled");
				Builder.RouteEvent(RouteId_EnvelopeValues,          "Audio", "AudioBusEnvelopeValues");
				Builder.RouteEvent(RouteId_Stop,                    "Audio", "AudioBusStop");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FAudioBusTraceAnalyzer"));

				FAudioBusMessages& Messages = GetProvider<FAudioBusTraceProvider>().TraceMessages;

				switch (RouteId)
				{
					case RouteId_Start:
					{
						CacheMessage<FAudioBusStartMessage>(Context, Messages.StartMessages);
						break;
					}

					case RouteId_HasActivity:
					{
						CacheMessage<FAudioBusHasActivityMessage>(Context, Messages.HasActivityMessages);
						break;
					}

					case RouteId_EnvelopeFollowerEnabled:
					{
						CacheMessage<FAudioBusEnvelopeFollowerEnabledMessage>(Context, Messages.EnvelopeFollowerEnabledMessages);
						break;
					}

					case RouteId_EnvelopeValues:
					{
						CacheMessage<FAudioBusEnvelopeValuesMessage>(Context, Messages.EnvelopeValuesMessages);
						break;
					}

					case RouteId_Stop:
					{
						CacheMessage<FAudioBusStopMessage>(Context, Messages.StopMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_Start,
				RouteId_HasActivity,
				RouteId_EnvelopeFollowerEnabled,
				RouteId_EnvelopeValues,
				RouteId_Stop
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FAudioBusTraceAnalyzer(AsShared(), InSession);
	}

	void FAudioBusTraceProvider::OnTraceChannelsEnabled()
	{
		FTraceProviderBase::OnTraceChannelsEnabled();

#if WITH_EDITOR
		AudioBusAssetProvider.RequestEntriesUpdate();
		OnAudioBusListUpdated.ExecuteIfBound();
#endif // WITH_EDITOR
	}

	bool FAudioBusTraceProvider::ProcessMessages()
	{
		// Helper lambdas
		auto CreateEntry = [this](const FAudioBusMessageBase& Msg)
		{
			TSharedPtr<FAudioBusDashboardEntry>* ToReturn = nullptr;
			
			UpdateDeviceEntry(Msg.DeviceId, Msg.AudioBusId, [&ToReturn, &Msg](TSharedPtr<FAudioBusDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FAudioBusDashboardEntry>();
					Entry->AudioBusId = Msg.AudioBusId;
				}

				Entry->Timestamp = Msg.Timestamp;

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		auto GetEntry = [this](const FAudioBusMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, Msg.AudioBusId);
		};

		// Process messages
		ProcessMessageQueue<FAudioBusStartMessage>(TraceMessages.StartMessages, CreateEntry,
		[this](const FAudioBusStartMessage& Msg, TSharedPtr<FAudioBusDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FAudioBusDashboardEntry& EntryRef = *OutEntry->Get();

				if (EntryRef.Name.IsEmpty())
				{
					EntryRef.Name = Msg.Name;
				}

				if (EntryRef.AudioMeterInfo->NumChannels == 0)
				{
					EntryRef.AudioMeterInfo->NumChannels = Msg.NumChannels;
				}

				if (EntryRef.AudioBusType == EAudioBusType::None)
				{
					EntryRef.AudioBusType = Msg.AudioBusType;
				}

				OnAudioBusStarted.ExecuteIfBound(Msg.AudioBusId);
			}
		});

		ProcessMessageQueue<FAudioBusHasActivityMessage>(TraceMessages.HasActivityMessages, GetEntry,
		[this](const FAudioBusHasActivityMessage& Msg, TSharedPtr<FAudioBusDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FAudioBusDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.bHasActivity = Msg.bHasActivity;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FAudioBusEnvelopeFollowerEnabledMessage>(TraceMessages.EnvelopeFollowerEnabledMessages, GetEntry,
		[this](const FAudioBusEnvelopeFollowerEnabledMessage& Msg, TSharedPtr<FAudioBusDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FAudioBusDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.bEnvelopeFollowerEnabled = Msg.bEnvelopeFollowerEnabled;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FAudioBusEnvelopeValuesMessage>(TraceMessages.EnvelopeValuesMessages, GetEntry,
		[this](const FAudioBusEnvelopeValuesMessage& Msg, TSharedPtr<FAudioBusDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FAudioBusDashboardEntry& EntryRef = *OutEntry->Get();

				EntryRef.AudioMeterInfo->EnvelopeValues = Msg.EnvelopeValues;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FAudioBusStopMessage>(TraceMessages.StopMessages, GetEntry,
		[this](const FAudioBusStopMessage& Msg, TSharedPtr<FAudioBusDashboardEntry>* OutEntry)
		{
			if (OutEntry && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
#if WITH_EDITOR
				// Turn off audio bus entry activity, do not remove it since we are in Editor mode.
				// We keep in the dashboard all registered audio bus assets from Content Browser
				FAudioBusDashboardEntry& EntryRef = *OutEntry->Get();

				EntryRef.bHasActivity = false;

				TArray<float>& EnvelopeValues = EntryRef.AudioMeterInfo->EnvelopeValues;
				FMemory::Memzero(EnvelopeValues.GetData(), EnvelopeValues.Num() * sizeof(float));

				if (EntryRef.AudioBusType == EAudioBusType::CodeGenerated)
#endif  // WITH_EDITOR
				{
					OnAudioBusRemoved.ExecuteIfBound(Msg.AudioBusId);
					RemoveDeviceEntry(Msg.DeviceId, Msg.AudioBusId);
				}
			}
		});

#if WITH_EDITOR
		if (bAssetsUpdated)
		{
			OnAudioBusListUpdated.ExecuteIfBound();
			bAssetsUpdated = false;
		}
#endif // WITH_EDITOR

		return true;
	}

#if WITH_EDITOR
	void FAudioBusTraceProvider::RequestEntriesUpdate()
	{
		AudioBusAssetProvider.RequestEntriesUpdate();
	}

	void FAudioBusTraceProvider::HandleOnAudioBusAssetAdded(const FString& InAssetPath)
	{
		using namespace FAudioBusTraceProviderPrivate;

		const TObjectPtr<UAudioBus> AudioBus = ResolveAudioBusFromPath(InAssetPath);
		if (AudioBus == nullptr)
		{
			UE_LOG(LogAudioInsights, Verbose, TEXT("HandleOnAudioBusAssetAdded: failed to resolve UAudioBus from path '%s'"), *InAssetPath);
			return;
		}

		const uint32 AudioBusId = AudioBus->GetUniqueID();
		const bool bUpgraded = ApplyAssetEntryToAllDevices(InAssetPath, AudioBus, AudioBusId);

		++LastUpdateId;
		OnAudioBusAdded.ExecuteIfBound(AudioBusId);
		bAssetsUpdated = true;

		if (bUpgraded)
		{
			OnAudioBusNameResolved.Broadcast(AudioBusId);
		}
	}

	void FAudioBusTraceProvider::HandleOnAudioBusAssetRemoved(const FString& InAssetPath)
	{
		const FAudioDeviceManager* const DeviceManager = FAudioDeviceManager::Get();
		if (DeviceManager == nullptr)
		{
			return;
		}

		// The UAudioBus is already gone by this point, so match the entry by its Name.
		for (const FAudioDevice* AudioDevice : DeviceManager->GetAudioDevices())
		{
			if (AudioDevice == nullptr)
			{
				continue;
			}

			const FDeviceData* const DeviceData = GetDeviceDataMap().Find(AudioDevice->DeviceID);
			if (DeviceData == nullptr)
			{
				continue;
			}

			uint32 AudioBusIdToRemove = INDEX_NONE;
			for (const FEntryPair& Pair : *DeviceData)
			{
				if (Pair.Value.IsValid() && Pair.Value->Name == InAssetPath)
				{
					AudioBusIdToRemove = Pair.Key;
					break;
				}
			}

			if (AudioBusIdToRemove != static_cast<uint32>(INDEX_NONE))
			{
				RemoveDeviceEntry(AudioDevice->DeviceID, AudioBusIdToRemove);

				++LastUpdateId;
				OnAudioBusRemoved.ExecuteIfBound(AudioBusIdToRemove);
				bAssetsUpdated = true;
			}
		}
	}

	void FAudioBusTraceProvider::HandleOnAudioBusAssetListUpdated(const TArray<FString>& InAssetPaths)
	{
		using namespace FAudioBusTraceProviderPrivate;

		for (const FString& AssetPath : InAssetPaths)
		{
			const TObjectPtr<UAudioBus> AudioBus = ResolveAudioBusFromPath(AssetPath);
			if (AudioBus == nullptr)
			{
				UE_LOG(LogAudioInsights, Verbose, TEXT("HandleOnAudioBusAssetListUpdated: failed to resolve UAudioBus from path '%s'"), *AssetPath);
				continue;
			}

			const uint32 AudioBusId = AudioBus->GetUniqueID();
			const bool bUpgraded = ApplyAssetEntryToAllDevices(AssetPath, AudioBus, AudioBusId);
			if (bUpgraded)
			{
				OnAudioBusNameResolved.Broadcast(AudioBusId);
			}
		}

		++LastUpdateId;
		bAssetsUpdated = true;
	}

	bool FAudioBusTraceProvider::ApplyAssetEntryToAllDevices(const FString& InAssetPath, const TObjectPtr<UAudioBus> InAudioBus, const uint32 InAudioBusId)
	{
		using namespace FAudioBusTraceProviderPrivate;

		const FAudioDeviceManager* const DeviceManager = FAudioDeviceManager::Get();
		if (DeviceManager == nullptr)
		{
			return false;
		}

		bool bAnyEntryUpgraded = false;

		for (const FAudioDevice* const AudioDevice : DeviceManager->GetAudioDevices())
		{
			if (AudioDevice == nullptr)
			{
				continue;
			}

			UpdateDeviceEntry(AudioDevice->DeviceID, InAudioBusId, [&InAssetPath, &InAudioBus, &bAnyEntryUpgraded](TSharedPtr<FAudioBusDashboardEntry>& Entry)
			{
				if (ApplyAssetToEntry(Entry, InAssetPath, InAudioBus))
				{
					bAnyEntryUpgraded = true;
				}
			});
		}

		return bAnyEntryUpgraded;
	}
#endif // WITH_EDITOR

	void FAudioBusTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		using namespace FAudioBusTraceProviderPrivate;

		DeviceDataMap.Empty();

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		// Collect all the audio bus start messages registered until this point in time 
		auto ProcessBusStartMessage = [this](const FAudioBusStartMessage& Msg)
		{
			UpdateDeviceEntry(Msg.DeviceId, Msg.AudioBusId, [&Msg](TSharedPtr<FAudioBusDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FAudioBusDashboardEntry>();
					Entry->AudioBusId = Msg.AudioBusId;
				}

				Entry->Timestamp = Msg.Timestamp;

				Entry->Name = *Msg.Name;
				Entry->AudioMeterInfo->NumChannels = Msg.NumChannels;
				Entry->AudioBusType = Msg.AudioBusType;
			});
		};

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();
		CacheManager.IterateOverRange<FAudioBusStartMessage>(AudioBusMessageNames::Start, RangeCheckStart, TimeMarker, [&ProcessBusStartMessage](const FAudioBusStartMessage& AudioBusStartCachedMessage)
		{
			ProcessBusStartMessage(AudioBusStartCachedMessage);
		});

		// Selectively remove start messages collected in the step above by knowing which audio buses were stopped.
		// With this we will know what are the available audio buses at this point in time.
		auto ProcessBusStopMessage = [this](const FAudioBusStopMessage& Msg)
		{
			auto* OutEntry = FindDeviceEntry(Msg.DeviceId, Msg.AudioBusId);

			if (OutEntry && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
				RemoveDeviceEntry(Msg.DeviceId, Msg.AudioBusId);
			}
		};

		CacheManager.IterateOverRange<FAudioBusStopMessage>(AudioBusMessageNames::Stop, RangeCheckStart, TimeMarker, [&ProcessBusStopMessage](const FAudioBusStopMessage& AudioBusStopCachedMessage)
		{
			ProcessBusStopMessage(AudioBusStopCachedMessage);
		});

		const FDeviceData* DeviceData = FindFilteredDeviceData();
		if (DeviceData)
		{
			// Collect has activity messages from audio buses (based on active audio buses AudioBusId)
			struct CachedEntryInfo
			{
				TOptional<FAudioBusHasActivityMessage> HasActivityMessage;
				TOptional<FAudioBusEnvelopeFollowerEnabledMessage> EnvelopeFollowerEnabledMessage;
				TOptional<FAudioBusEnvelopeValuesMessage> EnvelopeValuesMessage;
			};

			TArray<uint32> AudioBusIdArray;
			(*DeviceData).GenerateKeyArray(AudioBusIdArray);

			TArray<CachedEntryInfo> CachedEntryInfos;
			CachedEntryInfos.SetNum(AudioBusIdArray.Num());

			ParallelFor(AudioBusIdArray.Num(), [&AudioBusIdArray, &CachedEntryInfos, &CacheManager, TimeMarker, this](const int32 Index)
			{
				const uint32 AudioBusId = AudioBusIdArray[Index];

				// HasActivity
				const FAudioBusHasActivityMessage* FoundHasActivityCachedMessage = CacheManager.FindClosestMessage<FAudioBusHasActivityMessage>(AudioBusMessageNames::HasActivity, TimeMarker, AudioBusId);
				if (FoundHasActivityCachedMessage)
				{
					CachedEntryInfos[Index].HasActivityMessage = *FoundHasActivityCachedMessage;
				}

				// EnvelopeFollowerEnabled
				const FAudioBusEnvelopeFollowerEnabledMessage* FoundEnvelopeFollowerEnabledCachedMessage = CacheManager.FindClosestMessage<FAudioBusEnvelopeFollowerEnabledMessage>(AudioBusMessageNames::EnvelopeFollowerEnabled, TimeMarker, AudioBusId);
				if (FoundEnvelopeFollowerEnabledCachedMessage)
				{
					CachedEntryInfos[Index].EnvelopeFollowerEnabledMessage = *FoundEnvelopeFollowerEnabledCachedMessage;
				}

				// Envelope Values
				const FAudioBusEnvelopeValuesMessage* FoundEnvelopeValuesCachedMessage = CacheManager.FindClosestMessage<FAudioBusEnvelopeValuesMessage>(AudioBusMessageNames::EnvelopeValues, TimeMarker, AudioBusId);
				if (FoundEnvelopeValuesCachedMessage)
				{
					CachedEntryInfos[Index].EnvelopeValuesMessage = *FoundEnvelopeValuesCachedMessage;
				}
			});

			// Update the device entries with the collected info
			for (const CachedEntryInfo& CachedEntryInfo : CachedEntryInfos)
			{
				if (CachedEntryInfo.HasActivityMessage.IsSet())
				{
					const FAudioBusHasActivityMessage& HasActivityMessage = CachedEntryInfo.HasActivityMessage.GetValue();

					UpdateDeviceEntry(HasActivityMessage.DeviceId, HasActivityMessage.AudioBusId, [&HasActivityMessage](TSharedPtr<FAudioBusDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FAudioBusDashboardEntry>();
							Entry->AudioBusId = HasActivityMessage.AudioBusId;
						}

						Entry->Timestamp = HasActivityMessage.Timestamp;

						Entry->bHasActivity = HasActivityMessage.bHasActivity;
					});
				}

				if (CachedEntryInfo.EnvelopeFollowerEnabledMessage.IsSet())
				{
					const FAudioBusEnvelopeFollowerEnabledMessage& EnvelopeFollowerEnabledMessage = CachedEntryInfo.EnvelopeFollowerEnabledMessage.GetValue();

					UpdateDeviceEntry(EnvelopeFollowerEnabledMessage.DeviceId, EnvelopeFollowerEnabledMessage.AudioBusId, [&EnvelopeFollowerEnabledMessage](TSharedPtr<FAudioBusDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FAudioBusDashboardEntry>();
							Entry->AudioBusId = EnvelopeFollowerEnabledMessage.AudioBusId;
						}

						Entry->Timestamp = EnvelopeFollowerEnabledMessage.Timestamp;

						Entry->bEnvelopeFollowerEnabled = EnvelopeFollowerEnabledMessage.bEnvelopeFollowerEnabled;
					});
				}

				if (CachedEntryInfo.EnvelopeValuesMessage.IsSet())
				{
					const FAudioBusEnvelopeValuesMessage& EnvelopeValuesMessage = CachedEntryInfo.EnvelopeValuesMessage.GetValue();

					UpdateDeviceEntry(EnvelopeValuesMessage.DeviceId, EnvelopeValuesMessage.AudioBusId, [&EnvelopeValuesMessage](TSharedPtr<FAudioBusDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FAudioBusDashboardEntry>();
							Entry->AudioBusId = EnvelopeValuesMessage.AudioBusId;
						}

						Entry->Timestamp = EnvelopeValuesMessage.Timestamp;

						Entry->AudioMeterInfo->NumChannels = EnvelopeValuesMessage.EnvelopeValues.Num();
						Entry->AudioMeterInfo->EnvelopeValues = EnvelopeValuesMessage.EnvelopeValues;
					});
				}
			}
		}

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);

#if WITH_EDITOR
		// Reapply asset-resolved names; the rebuild above pulled Name from cached trace messages.
		AudioBusAssetProvider.RequestEntriesUpdate();
#endif // WITH_EDITOR

		OnTimeMarkerUpdated.ExecuteIfBound();
	}

#if WITH_EDITOR
	void FAudioBusTraceProvider::OnTimeControlMethodReset()
	{
		AudioBusAssetProvider.RequestEntriesUpdate();
		OnAudioBusListUpdated.ExecuteIfBound();
	}

	FString FAudioBusTraceProvider::GetResolvedBusName(const ::Audio::FDeviceId InDeviceId, const uint32 InAudioBusId, const FString& InFallbackName) const
	{
		const TSharedPtr<FAudioBusDashboardEntry>* const FoundEntry = FindDeviceEntry(InDeviceId, InAudioBusId);
		if (FoundEntry && FoundEntry->IsValid() && !(*FoundEntry)->Name.IsEmpty())
		{
			return (*FoundEntry)->Name;
		}

		return InFallbackName;
	}
#endif // WITH_EDITOR
} // namespace UE::Audio::Insights
