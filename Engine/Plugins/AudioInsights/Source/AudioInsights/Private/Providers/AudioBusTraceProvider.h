// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/AudioBusTraceMessages.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "Delegates/Delegate.h"
#include "Providers/AssetProvider.h"
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	class FAudioBusTraceProvider : public TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioBusDashboardEntry>>, public TSharedFromThis<FAudioBusTraceProvider>
	{
	public:
		FAudioBusTraceProvider();
		virtual ~FAudioBusTraceProvider();

		static FName GetName_Static();
		
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		void RequestEntriesUpdate();

		DECLARE_DELEGATE_OneParam(FOnAudioBusChanged, const uint32 /*AudioBusId*/);
		FOnAudioBusChanged OnAudioBusAdded;
		FOnAudioBusChanged OnAudioBusRemoved;
		FOnAudioBusChanged OnAudioBusStarted;

		DECLARE_DELEGATE(FOnAudioBusListUpdated);
		FOnAudioBusListUpdated OnAudioBusListUpdated;

		DECLARE_DELEGATE(FOnTimeMarkerUpdated);
		FOnTimeMarkerUpdated OnTimeMarkerUpdated;

#if WITH_EDITOR
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioBusNameResolved, uint32 /*AudioBusId*/);
		FOnAudioBusNameResolved OnAudioBusNameResolved;

		FString GetResolvedBusName(const ::Audio::FDeviceId InDeviceId, const uint32 InAudioBusId, const FString& InFallbackName) const;
#endif // WITH_EDITOR

	protected:
		virtual void OnTraceChannelsEnabled() override;

	private:
		virtual bool ProcessMessages() override;

#if WITH_EDITOR
		virtual void OnTimeControlMethodReset() override;

		void HandleOnAudioBusAssetAdded(const FString& InAssetPath);
		void HandleOnAudioBusAssetRemoved(const FString& InAssetPath);
		void HandleOnAudioBusAssetListUpdated(const TArray<FString>& InAssetPaths);

		// Returns true if any existing entry's Name was upgraded to InAssetPath.
		bool ApplyAssetEntryToAllDevices(const FString& InAssetPath, const TObjectPtr<UAudioBus> InAudioBus, const uint32 InAudioBusId);
#endif // WITH_EDITOR

		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;

		FAudioBusMessages TraceMessages;

#if WITH_EDITOR
		TAssetProvider<UAudioBus> AudioBusAssetProvider;
		bool bAssetsUpdated = false;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights
