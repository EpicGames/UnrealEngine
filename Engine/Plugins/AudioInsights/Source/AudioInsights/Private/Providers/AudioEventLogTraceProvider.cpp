// Copyright Epic Games, Inc. All Rights Reserved.
#include "Providers/AudioEventLogTraceProvider.h"

#include "AudioInsightsTimingViewExtender.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/ModuleService.h"

#if WITH_EDITOR
#include "AudioInsightsLog.h"
#endif 

#if !WITH_EDITOR
#include "AudioInsightsComponent.h"
#include "AudioInsightsModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FAudioEventLogTraceProvider::FAudioEventLogTraceProvider()
		: TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioEventLogDashboardEntry>>(GetName_Static())
	{
		FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
		CacheManager.OnChunkOverwritten.AddRaw(this, &FAudioEventLogTraceProvider::OnCacheChunkOverwritten);
	}

	FAudioEventLogTraceProvider::~FAudioEventLogTraceProvider()
	{
		if (FModuleManager::Get().IsModuleLoaded("AudioInsights"))
		{
			FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();
			CacheManager.OnChunkOverwritten.RemoveAll(this);
		}
	}

	UE::Trace::IAnalyzer* FAudioEventLogTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FAudioEventLogTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FAudioEventLogTraceAnalyzer(TSharedRef<FAudioEventLogTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
				Builder.RouteEvent(RouteId_EventLog, "Audio", "EventLog");
				Builder.RouteEvent(RouteId_SoundStart, "Audio", "SoundStart");
				Builder.RouteEvent(RouteId_SoundStop, "Audio", "SoundStop");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FAudioEventLogTraceAnalyzer"));

				FAudioEventLogMessages& Messages = GetProvider<FAudioEventLogTraceProvider>().TraceMessages;
				switch (RouteId)
				{
					case RouteId_EventLog:
					case RouteId_SoundStart:
					case RouteId_SoundStop:
					{
						CacheMessage<FAudioEventLogMessage>(Context, Messages.AudioEventLogMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		protected:
			virtual bool ShouldProcessNewEvents() const override
			{
				return GetProvider<FAudioEventLogTraceProvider>().GetMessageCacheAndProcessingStatus() != ECacheAndProcess::None;
			}

		private:
			enum : uint16
			{
				RouteId_EventLog,
				RouteId_SoundStart,
				RouteId_SoundStop
			};

			TraceServices::IAnalysisSession& Session;
		};

		PlayOrderToActorInfoMap.Empty();

		CacheStartTimestamp = 0.0;
		bCacheChunksUpdated = false;

		return new FAudioEventLogTraceAnalyzer(AsShared(), InSession);
	}

	FName FAudioEventLogTraceProvider::GetName_Static()
	{
		return "AudioEventLogProvider";
	}

	bool FAudioEventLogTraceProvider::IsMessageProcessingPaused() const
	{
		return GetMessageCacheAndProcessingStatus() == ECacheAndProcess::None;
	}

	bool FAudioEventLogTraceProvider::ProcessMessages()
	{
		auto AddEntryFunc = [this](const FAudioEventLogMessage& Msg)
		{
			TSharedPtr<FAudioEventLogDashboardEntry>* ToReturn = nullptr;
			UpdateDeviceEntry(Msg.DeviceId, Msg.MessageID, [&ToReturn, &Msg, this](TSharedPtr<FAudioEventLogDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = CreateNewEntry(Msg);
				}
				else
				{
#if WITH_EDITOR
					UE_LOGF(LogAudioInsights, Warning, "Duplicate entry processed inside FAudioEventLogTraceProvider - each entry should be unique");
#endif // WITH_EDITOR
					Entry->Timestamp = Msg.Timestamp;
				}

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		ProcessMessageQueue<FAudioEventLogMessage>(TraceMessages.AudioEventLogMessages, AddEntryFunc,
		[](const FAudioEventLogMessage& Msg, TSharedPtr<FAudioEventLogDashboardEntry>* OutEntry)
		{
			// Do nothing
		});

		return true;
	}

	bool FAudioEventLogTraceProvider::ProcessManuallyUpdatedEntries()
	{
		if (!bCacheChunksUpdated)
		{
			return false;
		}

#if !WITH_EDITOR
		if (!IAudioInsightsModule::IsLiveSession())
		{
			// Don't update the caching and processing status if the cache isn't limited (i.e. loading in a .urace file)
			bCacheChunksUpdated = false;
			return false;
		}
#endif // !WITH_EDITOR


		const double AboutToBeDeletedTimestamp = GetAboutToBeDeletedFromCacheTimeThreshold();

		FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		const uint32 HalfNumChunks = CacheManager.GetNumChunks() / 2;
		ensure(HalfNumChunks > 0u);
		if (HalfNumChunks == 0u)
		{
			return false;
		}

		bool bAnyEntriesRemoved = false;
		for (auto& [DeviceID, MessageIDToDashboardEntryMap] : DeviceDataMap)
		{
			TArray<uint32> MessagesToRemove;
			MessagesToRemove.Reserve(MessageIDToDashboardEntryMap.Num() / HalfNumChunks); // Reserve 2x average num messages per chunk to give us some wiggle room

			for (auto& [MessageID, DashboardEntry] : MessageIDToDashboardEntryMap)
			{
				if (DashboardEntry->Timestamp < CacheStartTimestamp)
				{
					MessagesToRemove.Add(MessageID);
				}
				else if (DashboardEntry->Timestamp <= AboutToBeDeletedTimestamp)
				{
					DashboardEntry->CachedState = EAudioEventCacheState::NextToBeDeleted;
					DashboardEntry->bCacheStatusIsDirty = true; // Mark as dirty so UI will update deleted rows and cache status icon at the same time
				}
				else
				{
					break;
				}
			}

			for (const uint32 MessageID : MessagesToRemove)
			{
				MessageIDToDashboardEntryMap.Remove(MessageID);
			}

			bAnyEntriesRemoved = bAnyEntriesRemoved || MessagesToRemove.Num() > 0;
		}

		bCacheChunksUpdated = false;
		return bAnyEntriesRemoved;
	}

	TSharedPtr<FAudioEventLogDashboardEntry> FAudioEventLogTraceProvider::CreateNewEntry(const FAudioEventLogMessage& Msg)
	{
		TSharedPtr<FAudioEventLogDashboardEntry> Entry = MakeShared<FAudioEventLogDashboardEntry>();

		Entry->MessageID = Msg.MessageID;
		Entry->DeviceId = Msg.DeviceId;
		Entry->Timestamp = Msg.Timestamp;
		Entry->PlayOrder = Msg.PlayOrder;
		Entry->CategoryName = Msg.CategoryName;
		Entry->Category = Msg.CategoryType;
		Entry->EventName = Msg.EventName;
		Entry->Name = Msg.AssetPath;

		// If this message did not provide an actor label, the object may have been destroying when the message was sent
		// Try to locate the actor label using the play order ID
		if (Msg.ActorLabel.IsEmpty())
		{
			if (const FActorInformation* FoundActorInfo = PlayOrderToActorInfoMap.Find(Entry->PlayOrder))
			{
				Entry->ActorLabel = *FoundActorInfo->ActorLabel;
				Entry->AudioComponentName = FoundActorInfo->AudioComponentName;
#if WITH_EDITOR
				Entry->ActorIconName = FoundActorInfo->ActorIconName;
#else
				Entry->ActorIconName = NAME_None;
#endif // WITH_EDITOR
			}
			else
			{
				Entry->ActorLabel = LOCTEXT("AudioEventLog_None", "None").ToString();
				Entry->ActorIconName = NAME_None;
			}
		}
		else
		{
			Entry->ActorLabel = Msg.ActorLabel;
			Entry->AudioComponentName = Msg.AudioComponentName;

#if WITH_EDITOR
			Entry->ActorIconName = FName(Msg.ActorIconName);
#else
			Entry->ActorIconName = NAME_None;
#endif // WITH_EDITOR
			
			// Cache the actor information associated with this PlayOrder ID so we can look it up later
			PlayOrderToActorInfoMap.Add(Entry->PlayOrder, FActorInformation{ Msg.ActorLabel, FName(Msg.ActorIconName), Msg.AudioComponentName, Msg.DeviceId });
		}

		return Entry;
	}

	void FAudioEventLogTraceProvider::OnCacheChunkOverwritten(const double NewCacheStartTimestamp)
	{
		CacheStartTimestamp = NewCacheStartTimestamp;
		bCacheChunksUpdated = true;
	}

	double FAudioEventLogTraceProvider::GetAboutToBeDeletedFromCacheTimeThreshold() const
	{
		FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		constexpr int32 NumChunksDeletedAtOnce = 1;

		const int32 NumUsedChunks = CacheManager.GetNumUsedChunks();
		const int32 MaxNumChunks = CacheManager.GetNumChunks();

		ensure(MaxNumChunks > NumChunksDeletedAtOnce);
		if (MaxNumChunks <= NumChunksDeletedAtOnce)
		{
			return TNumericLimits<double>::Lowest();
		}

		// We don't mark anything as leaving the cache if the cache isn't close to overwriting chunks yet
		if (NumUsedChunks < (MaxNumChunks - NumChunksDeletedAtOnce))
		{
			return TNumericLimits<double>::Lowest();
		}

		const int32 BeingDeletedChunkIndex = NumUsedChunks - MaxNumChunks + NumChunksDeletedAtOnce - 1;

		const FAudioCachedMessageChunk* const NextToBeDeletedChunk = BeingDeletedChunkIndex >= 0 ? CacheManager.GetChunk(BeingDeletedChunkIndex) : nullptr;

		return NextToBeDeletedChunk ? NextToBeDeletedChunk->GetChunkTimeRangeEnd() : TNumericLimits<double>::Lowest();
	}

	void FAudioEventLogTraceProvider::ClearActiveAudioDeviceEntries()
	{
#if WITH_EDITOR
		const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
		const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();
#else
		const ::Audio::FDeviceId AudioDeviceId = GetAudioDeviceIdFromDeviceDataMap();
#endif // WITH_EDITOR

		for (auto It = PlayOrderToActorInfoMap.CreateIterator(); It; ++It)
		{
			if (It.Value().AudioDeviceId == AudioDeviceId)
			{
				It.RemoveCurrent();
			}
		}

		Super::ClearActiveAudioDeviceEntries();
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE