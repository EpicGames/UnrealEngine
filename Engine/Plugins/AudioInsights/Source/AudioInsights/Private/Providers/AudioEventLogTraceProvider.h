// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/AudioEventLogTraceMessages.h"

namespace UE::Audio::Insights
{
	class FAudioEventLogTraceProvider
		: public TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioEventLogDashboardEntry>>
		, public TSharedFromThis<FAudioEventLogTraceProvider>
	{
	public:
		FAudioEventLogTraceProvider();
		virtual ~FAudioEventLogTraceProvider();

		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		static FName GetName_Static();

		virtual void ClearActiveAudioDeviceEntries() override;

	protected:
		virtual bool IsMessageProcessingPaused() const override;
		virtual void OnTraceChannelsEnabled() override { /* Do Nothing */ }

	private:
		virtual bool ProcessMessages() override;
		virtual bool ProcessManuallyUpdatedEntries() override;

		TSharedPtr<FAudioEventLogDashboardEntry> CreateNewEntry(const FAudioEventLogMessage& Msg);

		void OnCacheChunkOverwritten(const double NewCacheStartTimestamp);
		double GetAboutToBeDeletedFromCacheTimeThreshold() const;

		FAudioEventLogMessages TraceMessages;

		struct FActorInformation
		{
			const FString ActorLabel;
			const FName ActorIconName;
			const FString AudioComponentName;
			const ::Audio::FDeviceId AudioDeviceId = 0;
		};

		TMap<uint32, FActorInformation> PlayOrderToActorInfoMap;

		double CacheStartTimestamp = 0.0;
		bool bCacheChunksUpdated = false;
	};
} // namespace UE::Audio::Insights
