// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"

#define UE_API AUDIOINSIGHTS_API

class FText;
struct FSlateColor;

namespace UE::Audio::Insights
{
	struct FAudioMeterInfo
	{
		int32 NumChannels = 0;
		TArray<float> EnvelopeValues;
	};

	class FAudioMeterProvider
	{
	public:
		DECLARE_DELEGATE_FourParams(FOnAddAudioMeter, const uint32 /*EntryId*/, const TSharedRef<FAudioMeterInfo>& /*AudioMeterInfo*/, const FText& /*Name*/, const FSlateColor& /*NameColor*/);
		UE_API inline static FOnAddAudioMeter OnAddAudioMeter;

		DECLARE_DELEGATE_OneParam(FOnRemoveAudioMeter, const uint32 /*EntryId*/);
		UE_API inline static FOnRemoveAudioMeter OnRemoveAudioMeter;

		DECLARE_DELEGATE_ThreeParams(FOnUpdateAudioMeterInfo, const uint32 /*EntryId*/, const TSharedRef<FAudioMeterInfo>& /*AudioMeterInfo*/, const FText& /*Name*/);
		UE_API inline static FOnUpdateAudioMeterInfo OnUpdateAudioMeterInfo;
	};
} // namespace UE::Audio::Insights

#undef UE_API
