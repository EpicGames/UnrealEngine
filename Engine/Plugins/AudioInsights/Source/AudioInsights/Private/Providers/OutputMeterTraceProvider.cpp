// Copyright Epic Games, Inc. All Rights Reserved.
#include "Providers/OutputMeterTraceProvider.h"

#include "AudioInsightsModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::Audio::Insights
{
	FName FOutputMeterTraceProvider::GetName_Static()
	{
		return "OutputMeterProvider";
	}

	UE::Trace::IAnalyzer* FOutputMeterTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FOutputMeterTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FOutputMeterTraceAnalyzer(TSharedRef<FOutputMeterTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
				
				Builder.RouteEvent(RouteId_MainSubmixLoaded,    "Audio", "MainSubmixLoaded");
				Builder.RouteEvent(RouteId_MainSubmixAlivePing, "Audio", "MainSubmixAlivePing");
				Builder.RouteEvent(RouteId_TruePeakMaxValue,    "Audio", "MainSubmixTruePeakMaxValue");
				Builder.RouteEvent(RouteId_LKFSValues,          "Audio", "MainSubmixLKFSValues");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FOutputMeterTraceAnalyzer"));

				FOutputMeterMessages& Messages = GetProvider<FOutputMeterTraceProvider>().TraceMessages;
				
				switch (RouteId)
				{
					case RouteId_MainSubmixLoaded:
					{
						CacheMessage<FOutputMeterMainSubmixLoadedMessage>(Context, Messages.MainSubmixLoadedMessages);
						break;
					}

					case RouteId_MainSubmixAlivePing:
					{
						CacheMessage<FOutputMeterMainSubmixAlivePingMessage>(Context, Messages.MainSubmixAlivePingMessages);
						break;
					}

					case RouteId_TruePeakMaxValue:
					{
						CacheMessage<FOutputMeterTruePeakMessage>(Context, Messages.TruePeakMessages);
						break;
					}

					case RouteId_LKFSValues:
					{
						CacheMessage<FOutputMeterLKFSValuesMessage>(Context, Messages.LKFSValuesMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_MainSubmixLoaded,
				RouteId_MainSubmixAlivePing,
				RouteId_TruePeakMaxValue,
				RouteId_LKFSValues
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FOutputMeterTraceAnalyzer(AsShared(), InSession);
	}

	bool FOutputMeterTraceProvider::ProcessMessages()
	{
		auto GetOrCreateEntryFunc = [this](const FOutputMeterMessageBase& Msg)
		{
			TSharedPtr<FOutputMeterDashboardEntry>* ToReturn = nullptr;
			UpdateDeviceEntry(Msg.DeviceId, Msg.SubmixId, [&ToReturn, &Msg](TSharedPtr<FOutputMeterDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FOutputMeterDashboardEntry>();
					Entry->DeviceId  = Msg.DeviceId;
					Entry->SubmixId = Msg.SubmixId;
				}
				Entry->Timestamp = Msg.Timestamp;

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		ProcessMessageQueue<FOutputMeterMainSubmixLoadedMessage>(TraceMessages.MainSubmixLoadedMessages, GetOrCreateEntryFunc,
		[this](const FOutputMeterMainSubmixLoadedMessage& Msg, TSharedPtr<FOutputMeterDashboardEntry>* OutEntry)
		{
		});

		ProcessMessageQueue<FOutputMeterMainSubmixAlivePingMessage>(TraceMessages.MainSubmixAlivePingMessages, GetOrCreateEntryFunc,
		[this](const FOutputMeterMainSubmixAlivePingMessage& Msg, TSharedPtr<FOutputMeterDashboardEntry>* OutEntry)
		{
		});

		ProcessMessageQueue<FOutputMeterTruePeakMessage>(TraceMessages.TruePeakMessages, GetOrCreateEntryFunc,
		[this](const FOutputMeterTruePeakMessage& Msg, TSharedPtr<FOutputMeterDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FOutputMeterDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.TruePeakMaxValueDb = Msg.TruePeakMaxValueDb;
			}
		});

		ProcessMessageQueue<FOutputMeterLKFSValuesMessage>(TraceMessages.LKFSValuesMessages, GetOrCreateEntryFunc,
		[this](const FOutputMeterLKFSValuesMessage& Msg, TSharedPtr<FOutputMeterDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FOutputMeterDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.LongTermLoudness  = Msg.LongTermLoudness;
				EntryRef.ShortTermLoudness = Msg.ShortTermLoudness;
				EntryRef.MomentaryLoudness = Msg.MomentaryLoudness;
				EntryRef.LoudnessRangeLowerBound = Msg.LoudnessRangeLowerBound;
				EntryRef.LoudnessRangeUpperBound = Msg.LoudnessRangeUpperBound;
			}
		});

		return true;
	}

	void FOutputMeterTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		DeviceDataMap.Empty();

		// Collect all main submix loaded or alive ping messages registered until this point in time 
		auto ProcessLoadedOrAlivePingMessage = [this](const FOutputMeterMessageBase& InMsg)
		{
			UpdateDeviceEntry(InMsg.DeviceId, InMsg.SubmixId, [&InMsg](TSharedPtr<FOutputMeterDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FOutputMeterDashboardEntry>();
					Entry->DeviceId = InMsg.DeviceId;
					Entry->SubmixId = InMsg.SubmixId;
				}

				Entry->Timestamp = InMsg.Timestamp;
			});
		};

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		CacheManager.IterateOverRange<FOutputMeterMainSubmixLoadedMessage>(OutputMeterMessageNames::MainSubmixLoaded, RangeCheckStart, TimeMarker, [&ProcessLoadedOrAlivePingMessage](const FOutputMeterMainSubmixLoadedMessage& InMsg)
		{
			ProcessLoadedOrAlivePingMessage(InMsg);
		});

		CacheManager.IterateOverRange<FOutputMeterMainSubmixAlivePingMessage>(OutputMeterMessageNames::MainSubmixAlivePing, RangeCheckStart, TimeMarker, [&ProcessLoadedOrAlivePingMessage](const FOutputMeterMainSubmixAlivePingMessage& InMsg)
		{
			ProcessLoadedOrAlivePingMessage(InMsg);
		});

		// DeviceDataMap is now populated, resolve the filtered device and populate each submix entry with the closest TruePeak and LKFS values.
		if (const FDeviceData* DeviceData = FindFilteredDeviceData())
		{
			const ::Audio::FDeviceId FilteredDeviceId = GetFilteredAudioDeviceID();

			for (const auto& [SubmixId, DashboardEntry] : *DeviceData)
			{
				auto MatchesDeviceIdAndSubmix = [FilteredDeviceId, InSubmixId = SubmixId](const FOutputMeterMessageBase& InMsg)
				{
					return InMsg.DeviceId == FilteredDeviceId && InMsg.SubmixId == InSubmixId;
				};

				// Find closest TruePeak message for this submix at the given time marker
				if (const FOutputMeterTruePeakMessage* TruePeakMsg = CacheManager.FindClosestMessageByPredicate<FOutputMeterTruePeakMessage>(OutputMeterMessageNames::TruePeak, TimeMarker, MatchesDeviceIdAndSubmix))
				{
					UpdateDeviceEntry(TruePeakMsg->DeviceId, TruePeakMsg->SubmixId, [TruePeakMsg](TSharedPtr<FOutputMeterDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FOutputMeterDashboardEntry>();
							Entry->DeviceId = TruePeakMsg->DeviceId;
							Entry->SubmixId = TruePeakMsg->SubmixId;
						}

						Entry->Timestamp          = TruePeakMsg->Timestamp;
						Entry->TruePeakMaxValueDb = TruePeakMsg->TruePeakMaxValueDb;
					});
				}

				// Find closest LKFS message for this submix at the given time marker
				if (const FOutputMeterLKFSValuesMessage* LKFSMsg = CacheManager.FindClosestMessageByPredicate<FOutputMeterLKFSValuesMessage>(OutputMeterMessageNames::LKFSValues, TimeMarker, MatchesDeviceIdAndSubmix))
				{
					UpdateDeviceEntry(LKFSMsg->DeviceId, LKFSMsg->SubmixId, [LKFSMsg](TSharedPtr<FOutputMeterDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FOutputMeterDashboardEntry>();
							Entry->DeviceId = LKFSMsg->DeviceId;
							Entry->SubmixId = LKFSMsg->SubmixId;
						}

						Entry->Timestamp               = LKFSMsg->Timestamp;
						Entry->LongTermLoudness        = LKFSMsg->LongTermLoudness;
						Entry->ShortTermLoudness       = LKFSMsg->ShortTermLoudness;
						Entry->MomentaryLoudness       = LKFSMsg->MomentaryLoudness;
						Entry->LoudnessRangeLowerBound = LKFSMsg->LoudnessRangeLowerBound;
						Entry->LoudnessRangeUpperBound = LKFSMsg->LoudnessRangeUpperBound;
					});
				}
			}
		}

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);
	}
} // namespace UE::Audio::Insights
