// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/OutputMeterTraceMessages.h"

namespace UE::Audio::Insights
{
	class FOutputMeterTraceProvider
		: public TDeviceDataMapTraceProvider<uint32, TSharedPtr<FOutputMeterDashboardEntry>>
		, public TSharedFromThis<FOutputMeterTraceProvider>
	{
	public:
		FOutputMeterTraceProvider()
			: TDeviceDataMapTraceProvider<uint32, TSharedPtr<FOutputMeterDashboardEntry>>(GetName_Static())
		{
		}

		virtual ~FOutputMeterTraceProvider() = default;
		
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		static FName GetName_Static();

	private:
		virtual bool ProcessMessages() override;
		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;

		FOutputMeterMessages TraceMessages;
	};
} // namespace UE::Audio::Insights
