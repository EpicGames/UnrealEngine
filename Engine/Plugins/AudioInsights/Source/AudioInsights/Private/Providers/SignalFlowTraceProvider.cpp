// Copyright Epic Games, Inc. All Rights Reserved.
#include "Providers/SignalFlowTraceProvider.h"

#include "Async/ParallelFor.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsTraceModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "IAudioInsightsModule.h"
#include "Providers/AudioBusTraceProvider.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FSignalFlowTraceProviderPrivate
	{
		const FName AudioDeviceIconName = "AudioInsights.Icon.Dashboard";
		const FName AudioBusIconName = "AudioInsights.Icon";
		const FName SubmixIconName = "AudioInsights.Icon.Submix";

		const FName MetasoundIconName = "AudioInsights.Icon.SoundDashboard.MetaSound";
		const FName ProceduralSourceIconName = "AudioInsights.Icon.SoundDashboard.ProceduralSource";
		const FName SoundWaveIconName = "AudioInsights.Icon.SoundDashboard.SoundWave";
		const FName SoundCueIconName = "AudioInsights.Icon.SoundDashboard.SoundCue";
		const FName SoundCueTemplateIconName = "AudioInsights.Icon.SoundDashboard.SoundCue";

		FName GetSourceIconName(const ESoundDashboardEntryType SoundEntryType)
		{
			switch (SoundEntryType)
			{
				case ESoundDashboardEntryType::MetaSound:
					return MetasoundIconName;
				case ESoundDashboardEntryType::SoundCue:
					return SoundCueIconName;
				case ESoundDashboardEntryType::ProceduralSource:
					return ProceduralSourceIconName;
				case ESoundDashboardEntryType::SoundWave:
					return SoundWaveIconName;
				case ESoundDashboardEntryType::SoundCueTemplate:
					return SoundCueTemplateIconName;
			}
			
			return SoundWaveIconName;
		}
	} // FSignalFlowTraceProviderPrivate

	FSignalFlowTraceProvider::FSignalFlowTraceProvider()
		: TDeviceDataMapTraceProvider<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>(GetName_Static())
	{
	}

	FName FSignalFlowTraceProvider::GetName_Static()
	{
		static const FLazyName SignalFlowTraceProviderName = "SignalFlowProvider";
		return SignalFlowTraceProviderName;
	}

	UE::Trace::IAnalyzer* FSignalFlowTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FSignalFlowTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FSignalFlowTraceAnalyzer(TSharedRef<FSignalFlowTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

				Builder.RouteEvent(RouteID_SoundWaveStart, "Audio", "SoundWaveStart");
				Builder.RouteEvent(RouteID_SoundWaveIsAlivePing, "Audio", "SoundWaveIsAlivePing");
				Builder.RouteEvent(RouteID_SoundWaveStop, "Audio", "SoundWaveStop");
				Builder.RouteEvent(RouteID_ActiveSoundStop, "Audio", "SoundStop");

				Builder.RouteEvent(RouteId_BusStart, "Audio", "AudioBusStart");
				Builder.RouteEvent(RouteId_BusIsAlivePing, "Audio", "AudioBusIsAlivePing");
				Builder.RouteEvent(RouteId_BusStop, "Audio", "AudioBusStop");

				Builder.RouteEvent(RouteId_SubmixLoaded, "Audio", "SubmixLoaded");
				Builder.RouteEvent(RouteId_SubmixAlivePing, "Audio", "SubmixAlivePing");
				Builder.RouteEvent(RouteId_SubmixUnloaded, "Audio", "SubmixUnloaded");

				Builder.RouteEvent(RouteId_SoundToBusSend, "Audio", "SoundToBusSend");
				Builder.RouteEvent(RouteId_BusPatchWriterConnected, "Audio", "BusPatchWriterConnected");
				Builder.RouteEvent(RouteId_BusPatchReaderConnected, "Audio", "BusPatchReaderConnected");
				Builder.RouteEvent(RouteId_SoundToSubmixSend, "Audio", "SoundToSubmixSend");
				Builder.RouteEvent(RouteId_SubmixToSubmixSend, "Audio", "SubmixToSubmixSend");

				Builder.RouteEvent(RouteId_SoundSourceEnvelope, "Audio", "SoundSourceEnvelope");
				Builder.RouteEvent(RouteId_SoundSourceVolume, "Audio", "SoundVolume");
				Builder.RouteEvent(RouteId_SoundSourcePitch, "Audio", "SoundPitch");
				Builder.RouteEvent(RouteId_SoundSourceFilters, "Audio", "SoundSourceFilters");
				Builder.RouteEvent(RouteId_SoundSourcePriority, "Audio", "SoundPriority");
				Builder.RouteEvent(RouteId_SoundSourceDistance, "Audio", "SoundDistance");
				Builder.RouteEvent(RouteId_SoundSourceAttenuation, "Audio", "SoundDistanceAttenuation");
				Builder.RouteEvent(RouteId_SoundSourceRelativeRenderCost, "Audio", "SoundRelativeRenderCost");

				Builder.RouteEvent(RouteId_AudioBusEnvelope, "Audio", "AudioBusEnvelopeValues");
				Builder.RouteEvent(RouteId_SubmixEnvelope, "Audio", "SubmixEnvelopeValues");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FSignalFlowTraceAnalyzer"));

				FSignalFlowMessages& Messages = GetProvider<FSignalFlowTraceProvider>().TraceMessages;

				switch (RouteId)
				{
					// The first set of messages here are also used in other providers.
					// We do not need to add these messages to the cache multiple times.
					// Call AddMessageToQueue instead to just add them to the processing queue.
					case RouteID_SoundWaveStart:
					{
						AddMessageToQueue<FSoundWaveStartMessage>(Context, Messages.SoundWaveStartMessages);
						break;
					}

					case RouteID_SoundWaveIsAlivePing:
					{
						AddMessageToQueue<FSoundWaveIsAlivePingMessage>(Context, Messages.SoundWaveIsAlivePingMessages);
						break;
					}

					case RouteID_ActiveSoundStop:
					{
						AddMessageToQueue<FSoundStopMessage>(Context, Messages.SoundStopMessages);
						break;
					}

					case RouteId_BusStart:
					{
						AddMessageToQueue<FAudioBusStartMessage>(Context, Messages.BusStartMessages);
						break;
					}

					case RouteId_BusStop:
					{
						AddMessageToQueue<FAudioBusStopMessage>(Context, Messages.BusStopMessages);
						break;
					}

					case RouteId_SubmixLoaded:
					{
						AddMessageToQueue<FSubmixLoadedMessage>(Context, Messages.SubmixLoadedMessages);
						break;
					}

					case RouteId_SubmixAlivePing:
					{
						AddMessageToQueue<FSubmixAlivePingMessage>(Context, Messages.SubmixAliveMessages);
						break;
					}

					case RouteId_SubmixUnloaded:
					{
						AddMessageToQueue<FSubmixUnloadedMessage>(Context, Messages.SubmixUnloadedMessages);
						break;
					}

					case RouteId_SoundSourceEnvelope:
					{
						AddMessageToQueue<FSoundEnvelopeMessage>(Context, Messages.SoundEnvelopeMessages);
						break;
					}

					case RouteId_SoundSourceVolume:
					{
						AddMessageToQueue<FSoundVolumeMessage>(Context, Messages.SoundVolumeMessages);
						break;
					}

					case RouteId_SoundSourcePitch:
					{
						AddMessageToQueue<FSoundPitchMessage>(Context, Messages.SoundPitchMessages);
						break;
					}

					case RouteId_SoundSourceFilters:
					{
						AddMessageToQueue<FSoundLPFFreqMessage>(Context, Messages.SoundLPFFreqMessages);
						AddMessageToQueue<FSoundHPFFreqMessage>(Context, Messages.SoundHPFFreqMessages);
						break;
					}

					case RouteId_SoundSourcePriority:
					{
						AddMessageToQueue<FSoundPriorityMessage>(Context, Messages.SoundPriorityMessages);
						break;
					}

					case RouteId_SoundSourceDistance:
					{
						AddMessageToQueue<FSoundDistanceMessage>(Context, Messages.SoundDistanceMessages);
						break;
					}

					case RouteId_SoundSourceAttenuation:
					{
						AddMessageToQueue<FSoundDistanceAttenuationMessage>(Context, Messages.SoundAttenuationMessages);
						break;
					}

					case RouteId_SoundSourceRelativeRenderCost:
					{
						AddMessageToQueue<FSoundRelativeRenderCostMessage>(Context, Messages.SoundRelativeRenderCostMessages);
						break;
					}

					case RouteId_AudioBusEnvelope:
					{
						AddMessageToQueue<FAudioBusEnvelopeValuesMessage>(Context, Messages.AudioBusEnvelopeMessages);
						break;
					}

					case RouteId_SubmixEnvelope:
					{
						AddMessageToQueue<FSubmixEnvelopeValuesMessage>(Context, Messages.SubmixEnvelopeMessages);
						break;
					}

					// The second set of messages here are unique to signal flow.
					// Make sure these are added to the cache.
					case RouteID_SoundWaveStop:
					{
						CacheMessage<FSoundWaveStopMessage>(Context, Messages.SoundWaveStopMessages);
						break;
					}

					case RouteId_BusIsAlivePing:
					{
						CacheMessage<FAudioBusStartMessage>(Context, Messages.BusIsAlivePingMessages);
						break;
					}

					case RouteId_SoundToBusSend:
					{
						CacheMessage<FSoundToBusSendMessage>(Context, Messages.SoundToBusSendMessages);
						break;
					}

					case RouteId_BusPatchWriterConnected:
					{
						CacheMessage<FBusPatchWriterConnectedMessage>(Context, Messages.BusPatchWriterConnectedMessages);
						break;
					}

					case RouteId_BusPatchReaderConnected:
					{
						CacheMessage<FBusPatchReaderConnectedMessage>(Context, Messages.BusPatchReaderConnectedMessages);
						break;
					}

					case RouteId_SoundToSubmixSend:
					{
						CacheMessage<FSoundToSubmixSendMessage>(Context, Messages.SoundToSubmixSendMessages);
						break;
					}

					case RouteId_SubmixToSubmixSend:
					{
						CacheMessage<FSubmixToSubmixSendMessage>(Context, Messages.SubmixToSubmixSendMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteID_SoundWaveStart,
				RouteID_SoundWaveIsAlivePing,
				RouteID_SoundWaveStop,
				RouteID_ActiveSoundStop,
				RouteId_BusStart,
				RouteId_BusIsAlivePing,
				RouteId_BusStop,
				RouteId_SubmixLoaded,
				RouteId_SubmixAlivePing,
				RouteId_SubmixUnloaded,

				RouteId_SoundToBusSend,
				RouteId_BusPatchWriterConnected,
				RouteId_BusPatchReaderConnected,
				RouteId_SoundToSubmixSend,
				RouteId_SubmixToSubmixSend,

				RouteId_SoundSourceEnvelope,
				RouteId_SoundSourceVolume,
				RouteId_SoundSourcePitch,
				RouteId_SoundSourceFilters,
				RouteId_SoundSourcePriority,
				RouteId_SoundSourceDistance,
				RouteId_SoundSourceAttenuation,
				RouteId_SoundSourceRelativeRenderCost,
				RouteId_AudioBusEnvelope,
				RouteId_SubmixEnvelope
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FSignalFlowTraceAnalyzer(AsShared(), InSession);
	}

	void FSignalFlowTraceProvider::Reset()
	{
		TDeviceDataMapTraceProvider<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>::Reset();

		OnResetGraph.Broadcast();
	}

	void FSignalFlowTraceProvider::BindDelegates()
	{
#if WITH_EDITOR
		FTraceModule& TraceModule = static_cast<FTraceModule&>(IAudioInsightsModule::GetChecked().GetTraceModule());

		const TSharedPtr<FAudioBusTraceProvider> AudioBusProvider = TraceModule.FindAudioTraceProvider<FAudioBusTraceProvider>();
		if (AudioBusProvider.IsValid())
		{
			AudioBusProvider->OnAudioBusNameResolved.AddSP(this, &FSignalFlowTraceProvider::HandleOnAudioBusNameResolved);
		}
#endif // WITH_EDITOR
	}

	bool FSignalFlowTraceProvider::ProcessMessages()
	{
		auto GetOwnerEntry = [this](const FSoundStopMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::OwnerObject, Msg.ActiveSoundPlayOrder });
		};

		auto GetOwnerEntryFromSoundWaveStop = [this](const FSoundWaveStopMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::OwnerObject, Msg.ActiveSoundPlayOrder });
		};

		auto GetSoundwaveEntry = [this](const ::Audio::FDeviceId DeviceId, const uint32 WaveInstancePlayOrder)
		{
			return FindDeviceEntry(DeviceId, { DeviceId, ESignalFlowEntryType::SoundSource, WaveInstancePlayOrder });
		};

		auto GetSoundwaveEntryFromMessage = [this](const FSoundWaveStopMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::SoundSource, Msg.WaveInstancePlayOrder });
		};

		auto GetSoundwaveEntryFromParamMessage = [this](const FSoundParameterMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::SoundSource, Msg.WaveInstancePlayOrder });
		};

		auto GetEntryFromSendMessage = [&GetSoundwaveEntry](const FSignalFlowSendMessageBase& Msg)
		{
			return GetSoundwaveEntry(Msg.DeviceId, Msg.SenderID);
		};

		auto GetBusEntry = [this](const FAudioBusMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.AudioBusId });
		};

		auto GetBusPatchReaderEntry = [this](const FBusPatchReaderConnectedMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.SenderID });
		};

		auto GetSubmixEntry = [this](const FSubmixMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::Submix, Msg.SubmixId });
		};

		auto GetSubmixToSubmixEntry = [this](const FSubmixToSubmixSendMessage& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::Submix, Msg.SenderID });
		};

		TSet<FSignalFlowEntryKey> RemovedKeys;
		TSet<FSignalFlowEntryKey> NewBuses;

		ProcessMessageQueue<FAudioBusStartMessage>(TraceMessages.BusStartMessages, 
		[this](const FAudioBusStartMessage& Msg) 
		{ 
			return CreateAudioBusEntry(Msg); 
		},
		[this, &NewBuses](const FAudioBusStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewBusEntry(Msg, OutEntry, NewBuses);
		});

		ProcessMessageQueue<FAudioBusStartMessage>(TraceMessages.BusIsAlivePingMessages, 
		[this](const FAudioBusStartMessage& Msg)
		{
			return CreateAudioBusEntry(Msg);
		},
		[this, &NewBuses](const FAudioBusStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewBusEntry(Msg, OutEntry, NewBuses);
		});

		ProcessMessageQueue<FAudioBusStopMessage>(TraceMessages.BusStopMessages, GetBusEntry,
		[this, &RemovedKeys](const FAudioBusStopMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, OutEntry, RemovedKeys);
		});

		ProcessMessageQueue<FSubmixLoadedMessage>(TraceMessages.SubmixLoadedMessages,
		[this](const FSubmixLoadedMessage& Msg)
		{
			return CreateSubmixEntry(Msg);
		},
		[this](const FSubmixLoadedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewSubmixEntry(Msg, OutEntry);
		});

		ProcessMessageQueue<FSubmixAlivePingMessage>(TraceMessages.SubmixAliveMessages,
		[this](const FSubmixLoadedMessage& Msg)
		{
			return CreateSubmixEntry(Msg);
		},
		[this](const FSubmixAlivePingMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewSubmixEntry(Msg, OutEntry);
		});

		ProcessMessageQueue<FSubmixUnloadedMessage>(TraceMessages.SubmixUnloadedMessages, GetSubmixEntry,
		[this, &RemovedKeys](const FSubmixUnloadedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, OutEntry, RemovedKeys);
		});

		ProcessMessageQueue<FSoundWaveStartMessage>(TraceMessages.SoundWaveStartMessages, 
		[this](const FSoundWaveStartMessage& Msg)
		{
			return CreateSoundSourceEntry(Msg);
		},
		[this](const FSoundWaveStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewSoundwaveEntry(Msg, OutEntry);
		});

		ProcessMessageQueue<FSoundWaveIsAlivePingMessage>(TraceMessages.SoundWaveIsAlivePingMessages, 
		[this](const FSoundWaveIsAlivePingMessage& Msg)
		{
			return CreateSoundSourceEntry(Msg);
		},
		[this](const FSoundWaveIsAlivePingMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessNewSoundwaveEntry(Msg, OutEntry);
		});

		ProcessMessageQueue<FSoundToBusSendMessage>(TraceMessages.SoundToBusSendMessages, GetEntryFromSendMessage,
		[this](const FSoundToBusSendMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			for (const FSignalFlowSendMessageBase::FSendInfoPair& BusSend : Msg.SendInfos)
			{
				const FSignalFlowEntryKey OutputEntryKey{ Msg.DeviceId, ESignalFlowEntryType::AudioBus, BusSend.ReceiverID };
				ProcessOutputSendMessage(OutEntry, OutputEntryKey, BusSend.SendLevel);
			}
		});

		ProcessMessageQueue<FSoundToSubmixSendMessage>(TraceMessages.SoundToSubmixSendMessages, GetEntryFromSendMessage,
		[this](const FSoundToSubmixSendMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			for (const FSignalFlowSendMessageBase::FSendInfoPair& SubmixSend : Msg.SendInfos)
			{
				const FSignalFlowEntryKey OutputEntryKey{ Msg.DeviceId, ESignalFlowEntryType::Submix, SubmixSend.ReceiverID };
				ProcessOutputSendMessage(OutEntry, OutputEntryKey, SubmixSend.SendLevel);
			}
		});

		ProcessMessageQueue<FBusPatchWriterConnectedMessage>(TraceMessages.BusPatchWriterConnectedMessages, GetEntryFromSendMessage,
		[this](const FBusPatchWriterConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessBusPatchWriterMessage(Msg, OutEntry);
		});

		ProcessMessageQueue<FBusPatchReaderConnectedMessage>(TraceMessages.BusPatchReaderConnectedMessages, GetBusPatchReaderEntry,
		[this](const FBusPatchReaderConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessBusPatchReaderMessage(Msg, OutEntry);
		});

		ProcessMessageQueue<FSubmixToSubmixSendMessage>(TraceMessages.SubmixToSubmixSendMessages, GetSubmixToSubmixEntry,
		[this](const FSubmixToSubmixSendMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			const FSignalFlowEntryKey OutputEntryKey{ Msg.DeviceId, Msg.bIsFinalStage ? ESignalFlowEntryType::AudioDevice : ESignalFlowEntryType::Submix, Msg.ReceiverID };
			ProcessOutputSendMessage(OutEntry, OutputEntryKey, Msg.SendLevel);
		});

		ProcessMessageQueue<FSoundWaveStopMessage>(TraceMessages.SoundWaveStopMessages, GetSoundwaveEntryFromMessage,
		[this, &RemovedKeys, &GetOwnerEntryFromSoundWaveStop](const FSoundWaveStopMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				if (TSharedPtr<FSignalFlowDashboardEntry>* OwnerObjectEntry = GetOwnerEntryFromSoundWaveStop(Msg))
				{
					if (OwnerObjectEntry->IsValid())
					{
						(*OwnerObjectEntry)->Outputs.Remove((*OutEntry)->GetSignalFlowEntryKey());
					}
				}
			}

			ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, OutEntry, RemovedKeys);
		});

		const TArray<FSoundStopMessage> StopMessages = TraceMessages.SoundStopMessages.DequeueAll();
		for (const FSoundStopMessage& SoundStop : StopMessages)
		{
			// Try find owning object
			if (TSharedPtr<FSignalFlowDashboardEntry>* OwnerObjectEntry = GetOwnerEntry(SoundStop))
			{
				if (OwnerObjectEntry->IsValid())
				{
					RemovedKeys.Add((*OwnerObjectEntry)->GetSignalFlowEntryKey());
				}

				RemoveDeviceEntry(SoundStop.DeviceId, { SoundStop.DeviceId, ESignalFlowEntryType::OwnerObject, SoundStop.ActiveSoundPlayOrder });
			}

			if (TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>* DeviceData = DeviceDataMap.Find(SoundStop.DeviceId))
			{
				// Remove the audio device entry if it's the only remaining entry in the map
				if (DeviceData->Num() == 1 && DeviceData->Contains({ SoundStop.DeviceId, ESignalFlowEntryType::AudioDevice, SoundStop.DeviceId }))
				{
					DeviceDataMap.Remove(SoundStop.DeviceId);
				}
			}
		}

		if (!RemovedKeys.IsEmpty() || !NewBuses.IsEmpty())
		{
			TidyConnections(RemovedKeys, NewBuses);
			bForceGraphRefresh = true;
		}

		ProcessMessageQueue<FSoundEnvelopeMessage>(TraceMessages.SoundEnvelopeMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundEnvelopeMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Amplitude = Msg.Envelope;
				(*OutEntry)->WriteNewAmplitudeValue(Msg.Envelope);
			}
		});

		ProcessMessageQueue<FSoundVolumeMessage>(TraceMessages.SoundVolumeMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundVolumeMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Volume = Msg.Volume;
			}
		});

		ProcessMessageQueue<FSoundPitchMessage>(TraceMessages.SoundPitchMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundPitchMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Pitch = Msg.Pitch;
			}
		});

		ProcessMessageQueue<FSoundLPFFreqMessage>(TraceMessages.SoundLPFFreqMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundLPFFreqMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->LPFFreq = Msg.LPFFrequency;
			}
		});

		ProcessMessageQueue<FSoundHPFFreqMessage>(TraceMessages.SoundHPFFreqMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundHPFFreqMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->HPFFreq = Msg.HPFFrequency;
			}
		});

		ProcessMessageQueue<FSoundPriorityMessage>(TraceMessages.SoundPriorityMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundPriorityMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Priority = Msg.Priority;
			}
		});

		ProcessMessageQueue<FSoundDistanceMessage>(TraceMessages.SoundDistanceMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundDistanceMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Distance = Msg.Distance;
			}
		});

		ProcessMessageQueue<FSoundDistanceAttenuationMessage>(TraceMessages.SoundAttenuationMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundDistanceAttenuationMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->Attenuation = Msg.DistanceAttenuation;
			}
		});

		ProcessMessageQueue<FSoundRelativeRenderCostMessage>(TraceMessages.SoundRelativeRenderCostMessages, GetSoundwaveEntryFromParamMessage,
		[this](const FSoundRelativeRenderCostMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			if (OutEntry && OutEntry->IsValid())
			{
				(*OutEntry)->RelativeRenderCost = Msg.RelativeRenderCost;
			}
		});

		ProcessMessageQueue<FAudioBusEnvelopeValuesMessage>(TraceMessages.AudioBusEnvelopeMessages, GetBusEntry,
		[this](const FAudioBusEnvelopeValuesMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessMultiChannelEnvelopeMessage(OutEntry, Msg.EnvelopeValues);
		});

		ProcessMessageQueue<FSubmixEnvelopeValuesMessage>(TraceMessages.SubmixEnvelopeMessages, GetSubmixEntry,
		[this](const FSubmixEnvelopeValuesMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
		{
			ProcessMultiChannelEnvelopeMessage(OutEntry, Msg.EnvelopeValues);
		});

		if (bForceGraphRefresh)
		{
			OnRequestGraphRefresh.Broadcast();
			bForceGraphRefresh = false;
		}

		return true;
	}

	void FSignalFlowTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		TSet<FSignalFlowEntryKey> ActiveEntries;
		TSet<FSignalFlowEntryKey> RemovedKeys;
		TSet<FSignalFlowEntryKey> NewBuses;

		const FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		/*
		 * Audio Buses
		*/
		CacheManager.IterateOverRange<FAudioBusStartMessage>(AudioBusMessageNames::Start, RangeCheckStart, TimeMarker, [this, &ActiveEntries, &NewBuses](const FAudioBusStartMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = CreateAudioBusEntry(Msg, &ActiveEntries);
			if (Entry && Entry->IsValid())
			{
				ProcessNewBusEntry(Msg, Entry, NewBuses);
			}
		});

		CacheManager.IterateOverRange<FAudioBusStopMessage>(AudioBusMessageNames::Stop, RangeCheckStart, TimeMarker, [this, &ActiveEntries, &RemovedKeys](const FAudioBusStopMessage& Msg)
		{
			const FSignalFlowEntryKey EntryKey = { Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.AudioBusId };
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Msg.DeviceId, EntryKey);
			if (Entry && Entry->IsValid() && (*Entry)->Timestamp < Msg.Timestamp)
			{
				ActiveEntries.Remove(EntryKey);
				ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, Entry, RemovedKeys);
			}
		});

		/*
		 * Submixes
		*/
		CacheManager.IterateOverRange<FSubmixLoadedMessage>(SubmixMessageNames::Loaded, RangeCheckStart, TimeMarker, [this, &ActiveEntries](const FSubmixLoadedMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = CreateSubmixEntry(Msg, &ActiveEntries);
			if (Entry && Entry->IsValid())
			{
				ProcessNewSubmixEntry(Msg, Entry);
			}
		});

		CacheManager.IterateOverRange<FSubmixAlivePingMessage>(SubmixMessageNames::IsAlivePing, RangeCheckStart, TimeMarker, [this, &ActiveEntries](const FSubmixAlivePingMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = CreateSubmixEntry(Msg, &ActiveEntries);
			if (Entry && Entry->IsValid())
			{
				ProcessNewSubmixEntry(Msg, Entry);
			}
		});

		CacheManager.IterateOverRange<FSubmixUnloadedMessage>(SubmixMessageNames::Unloaded, RangeCheckStart, TimeMarker, [this, &ActiveEntries, &RemovedKeys](const FSubmixUnloadedMessage& Msg)
		{
			const FSignalFlowEntryKey EntryKey = { Msg.DeviceId, ESignalFlowEntryType::Submix, Msg.SubmixId };
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Msg.DeviceId, EntryKey);
			if (Entry && Entry->IsValid() && (*Entry)->Timestamp < Msg.Timestamp)
			{
				ActiveEntries.Remove(EntryKey);
				ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, Entry, RemovedKeys);
			}
		});

		/*
		 * Sound Sources
		*/
		CacheManager.IterateOverRange<FSoundWaveStartMessage>(SoundMessageNames::SoundWaveStart, RangeCheckStart, TimeMarker, [this, &ActiveEntries](const FSoundWaveStartMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = CreateSoundSourceEntry(Msg, &ActiveEntries);
			if (Entry && Entry->IsValid())
			{
				ProcessNewSoundwaveEntry(Msg, Entry, &ActiveEntries);
			}
		});

		CacheManager.IterateOverRange<FSoundWaveIsAlivePingMessage>(SoundMessageNames::SoundWaveIsAlivePing, RangeCheckStart, TimeMarker, [this, &ActiveEntries](const FSoundWaveIsAlivePingMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = CreateSoundSourceEntry(Msg, &ActiveEntries);
			if (Entry && Entry->IsValid())
			{
				ProcessNewSoundwaveEntry(Msg, Entry, &ActiveEntries);
			}
		});

		/*
		 * Sound source stop
		*/
		CacheManager.IterateOverRange<FSoundWaveStopMessage>(SoundMessageNames::SoundWaveStop, RangeCheckStart, TimeMarker, [this, &ActiveEntries, &RemovedKeys](const FSoundWaveStopMessage& Msg)
		{
			const FSignalFlowEntryKey EntryKey = { Msg.DeviceId, ESignalFlowEntryType::SoundSource, Msg.WaveInstancePlayOrder };
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Msg.DeviceId, EntryKey);
			if (Entry && Entry->IsValid() && (*Entry)->Timestamp < Msg.Timestamp)
			{
				TSharedPtr<FSignalFlowDashboardEntry>* OwnerObjectEntry = FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::OwnerObject, Msg.ActiveSoundPlayOrder });
				if (OwnerObjectEntry && OwnerObjectEntry->IsValid())
				{
					(*OwnerObjectEntry)->Outputs.Remove(EntryKey);
				}
		
				ActiveEntries.Remove(EntryKey);
				ProcessStopMessage(Msg.DeviceId, Msg.Timestamp, Entry, RemovedKeys);
			}
		});
		
		CacheManager.IterateOverRange<FSoundStopMessage>(SoundMessageNames::SoundStop, RangeCheckStart, TimeMarker, [this, &ActiveEntries, &RemovedKeys](const FSoundStopMessage& Msg)
		{
			const FSignalFlowEntryKey OwnerObjectEntryKey = { Msg.DeviceId, ESignalFlowEntryType::OwnerObject, Msg.ActiveSoundPlayOrder };
			TSharedPtr<FSignalFlowDashboardEntry>* OwnerObjectEntry = FindDeviceEntry(Msg.DeviceId, OwnerObjectEntryKey);
			if (OwnerObjectEntry && OwnerObjectEntry->IsValid() && (*OwnerObjectEntry)->Timestamp < Msg.Timestamp)
			{
				RemovedKeys.Add(OwnerObjectEntryKey);
				ActiveEntries.Remove(OwnerObjectEntryKey);

				RemoveDeviceEntry(Msg.DeviceId, OwnerObjectEntryKey);
			}
		});

		/*
		 * Bus patch connections
		*/
		CacheManager.IterateOverRange<FBusPatchWriterConnectedMessage>(SignalFlowMessageNames::BusPatchWriterConnected, RangeCheckStart, TimeMarker,
		[this](const FBusPatchWriterConnectedMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::SoundSource, Msg.SenderID });
			ProcessBusPatchWriterMessage(Msg, Entry);
		});

		CacheManager.IterateOverRange<FBusPatchReaderConnectedMessage>(SignalFlowMessageNames::BusPatchReaderConnected, RangeCheckStart, TimeMarker,
		[this](const FBusPatchReaderConnectedMessage& Msg)
		{
			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Msg.DeviceId, { Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.SenderID });
			ProcessBusPatchReaderMessage(Msg, Entry);
		});

		/*
		 * Send levels/remove unreferenced entries
		*/
		for (auto DeviceItr = DeviceDataMap.CreateIterator(); DeviceItr; ++DeviceItr)
		{
			for (auto DataItr = DeviceItr.Value().CreateIterator(); DataItr; ++DataItr)
			{
				if (ActiveEntries.Contains(DataItr.Key()))
				{
					CollectSendInfoForTimestamp(DataItr.Key(), &DataItr.Value(), CacheManager, TimeMarker, RangeCheckStart);
				}
				else
				{
					RemovedKeys.Add(DataItr.Key());
					DataItr.RemoveCurrent();
					bForceGraphRefresh = true;
				}
			}

			if (DeviceItr.Value().Num() == 1 && DeviceItr.Value().Contains({ DeviceItr.Key(), ESignalFlowEntryType::AudioDevice, DeviceItr.Key() }))
			{
				RemovedKeys.Add({ DeviceItr.Key(), ESignalFlowEntryType::AudioDevice, DeviceItr.Key() });
				DeviceItr.RemoveCurrent();
				bForceGraphRefresh = true;
			}
		}

		if (!RemovedKeys.IsEmpty() || !NewBuses.IsEmpty())
		{
			TidyConnections(RemovedKeys, NewBuses);
			bForceGraphRefresh = true;
		}

		CollectParamsForTimestamp(CacheManager, ActiveEntries, TimeMarker);
				
		if (bForceGraphRefresh)
		{
			OnRequestGraphRefresh.Broadcast();
			bForceGraphRefresh = false;
		}

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);
	}

	void FSignalFlowTraceProvider::CollectSendInfoForTimestamp(const FSignalFlowEntryKey& EntryKey, TSharedPtr<FSignalFlowDashboardEntry>* Entry, const FAudioInsightsCacheManager& CacheManager, const double TimeMarker, const double RangeCheckStart)
	{
		if (Entry == nullptr || !Entry->IsValid())
		{
			return;
		}

		switch (EntryKey.EntryType)
		{
			case ESignalFlowEntryType::SoundSource:
			{
				const FSoundToBusSendMessage* SoundToBusMsg = CacheManager.FindClosestMessage<FSoundToBusSendMessage>(SignalFlowMessageNames::SoundWaveToBusSend, TimeMarker, GetTypeHash(EntryKey), RangeCheckStart);
				if (SoundToBusMsg)
				{
					for (const FSignalFlowSendMessageBase::FSendInfoPair& SendInfo : SoundToBusMsg->SendInfos)
					{
						const FSignalFlowEntryKey OutputEntryKey{ SoundToBusMsg->DeviceId, ESignalFlowEntryType::AudioBus, SendInfo.ReceiverID };
						ProcessOutputSendMessage(Entry, OutputEntryKey, SendInfo.SendLevel);
					}
				}

				const FSoundToSubmixSendMessage* SoundToSubmixMsg = CacheManager.FindClosestMessage<FSoundToSubmixSendMessage>(SignalFlowMessageNames::SoundWaveToSubmixSend, TimeMarker, GetTypeHash(EntryKey), RangeCheckStart);
				if (SoundToSubmixMsg)
				{
					for (const FSignalFlowSendMessageBase::FSendInfoPair& SendInfo : SoundToSubmixMsg->SendInfos)
					{
						const FSignalFlowEntryKey OutputEntryKey{ SoundToSubmixMsg->DeviceId, ESignalFlowEntryType::Submix, SendInfo.ReceiverID };
						ProcessOutputSendMessage(Entry, OutputEntryKey, SendInfo.SendLevel);
					}
				}
				break;
			}
				

			case ESignalFlowEntryType::AudioBus:
			{
				// TODO
				break;
			}

			case ESignalFlowEntryType::Submix:
			{
				const FSubmixToSubmixSendMessage* SubmixToSubmixMsg = CacheManager.FindClosestMessage<FSubmixToSubmixSendMessage>(SignalFlowMessageNames::SubmixToSubmixSend, TimeMarker, GetTypeHash(EntryKey), RangeCheckStart);
				if (SubmixToSubmixMsg)
				{
					const FSignalFlowEntryKey OutputEntryKey{ SubmixToSubmixMsg->DeviceId, ESignalFlowEntryType::Submix, SubmixToSubmixMsg->ReceiverID };
					ProcessOutputSendMessage(Entry, OutputEntryKey, SubmixToSubmixMsg->SendLevel);
				}
				break;
			}
		}
	}

	void FSignalFlowTraceProvider::CollectParamsForTimestamp(const FAudioInsightsCacheManager& CacheManager, const TSet<FSignalFlowEntryKey>& ActiveEntries, const double TimeMarker)
	{
		TArray<const FSignalFlowEntryKey> ActiveEntriesArray;
		ActiveEntriesArray.Reserve(ActiveEntries.Num());

		for (auto Itr = ActiveEntries.CreateConstIterator(); Itr; ++Itr)
		{
			ActiveEntriesArray.Add(*Itr);
		}

		// Start by gathering all parameters that are represented as single values
		// Only amplitude envelopes require a range, and only sound sources have more parameters than just amplitude
		ParallelFor(ActiveEntriesArray.Num(), [&CacheManager, TimeMarker, &ActiveEntriesArray, this](const int32 Index)
		{
			const FSignalFlowEntryKey& EntryKey = ActiveEntriesArray[Index];

			const bool bIsEntryWithParam = EntryKey.EntryType == ESignalFlowEntryType::SoundSource
										|| (!bAnimateWires && (EntryKey.EntryType == ESignalFlowEntryType::AudioBus 
															|| EntryKey.EntryType == ESignalFlowEntryType::Submix));

			if (!bIsEntryWithParam)
			{
				return;
			}

			TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(EntryKey.DeviceID, EntryKey);
			if (Entry == nullptr || !Entry->IsValid())
			{
				return;
			}

			auto MatchingParamPredicate = [&EntryKey](const FSoundParameterMessage& Msg)
			{
				return Msg.WaveInstancePlayOrder == EntryKey.EntryID;
			};

			if (EntryKey.EntryType == ESignalFlowEntryType::SoundSource)
			{
				const FSoundVolumeMessage* VolumeMessage = CacheManager.FindClosestMessageByPredicate<FSoundVolumeMessage>(SoundMessageNames::VolumeParam, TimeMarker, MatchingParamPredicate);
				if (VolumeMessage)
				{
					(*Entry)->Volume = VolumeMessage->Volume;
				}

				const FSoundPitchMessage* PitchMessage = CacheManager.FindClosestMessageByPredicate<FSoundPitchMessage>(SoundMessageNames::PitchParam, TimeMarker, MatchingParamPredicate);
				if (PitchMessage)
				{
					(*Entry)->Pitch = PitchMessage->Pitch;
				}

				const FSoundLPFFreqMessage* LPFMessage = CacheManager.FindClosestMessageByPredicate<FSoundLPFFreqMessage>(SoundMessageNames::LPFFreqParam, TimeMarker, MatchingParamPredicate);
				if (LPFMessage)
				{
					(*Entry)->LPFFreq = LPFMessage->LPFFrequency;
				}

				const FSoundHPFFreqMessage* HPFMessage = CacheManager.FindClosestMessageByPredicate<FSoundHPFFreqMessage>(SoundMessageNames::HPFFreqParam, TimeMarker, MatchingParamPredicate);
				if (HPFMessage)
				{
					(*Entry)->HPFFreq = HPFMessage->HPFFrequency;
				}

				const FSoundPriorityMessage* PriorityMessage = CacheManager.FindClosestMessageByPredicate<FSoundPriorityMessage>(SoundMessageNames::PriorityParam, TimeMarker, MatchingParamPredicate);
				if (PriorityMessage)
				{
					(*Entry)->Priority = PriorityMessage->Priority;
				}

				const FSoundDistanceMessage* DistanceMessage = CacheManager.FindClosestMessageByPredicate<FSoundDistanceMessage>(SoundMessageNames::DistanceParam, TimeMarker, MatchingParamPredicate);
				if (DistanceMessage)
				{
					(*Entry)->Distance = DistanceMessage->Distance;
				}

				const FSoundDistanceAttenuationMessage* AttenuationMessage = CacheManager.FindClosestMessageByPredicate<FSoundDistanceAttenuationMessage>(SoundMessageNames::DistanceAttenuationParam, TimeMarker, MatchingParamPredicate);
				if (AttenuationMessage)
				{
					(*Entry)->Attenuation = AttenuationMessage->DistanceAttenuation;
				}

				const FSoundRelativeRenderCostMessage* RelativeRenderCostMessage = CacheManager.FindClosestMessageByPredicate<FSoundRelativeRenderCostMessage>(SoundMessageNames::RelativeRenderCostParam, TimeMarker, MatchingParamPredicate);
				if (RelativeRenderCostMessage)
				{
					(*Entry)->RelativeRenderCost = RelativeRenderCostMessage->RelativeRenderCost;
				}
			}

			// If we are not animating wires, grab the latest envelope value here
			if (!bAnimateWires)
			{
				if (EntryKey.EntryType == ESignalFlowEntryType::SoundSource)
				{
					const FSoundEnvelopeMessage* EnvelopeMessage = CacheManager.FindClosestMessageByPredicate<FSoundEnvelopeMessage>(SoundMessageNames::EnvelopeParam, TimeMarker, MatchingParamPredicate);
					if (EnvelopeMessage)
					{
						(*Entry)->Amplitude = EnvelopeMessage->Envelope;
					}
				}
				else if (EntryKey.EntryType == ESignalFlowEntryType::AudioBus)
				{
					const FAudioBusEnvelopeValuesMessage* AudioBusEnvelopeMessage = CacheManager.FindClosestMessage<FAudioBusEnvelopeValuesMessage>(AudioBusMessageNames::EnvelopeValues, TimeMarker, EntryKey.EntryID);
					if (AudioBusEnvelopeMessage)
					{
						ProcessMultiChannelEnvelopeMessage(Entry, AudioBusEnvelopeMessage->EnvelopeValues);
					}
				}
				else if (EntryKey.EntryType == ESignalFlowEntryType::Submix)
				{
					const FSubmixEnvelopeValuesMessage* SubmixEnvelopeMessage = CacheManager.FindClosestMessage<FSubmixEnvelopeValuesMessage>(SubmixMessageNames::EnvelopeValues, TimeMarker, EntryKey.EntryID);
					if (SubmixEnvelopeMessage)
					{
						ProcessMultiChannelEnvelopeMessage(Entry, SubmixEnvelopeMessage->EnvelopeValues);
					}
				}
			}
			
		});

		if (bAnimateWires)
		{
			// Next, gather an amplitude range large enough to cover the wire animation
			constexpr float AmplitudeLookbackTime = 1.0f;
			const float StartTime = TimeMarker - AmplitudeLookbackTime;

			constexpr int32 NumMessagesToProcess = 3; // we have 3 different types of amplitude message we want to retrieve a range for from the cache

			ParallelFor(NumMessagesToProcess, [&CacheManager, TimeMarker, StartTime, this](const int32 Index)
			{
				switch (Index)
				{
					case 0: // FSoundEnvelopeMessage
					{
						CacheManager.IterateOverRange<FSoundEnvelopeMessage>(SoundMessageNames::EnvelopeParam, StartTime, TimeMarker, [this](const FSoundEnvelopeMessage& Message)
						{
							const FSignalFlowEntryKey EntryKey(Message.DeviceId, ESignalFlowEntryType::SoundSource, Message.WaveInstancePlayOrder);
							TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Message.DeviceId, EntryKey);

							if (Entry && Entry->IsValid())
							{
								(*Entry)->Amplitude = Message.Envelope;
								(*Entry)->WriteNewAmplitudeValue(Message.Envelope);
							}
						});

						break;
					}

					case 1: // FAudioBusEnvelopeValuesMessage
					{
						CacheManager.IterateOverRange<FAudioBusEnvelopeValuesMessage>(AudioBusMessageNames::EnvelopeValues, StartTime, TimeMarker, [this](const FAudioBusEnvelopeValuesMessage& Message)
						{
							const FSignalFlowEntryKey EntryKey(Message.DeviceId, ESignalFlowEntryType::AudioBus, Message.AudioBusId);
							TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Message.DeviceId, EntryKey);

							if (Entry && Entry->IsValid())
							{
								ProcessMultiChannelEnvelopeMessage(Entry, Message.EnvelopeValues);
							}
						});
						break;
					}

					case 2: // FSubmixEnvelopeValuesMessage
					{
						CacheManager.IterateOverRange<FSubmixEnvelopeValuesMessage>(SubmixMessageNames::EnvelopeValues, StartTime, TimeMarker, [this](const FSubmixEnvelopeValuesMessage& Message)
						{
							const FSignalFlowEntryKey EntryKey(Message.DeviceId, ESignalFlowEntryType::Submix, Message.SubmixId);
							TSharedPtr<FSignalFlowDashboardEntry>* Entry = FindDeviceEntry(Message.DeviceId, EntryKey);

							if (Entry && Entry->IsValid())
							{
								ProcessMultiChannelEnvelopeMessage(Entry, Message.EnvelopeValues);
							}
						});
						break;
					}
				}
			});
		}
	}

	void FSignalFlowTraceProvider::OnTimeControlMethodReset()
	{
		Reset();
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateEntry(const IAudioCachedMessage& Msg, const ::Audio::FDeviceId DeviceID, const ESignalFlowEntryType EntryType, const uint32 NodeID, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		TSharedPtr<FSignalFlowDashboardEntry>* ToReturn = nullptr;

		UpdateDeviceEntry(DeviceID, { DeviceID, EntryType, NodeID }, [&ToReturn, &Msg, EntryType, DeviceID, NodeID](TSharedPtr<FSignalFlowDashboardEntry>& Entry)
		{
			if (!Entry.IsValid())
			{
				Entry = MakeShared<FSignalFlowDashboardEntry>();
				Entry->SignalFlowNodeID = NodeID;
				Entry->DeviceId = DeviceID;
				Entry->EntryType = EntryType;
			}

			Entry->Timestamp = Msg.Timestamp;

			ToReturn = &Entry;
		});

		if (OutActiveEntriesTracker && ToReturn && ToReturn->IsValid())
		{
			OutActiveEntriesTracker->Add((*ToReturn)->GetSignalFlowEntryKey());
		}

		return ToReturn;
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateAudioDeviceEntry(const IAudioCachedMessage& Msg, const::Audio::FDeviceId DeviceID, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		TSharedPtr<FSignalFlowDashboardEntry>* AudioDeviceEntry = CreateEntry(Msg, DeviceID, ESignalFlowEntryType::AudioDevice, DeviceID, OutActiveEntriesTracker);
		if (AudioDeviceEntry && AudioDeviceEntry->IsValid())
		{
			(*AudioDeviceEntry)->Name = FText::Format(LOCTEXT("SignalFlow_AudioDevice", "Audio Device {0}"), static_cast<uint32>(DeviceID)).ToString();
			(*AudioDeviceEntry)->IconName = FSignalFlowTraceProviderPrivate::AudioDeviceIconName;
		}

		return AudioDeviceEntry;
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateOwnerEntry(const FSoundWaveStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		CreateAudioDeviceEntry(Msg, Msg.DeviceId, OutActiveEntriesTracker);

		return CreateEntry(Msg, Msg.DeviceId, ESignalFlowEntryType::OwnerObject, Msg.ActiveSoundPlayOrder, OutActiveEntriesTracker);
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateSoundSourceEntry(const FSoundWaveStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		CreateAudioDeviceEntry(Msg, Msg.DeviceId, OutActiveEntriesTracker);

		return CreateEntry(Msg, Msg.DeviceId, ESignalFlowEntryType::SoundSource, Msg.WaveInstancePlayOrder, OutActiveEntriesTracker);
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateAudioBusEntry(const FAudioBusStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		CreateAudioDeviceEntry(Msg, Msg.DeviceId, OutActiveEntriesTracker);

		return CreateEntry(Msg, Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.AudioBusId, OutActiveEntriesTracker);
	}

	TSharedPtr<FSignalFlowDashboardEntry>* FSignalFlowTraceProvider::CreateSubmixEntry(const FSubmixMessageBase& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		CreateAudioDeviceEntry(Msg, Msg.DeviceId, OutActiveEntriesTracker);

		return CreateEntry(Msg, Msg.DeviceId, ESignalFlowEntryType::Submix, Msg.SubmixId, OutActiveEntriesTracker);
	}

	void FSignalFlowTraceProvider::LinkInput(const FSignalFlowDashboardEntry& Entry, const FSignalFlowEntryKey& OutputKey, const ::Audio::FDeviceId DeviceID)
	{
		if (const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = FindDeviceEntry(DeviceID, OutputKey))
		{
			if (OutputEntry->IsValid())
			{
				bool bIsAlreadySet = false;
				(*OutputEntry)->Inputs.FindOrAdd(Entry.GetSignalFlowEntryKey(), &bIsAlreadySet);

				if (!bIsAlreadySet)
				{
					bForceGraphRefresh = true;
				}
			}
		}
	}

	void FSignalFlowTraceProvider::ProcessNewBusEntry(const FAudioBusStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>& OutNewBuses)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			FSignalFlowDashboardEntry& EntryRef = *OutEntry->Get();

#if WITH_EDITOR
			// Prefer the asset-resolved name held by the audio-bus provider over the trace message's name.
			const FTraceModule& TraceModule = static_cast<FTraceModule&>(IAudioInsightsModule::GetChecked().GetTraceModule());
			const TSharedPtr<FAudioBusTraceProvider> AudioBusProvider = TraceModule.FindAudioTraceProvider<FAudioBusTraceProvider>();

			EntryRef.Name = AudioBusProvider.IsValid() ? AudioBusProvider->GetResolvedBusName(Msg.DeviceId, Msg.AudioBusId, Msg.Name)
													   : Msg.Name;
#else
			EntryRef.Name = Msg.Name;
#endif // WITH_EDITOR

			EntryRef.IconName = FSignalFlowTraceProviderPrivate::AudioBusIconName;
			EntryRef.Timestamp = Msg.Timestamp;

			OutNewBuses.Add(EntryRef.GetSignalFlowEntryKey());
		}
	}

#if WITH_EDITOR
	void FSignalFlowTraceProvider::HandleOnAudioBusNameResolved(const uint32 InAudioBusId)
	{
		const FTraceModule& TraceModule = static_cast<FTraceModule&>(IAudioInsightsModule::GetChecked().GetTraceModule());
		const TSharedPtr<FAudioBusTraceProvider> AudioBusProvider = TraceModule.FindAudioTraceProvider<FAudioBusTraceProvider>();
		if (!AudioBusProvider.IsValid())
		{
			return;
		}

		bool bAnyEntryRefreshed = false;

		for (TPair<::Audio::FDeviceId, FDeviceData>& DevicePair : DeviceDataMap)
		{
			const ::Audio::FDeviceId DeviceId = DevicePair.Key;
			const FSignalFlowEntryKey BusKey(DeviceId, ESignalFlowEntryType::AudioBus, InAudioBusId);

			TSharedPtr<FSignalFlowDashboardEntry>* const EntryPtr = DevicePair.Value.Find(BusKey);
			if (EntryPtr == nullptr || !EntryPtr->IsValid())
			{
				continue;
			}

			FSignalFlowDashboardEntry& EntryRef = **EntryPtr;
			const FString ResolvedName = AudioBusProvider->GetResolvedBusName(DeviceId, InAudioBusId, EntryRef.Name);

			if (ResolvedName != EntryRef.Name)
			{
				EntryRef.Name = ResolvedName;
				bAnyEntryRefreshed = true;
			}
		}

		if (bAnyEntryRefreshed)
		{
			OnRequestGraphRefresh.Broadcast();
		}
	}
#endif // WITH_EDITOR

	void FSignalFlowTraceProvider::ProcessNewSubmixEntry(const FSubmixLoadedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			FSignalFlowDashboardEntry& EntryRef = *OutEntry->Get();
			EntryRef.Name = Msg.Name;
			EntryRef.IconName = FSignalFlowTraceProviderPrivate::SubmixIconName;
			EntryRef.Timestamp = Msg.Timestamp;

			if (Msg.bIsMainSubmix)
			{
				const FSignalFlowEntryKey OutputAudioDeviceKey = { Msg.DeviceId, ESignalFlowEntryType::AudioDevice, Msg.DeviceId };

				EntryRef.Outputs.FindOrAdd(OutputAudioDeviceKey);
				LinkInput(EntryRef, OutputAudioDeviceKey, Msg.DeviceId);
			}
		}
	}

	void FSignalFlowTraceProvider::ProcessNewSoundwaveEntry(const FSoundWaveStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker /*= nullptr*/)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			FSignalFlowDashboardEntry& EntryRef = *OutEntry->Get();
			EntryRef.Name = Msg.Name;
			EntryRef.IconName = FSignalFlowTraceProviderPrivate::GetSourceIconName(Msg.EntryType);

			if (!Msg.AudioComponentName.IsEmpty())
			{
				EntryRef.AudioComponentName = Msg.AudioComponentName;
			}

			if (Msg.SourceBusID != INDEX_NONE)
			{
				EntryRef.LinkedSourceBus = FSignalFlowEntryKey(Msg.DeviceId, ESignalFlowEntryType::AudioBus, Msg.SourceBusID);

				const TSharedPtr<FSignalFlowDashboardEntry>* FoundBusEntry = FindDeviceEntry(Msg.DeviceId, EntryRef.LinkedSourceBus.GetValue());
				if (FoundBusEntry && FoundBusEntry->IsValid())
				{
					(*FoundBusEntry)->LinkedSoundSources.Add(EntryRef.GetSignalFlowEntryKey());
				}
			}

			TSharedPtr<FSignalFlowDashboardEntry>* OwnerObjectEntry = CreateOwnerEntry(Msg, OutActiveEntriesTracker);
			if (OwnerObjectEntry && OwnerObjectEntry->IsValid())
			{
				(*OwnerObjectEntry)->Name = Msg.ActorLabel;
				(*OwnerObjectEntry)->EntryType = ESignalFlowEntryType::OwnerObject;
				(*OwnerObjectEntry)->Name = Msg.ActorLabel;
				(*OwnerObjectEntry)->IconName = FName(Msg.ActorIconName);

				LinkInput((**OwnerObjectEntry), EntryRef.GetSignalFlowEntryKey(), Msg.DeviceId);
				(*OwnerObjectEntry)->Outputs.FindOrAdd(EntryRef.GetSignalFlowEntryKey());
			}
		}
	}

	void FSignalFlowTraceProvider::ProcessStopMessage(const ::Audio::FDeviceId DeviceID, const double MsgTimestamp, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>& OutRemovedKeys)
	{
		if (OutEntry && OutEntry->IsValid() && (*OutEntry)->Timestamp < MsgTimestamp)
		{
			const FSignalFlowEntryKey EntryKey = (*OutEntry)->GetSignalFlowEntryKey();

			OutRemovedKeys.Add(EntryKey);
			RemoveDeviceEntry(DeviceID, EntryKey);
		}
	}

	void FSignalFlowTraceProvider::ProcessOutputSendMessage(TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, const FSignalFlowEntryKey& OutputEntryKey, const float SendLevel)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			if (FSignalFlowOutputData* SendData = (*OutEntry)->Outputs.Find(OutputEntryKey))
			{
				SendData->SendLevel = SendLevel;
			}
			else
			{
				(*OutEntry)->Outputs.Add(OutputEntryKey, { SendLevel });
				bForceGraphRefresh = true;
			}

			LinkInput(**OutEntry, OutputEntryKey, (*OutEntry)->DeviceId);
		}
	}

	// Writer: a source writes to a bus. OutEntry is the source.
	// Populates the source's LinkedBusPatchOutputs (bus keys) and the bus's LinkedBusPatchInputs (source key).
	void FSignalFlowTraceProvider::ProcessBusPatchWriterMessage(const FBusPatchWriterConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			const FSignalFlowEntryKey SourceEntryKey = (*OutEntry)->GetSignalFlowEntryKey();

			for (const FSignalFlowSendMessageBase::FSendInfoPair& BusPatchInput : Msg.SendInfos)
			{
				const FSignalFlowEntryKey BusEntryKey{ Msg.DeviceId, ESignalFlowEntryType::AudioBus, BusPatchInput.ReceiverID };

				bool bIsAlreadySet = false;
				(*OutEntry)->LinkedBusPatchOutputs.FindOrAdd(BusEntryKey, &bIsAlreadySet);

				if (!bIsAlreadySet)
				{
					bForceGraphRefresh = true;
				}

				TSharedPtr<FSignalFlowDashboardEntry>* BusEntry = FindDeviceEntry(Msg.DeviceId, BusEntryKey);
				if (BusEntry && BusEntry->IsValid())
				{
					(*BusEntry)->LinkedBusPatchInputs.Add(SourceEntryKey);
				}
			}
		}
	}

	// Reader: a bus outputs to a source. OutEntry is the bus.
	// Populates the bus's LinkedBusPatchOutputs (source keys) and the source's LinkedBusPatchInputs (bus key).
	void FSignalFlowTraceProvider::ProcessBusPatchReaderMessage(const FBusPatchReaderConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry)
	{
		if (OutEntry && OutEntry->IsValid())
		{
			const FSignalFlowEntryKey BusEntryKey = (*OutEntry)->GetSignalFlowEntryKey();

			for (const FSignalFlowSendMessageBase::FSendInfoPair& BusPatchOutput : Msg.SendInfos)
			{
				const FSignalFlowEntryKey SourceEntryKey{ Msg.DeviceId, ESignalFlowEntryType::SoundSource, BusPatchOutput.ReceiverID };

				bool bIsAlreadySet = false;
				(*OutEntry)->LinkedBusPatchOutputs.FindOrAdd(SourceEntryKey, &bIsAlreadySet);

				if (!bIsAlreadySet)
				{
					bForceGraphRefresh = true;
				}

				TSharedPtr<FSignalFlowDashboardEntry>* SourceEntry = FindDeviceEntry(Msg.DeviceId, SourceEntryKey);
				if (SourceEntry && SourceEntry->IsValid())
				{
					(*SourceEntry)->LinkedBusPatchInputs.Add(BusEntryKey);
				}
			}
		}
	}

	void FSignalFlowTraceProvider::ProcessMultiChannelEnvelopeMessage(TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, const TArray<float>& EnvelopeValues)
	{
		if (OutEntry == nullptr || !OutEntry->IsValid() || EnvelopeValues.IsEmpty())
		{
			return;
		}

		float MaxPeakAmp = TNumericLimits<float>::Lowest();
		for (const float AmpValue : EnvelopeValues)
		{
			MaxPeakAmp = FMath::Max(MaxPeakAmp, AmpValue);
		}

		if (MaxPeakAmp > TNumericLimits<float>::Lowest())
		{
			(*OutEntry)->Amplitude = MaxPeakAmp;
			(*OutEntry)->WriteNewAmplitudeValue(MaxPeakAmp);
		}
	}

	void FSignalFlowTraceProvider::TidyConnections(const TSet<FSignalFlowEntryKey>& RemovedKeys, const TSet<FSignalFlowEntryKey>& NewBuses)
	{
		// Iterate over the device data map - remove old inputs and ensure we have connections between source buses and soung sources
		for (auto& [DeviceID, Entries] : DeviceDataMap)
		{
			for (auto& [EntryKey, Entry] : Entries)
			{
				if (!Entry.IsValid())
				{
					continue;
				}

				// Clean up old inputs
				for (auto Itr = Entry->Inputs.CreateIterator(); Itr; ++Itr)
				{
					if (RemovedKeys.Contains(*Itr))
					{
						Itr.RemoveCurrent();
					}
				}

				// Ensure source buses are linked
				if (Entry->LinkedSourceBus.IsSet())
				{
					const FSignalFlowEntryKey SourceBusEntryKey = Entry->LinkedSourceBus.GetValue();
					if (RemovedKeys.Contains(SourceBusEntryKey))
					{
						Entry->LinkedSourceBus.Reset();
					}
					else if (NewBuses.Contains(SourceBusEntryKey))
					{
						const TSharedPtr<FSignalFlowDashboardEntry>* SourceBusEntry = FindDeviceEntry(DeviceID, SourceBusEntryKey);
						if (SourceBusEntry && SourceBusEntry->IsValid())
						{
							(*SourceBusEntry)->LinkedSoundSources.Add(EntryKey);
						}
					}
				}

				// Ensure stale linked sound sources are removed
				for (auto Itr = Entry->LinkedSoundSources.CreateIterator(); Itr; ++Itr)
				{
					if (RemovedKeys.Contains(*Itr))
					{
						Itr.RemoveCurrent();
					}
				}

				// Clean stale bus patch input connections
				for (auto Itr = Entry->LinkedBusPatchInputs.CreateIterator(); Itr; ++Itr)
				{
					if (RemovedKeys.Contains(*Itr))
					{
						Itr.RemoveCurrent();
					}
				}

				// Clean stale bus patch output connections
				for (auto Itr = Entry->LinkedBusPatchOutputs.CreateIterator(); Itr; ++Itr)
				{
					if (RemovedKeys.Contains(*Itr))
					{
						Itr.RemoveCurrent();
					}
				}
			}
		}
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE