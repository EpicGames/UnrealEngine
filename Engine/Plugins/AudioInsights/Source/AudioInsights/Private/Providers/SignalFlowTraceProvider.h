// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/SignalFlowTraceMessages.h"

namespace UE::Audio::Insights
{
	class FSignalFlowTraceProvider : public TDeviceDataMapTraceProvider<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>, public TSharedFromThis<FSignalFlowTraceProvider>
	{
	public:
		FSignalFlowTraceProvider();
		virtual ~FSignalFlowTraceProvider() = default;

		static FName GetName_Static();
		
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;
		virtual void Reset() override;

		void BindDelegates();

		void SetConnectionWiresAreAnimated(const bool bInAnimated) { bAnimateWires = bInAnimated; }

		DECLARE_MULTICAST_DELEGATE(FOnRequestGraphRefresh);
		FOnRequestGraphRefresh OnRequestGraphRefresh;

		DECLARE_MULTICAST_DELEGATE(FOnResetGraph);
		FOnResetGraph OnResetGraph;

	private:
		virtual bool ProcessMessages() override;
		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;

		void CollectSendInfoForTimestamp(const FSignalFlowEntryKey& EntryKey, TSharedPtr<FSignalFlowDashboardEntry>* Entry, const FAudioInsightsCacheManager& CacheManager, const double TimeMarker, const double RangeCheckStart);
		void CollectParamsForTimestamp(const FAudioInsightsCacheManager& CacheManager, const TSet<FSignalFlowEntryKey>& ActiveEntries, const double TimeMarker);

		virtual void OnTimeControlMethodReset() override;

		TSharedPtr<FSignalFlowDashboardEntry>* CreateEntry(const IAudioCachedMessage& Msg, const ::Audio::FDeviceId DeviceID, const ESignalFlowEntryType EntryType, const uint32 NodeID, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		TSharedPtr<FSignalFlowDashboardEntry>* CreateAudioDeviceEntry(const IAudioCachedMessage& Msg, const ::Audio::FDeviceId DeviceID, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		TSharedPtr<FSignalFlowDashboardEntry>* CreateOwnerEntry(const FSoundWaveStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		TSharedPtr<FSignalFlowDashboardEntry>* CreateSoundSourceEntry(const FSoundWaveStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		TSharedPtr<FSignalFlowDashboardEntry>* CreateAudioBusEntry(const FAudioBusStartMessage& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		TSharedPtr<FSignalFlowDashboardEntry>* CreateSubmixEntry(const FSubmixMessageBase& Msg, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);

		void ProcessNewBusEntry(const FAudioBusStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>& OutNewBuses);
		void ProcessNewSubmixEntry(const FSubmixLoadedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry);
		void ProcessNewSoundwaveEntry(const FSoundWaveStartMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>* OutActiveEntriesTracker = nullptr);
		void ProcessStopMessage(const ::Audio::FDeviceId DeviceID, const double MsgTimestamp, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, TSet<FSignalFlowEntryKey>& OutRemovedKeys);

		void ProcessOutputSendMessage(TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, const FSignalFlowEntryKey& OutputEntryKey, const float SendLevel);
		void ProcessBusPatchWriterMessage(const FBusPatchWriterConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry);
		void ProcessBusPatchReaderMessage(const FBusPatchReaderConnectedMessage& Msg, TSharedPtr<FSignalFlowDashboardEntry>* OutEntry);
		void ProcessMultiChannelEnvelopeMessage(TSharedPtr<FSignalFlowDashboardEntry>* OutEntry, const TArray<float>& EnvelopeValues);

		void LinkInput(const FSignalFlowDashboardEntry& Entry, const FSignalFlowEntryKey& OutputKey, const ::Audio::FDeviceId DeviceID);
		void TidyConnections(const TSet<FSignalFlowEntryKey>& RemovedKeys, const TSet<FSignalFlowEntryKey>& NewBuses);

#if WITH_EDITOR
		void HandleOnAudioBusNameResolved(const uint32 InAudioBusId);
#endif // WITH_EDITOR

		FSignalFlowMessages TraceMessages;

		bool bForceGraphRefresh = false;
		bool bAnimateWires = true;
	};
} // namespace UE::Audio::Insights
