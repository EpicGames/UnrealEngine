// Copyright Epic Games, Inc. All Rights Reserved.
#include "Providers/SoundTraceProvider.h"

#include "Async/ParallelFor.h"
#include "AudioInsightsModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Settings/SoundDashboardSettings.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FSoundTraceProviderPrivate
	{
		const FString MetaSoundDisplayName			{ LOCTEXT("AudioDashboard_Sounds_EntryType_MetaSound", "MetaSounds").ToString() };
		const FString SoundCueDisplayName			{ LOCTEXT("AudioDashboard_Sounds_EntryType_SoundCues", "Sound Cues").ToString() };
		const FString SoundWaveDisplayName			{ LOCTEXT("AudioDashboard_Sounds_EntryType_SoundWaves", "Sound Waves").ToString() };
		const FString ProceduralSourceDisplayName	{ LOCTEXT("AudioDashboard_Sounds_EntryType_ProceduralSources", "Procedural Sources").ToString() };
		const FString SoundCueTemplateName			{ LOCTEXT("AudioDashboard_Sounds_EntryType_SoundCueTemplate", "Sound Cue Templates").ToString() };
		const FString UncategorizedSoundName		{ LOCTEXT("AudioDashboard_Sounds_EntryType_UncategorizedSound", "Others").ToString() };

		FSoundDashboardEntry& CastEntry(IDashboardDataTreeViewEntry& InData)
		{
			return static_cast<FSoundDashboardEntry&>(InData);
		};

		FString GetEntryTypeDisplayName(const ESoundDashboardEntryType EntryType)
		{
			switch (EntryType)
			{
				case ESoundDashboardEntryType::MetaSound:
					return MetaSoundDisplayName;
				case ESoundDashboardEntryType::SoundCue:
					return SoundCueDisplayName;
				case ESoundDashboardEntryType::SoundWave:
					return SoundWaveDisplayName;
				case ESoundDashboardEntryType::ProceduralSource:
					return ProceduralSourceDisplayName;
				case ESoundDashboardEntryType::SoundCueTemplate:
					return SoundCueTemplateName;
			}

			return UncategorizedSoundName;
		}

		bool EntryTypeHasSoundWaveEntries(const ESoundDashboardEntryType EntryType)
		{
			switch (EntryType)
			{
				case ESoundDashboardEntryType::SoundCue:
				case ESoundDashboardEntryType::SoundCueTemplate:
					return true;

				case ESoundDashboardEntryType::MetaSound:
				case ESoundDashboardEntryType::ProceduralSource:
				case ESoundDashboardEntryType::SoundWave:
					return false;
				}

			return true;
		}

		FDataPoint PeekLastValue(const ::Audio::TCircularAudioBuffer<FDataPoint>& ParameterBuffer, const float DefaultReturn = 0.0f)
		{
			if (ParameterBuffer.Num() == 0)
			{
				return { 0.0, DefaultReturn };
			}

			const ::Audio::DisjointedArrayView<const FDataPoint> DataPointsDisjointedArrayView = ParameterBuffer.PeekInPlace(ParameterBuffer.Num());
			return DataPointsDisjointedArrayView.FirstBuffer.Last();
		}

		void UpdateParameterEntry(const FSoundParameterMessage& Msg, const float DataPoint, float& OutDataPoint, ::Audio::TCircularAudioBuffer<FDataPoint>& OutDataRange, TArray<int32, TInlineAllocator<64>>& OutEntriesWithPoppedDataPoints)
		{
			if (!OutEntriesWithPoppedDataPoints.Contains(Msg.WaveInstancePlayOrder))
			{
				OutDataRange.Pop(OutDataRange.Num());
				OutEntriesWithPoppedDataPoints.Add(Msg.WaveInstancePlayOrder);
			}

			OutDataPoint = DataPoint;
			OutDataRange.Push({ Msg.Timestamp, DataPoint });
		}

		//////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Finds an Active Sound entry inside a Category entry
		void FindActiveSoundEntryInCategory(const TSharedPtr<FSoundDashboardEntry>* CategoryEntry, const uint32 ActiveSoundPlayOrder, TSharedPtr<FSoundDashboardEntry>& OutActiveSoundEntry)
		{
			OutActiveSoundEntry = nullptr;
			if (CategoryEntry == nullptr || !CategoryEntry->IsValid())
			{
				return;
			}

			TSharedPtr<IDashboardDataTreeViewEntry>* ActiveSoundEntry = CategoryEntry->Get()->Children.FindByPredicate([ActiveSoundPlayOrder](TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry)
			{
				FSoundDashboardEntry& NewEntry = CastEntry(*ChildEntry.Get());
				return NewEntry.ActiveSoundPlayOrder == ActiveSoundPlayOrder;
			});

			if (ActiveSoundEntry == nullptr || !ActiveSoundEntry->IsValid())
			{
				return;
			}

			OutActiveSoundEntry = StaticCastSharedPtr<FSoundDashboardEntry>(*ActiveSoundEntry);
		}

		//////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Will return either the Active Sound entry or the child SoundWave entry depending on the sound's EntryType
		FSoundDashboardEntry* FindWaveEntryInActiveSound(const uint32 SoundWavePlayOrder, const TSharedPtr<FSoundDashboardEntry>* ActiveSoundEntry)
		{
			if (ActiveSoundEntry == nullptr || !ActiveSoundEntry->IsValid())
			{
				return nullptr;
			}

			if (!EntryTypeHasSoundWaveEntries((*ActiveSoundEntry)->EntryType))
			{
				return ActiveSoundEntry->Get();
			}

			for (TSharedPtr<IDashboardDataTreeViewEntry> SoundWaveEntry : (*ActiveSoundEntry)->Children)
			{
				if (SoundWaveEntry == nullptr || !SoundWaveEntry.IsValid())
				{
					continue;
				}

				FSoundDashboardEntry* SoundWaveEntryCast = static_cast<FSoundDashboardEntry*>(SoundWaveEntry.Get());
				if (SoundWaveEntryCast->WaveInstancePlayOrder == SoundWavePlayOrder)
				{
					return SoundWaveEntryCast;
				}
			}

			return nullptr;
		}
	}

	FSoundTraceProvider::FSoundTraceProvider()
		: TDeviceDataMapTraceProvider<ESoundDashboardEntryType, TSharedPtr<FSoundDashboardEntry>>(GetName_Static())
	{
#if WITH_EDITOR
		OnReadSettingsHandle = FSoundDashboardSettings::OnReadSettings.AddLambda([this](const FSoundDashboardSettings& InSettings) 
		{
			DashboardTimeoutTime = static_cast<double>(InSettings.StoppedSoundTimeoutTime);
		});
#endif
	}

	FSoundTraceProvider::~FSoundTraceProvider()
	{
#if WITH_EDITOR
		FSoundDashboardSettings::OnReadSettings.Remove(OnReadSettingsHandle);
#endif
	}

	FName FSoundTraceProvider::GetName_Static()
	{
		static const FLazyName SoundTraceProviderName = "SoundProvider";
		return SoundTraceProviderName;
	}

	UE::Trace::IAnalyzer* FSoundTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FSoundTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FSoundTraceAnalyzer(TSharedRef<FSoundTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

				Builder.RouteEvent(RouteId_ActiveSoundStart,		"Audio", "SoundStart");
				Builder.RouteEvent(RouteId_SoundWaveStart,			"Audio", "SoundWaveStart");
				Builder.RouteEvent(RouteId_SoundIsAlivePing,		"Audio", "SoundIsAlivePing");
				Builder.RouteEvent(RouteId_SoundWaveIsAlivePing,	"Audio", "SoundWaveIsAlivePing");
				Builder.RouteEvent(RouteId_Stop,					"Audio", "SoundStop");
				Builder.RouteEvent(RouteId_Priority,				"Audio", "SoundPriority");
				Builder.RouteEvent(RouteId_Distance,				"Audio", "SoundDistance");
				Builder.RouteEvent(RouteId_DistanceAttenuation,		"Audio", "SoundDistanceAttenuation");
				Builder.RouteEvent(RouteId_Filters,					"Audio", "SoundSourceFilters");
				Builder.RouteEvent(RouteId_Amplitude,				"Audio", "SoundSourceEnvelope");
				Builder.RouteEvent(RouteId_Volume,					"Audio", "SoundVolume");
				Builder.RouteEvent(RouteId_Pitch,					"Audio", "SoundPitch");
				Builder.RouteEvent(RouteId_RelativeRenderCost,		"Audio", "SoundRelativeRenderCost");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FSoundTraceAnalyzer"));

				FSoundMessages& Messages = GetProvider<FSoundTraceProvider>().TraceMessages;

				switch (RouteId)
				{
					case RouteId_ActiveSoundStart:
					{
						CacheMessage<FSoundStartMessage>(Context, Messages.ActiveSoundStartMessages);
						break;
					}

					case RouteId_SoundWaveStart:
					{
						CacheMessage<FSoundWaveStartMessage>(Context, Messages.SoundWaveStartMessages);
						break;
					}

					case RouteId_SoundIsAlivePing:
					{
						CacheMessage<FSoundIsAlivePingMessage>(Context, Messages.ActiveSoundIsAlivePingMessages);
						break;
					}

					case RouteId_SoundWaveIsAlivePing:
					{
						CacheMessage<FSoundWaveIsAlivePingMessage>(Context, Messages.SoundWaveIsAlivePingMessages);
						break;
					}

					case RouteId_Stop:
					{
						CacheMessage<FSoundStopMessage>(Context, Messages.StopMessages);
						break;
					}

					case RouteId_Priority:
					{
						CacheMessage<FSoundPriorityMessage>(Context, Messages.PriorityMessages);
						break;
					}

					case RouteId_Distance:
					{
						CacheMessage<FSoundDistanceMessage>(Context, Messages.DistanceMessages);
						break;
					}

					case RouteId_DistanceAttenuation:
					{
						CacheMessage<FSoundDistanceAttenuationMessage>(Context, Messages.DistanceAttenuationMessages);
						break;
					}

					case RouteId_Filters:
					{
						CacheMessage<FSoundLPFFreqMessage>(Context, Messages.LPFFreqMessages);
						CacheMessage<FSoundHPFFreqMessage>(Context, Messages.HPFFreqMessages);
						break;
					}

					case RouteId_Amplitude:
					{
						CacheMessage<FSoundEnvelopeMessage>(Context, Messages.AmplitudeMessages);
						break;
					}

					case RouteId_Volume:
					{
						CacheMessage<FSoundVolumeMessage>(Context, Messages.VolumeMessages);
						break;
					}

					case RouteId_Pitch:
					{
						CacheMessage<FSoundPitchMessage>(Context, Messages.PitchMessages);
						break;
					}

					case RouteId_RelativeRenderCost:
					{
						CacheMessage<FSoundRelativeRenderCostMessage>(Context, Messages.RelativeRenderCostMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_ActiveSoundStart,
				RouteId_SoundWaveStart,
				RouteId_SoundIsAlivePing,
				RouteId_SoundWaveIsAlivePing,
				RouteId_Stop,
				RouteId_Priority,
				RouteId_Distance,
				RouteId_DistanceAttenuation,
				RouteId_Filters,
				RouteId_Amplitude,
				RouteId_Volume,
				RouteId_Pitch,
				RouteId_RelativeRenderCost
			};

			TraceServices::IAnalysisSession& Session;
		};

		ActiveSoundToEntryKeysMap.Empty();
		EntriesTimingOut.Empty();
		SoundsStoppedBeforeStart.Empty();
		PlottingSoundEntries.Empty();
		PlottingEntryColors.Empty();
		PinnedSoundEntries.Empty();

		return new FSoundTraceAnalyzer(AsShared(), InSession);
	}

	bool FSoundTraceProvider::ProcessMessages()
	{
		using namespace FSoundTraceProviderPrivate;

		// Helper lambdas
		TSharedPtr<FSoundDashboardEntry> ActiveSoundEntryReturn = nullptr;
		auto GetActiveSoundEntryFromActiveStartMessage = [this, &ActiveSoundEntryReturn](const FSoundStartMessage& Msg) -> TSharedPtr<FSoundDashboardEntry>*
		{
			ActiveSoundEntryReturn = nullptr;

			GetOrCreateActiveSoundEntry(Msg, ActiveSoundEntryReturn);
			return &ActiveSoundEntryReturn;
		};

		TSharedPtr<FSoundDashboardEntry> AlivePingSoundEntryReturn = nullptr;
		auto CreateAndGetActiveSoundEntryFromPingIfNotFound = [this, &AlivePingSoundEntryReturn](const FSoundIsAlivePingMessage& Msg) -> TSharedPtr<FSoundDashboardEntry>*
		{
			AlivePingSoundEntryReturn = nullptr;

			CreateAndGetActiveSoundEntryIfNotFound(Msg, AlivePingSoundEntryReturn);

			return &AlivePingSoundEntryReturn;
		};

		TSharedPtr<FSoundDashboardEntry> SoundWaveActiveEntryReturn = nullptr;
		auto GetActiveSoundEntryFromSoundWaveStartMessage = [this, &SoundWaveActiveEntryReturn](const FSoundWaveStartMessage& Msg) -> TSharedPtr<FSoundDashboardEntry>*
		{
			SoundWaveActiveEntryReturn = nullptr;

			GetActiveSoundEntryFromIDs(Msg.ActiveSoundPlayOrder, SoundWaveActiveEntryReturn);
			return &SoundWaveActiveEntryReturn;
		};

		TSharedPtr<FSoundDashboardEntry> ParamReturn = nullptr;
		auto GetActiveSoundEntryFromParameterMessage = [this, &ParamReturn](const FSoundParameterMessage& Msg) -> TSharedPtr<FSoundDashboardEntry>*
		{
			ParamReturn = nullptr;

			GetActiveSoundEntryFromIDs(Msg.ActiveSoundPlayOrder, ParamReturn);
			return &ParamReturn;
		};

		TSharedPtr<FSoundDashboardEntry> SoundEntryReturn = nullptr;
		auto GetSoundEntryFromStopMessage = [this, &SoundEntryReturn](const FSoundStopMessage& Msg) -> TSharedPtr<FSoundDashboardEntry>*
		{
			SoundEntryReturn = nullptr;

			GetActiveSoundEntryFromIDs(Msg.ActiveSoundPlayOrder, SoundEntryReturn);
			return &SoundEntryReturn;
		};

		auto ProcessStartMessage = [this](const FSoundStartMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			if (OutActiveSoundEntry == nullptr || !OutActiveSoundEntry->IsValid())
			{
				return;
			}

			FSoundDashboardEntry& ActiveSoundEntryRef = *OutActiveSoundEntry->Get();
			ActiveSoundEntryRef.SetName(*Msg.Name);
			ActiveSoundEntryRef.EntryType = Msg.EntryType;
			ActiveSoundEntryRef.CategoryName = FText::FromString(FSoundTraceProviderPrivate::GetEntryTypeDisplayName(Msg.EntryType));
			ActiveSoundEntryRef.ActiveSoundPlayOrder = Msg.ActiveSoundPlayOrder;

			if (PlottingSoundEntries.Contains(ActiveSoundEntryRef.GetEntryID()))
			{
				ActiveSoundEntryRef.bIsPlotActive = true;
			}

			RestoreCachedStateForEntry(ActiveSoundEntryRef);

			ActiveSoundEntryRef.ActorLabel = FText::FromString(*Msg.ActorLabel);
			ActiveSoundEntryRef.AudioComponentName = FText::FromString(*Msg.AudioComponentName);
		};


		auto ProcessSoundWaveStartMessage = [this](const FSoundWaveStartMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			if (OutActiveSoundEntry == nullptr || !OutActiveSoundEntry->IsValid() || !FSoundTraceProviderPrivate::EntryTypeHasSoundWaveEntries((*OutActiveSoundEntry)->EntryType))
			{
				return;
			}

			// Avoid creating duplicate entries
			const bool bEntryExists = (*OutActiveSoundEntry)->Children.ContainsByPredicate([&Msg](const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry)
			{
				return ChildEntry.IsValid() && Msg.WaveInstancePlayOrder == CastEntry(*ChildEntry).WaveInstancePlayOrder;
			});

			if (bEntryExists)
			{
				return;
			}
			
			TSharedPtr<FSoundDashboardEntry> SoundWaveEntry = MakeShared<FSoundDashboardEntry>();
			SoundWaveEntry->DeviceId = Msg.DeviceId;
			SoundWaveEntry->WaveInstancePlayOrder = Msg.WaveInstancePlayOrder;
			SoundWaveEntry->ActiveSoundPlayOrder = Msg.ActiveSoundPlayOrder;
			SoundWaveEntry->Timestamp = Msg.Timestamp;
			SoundWaveEntry->SetName(*Msg.Name);
			SoundWaveEntry->EntryType = Msg.EntryType;
			SoundWaveEntry->CategoryName = FText::FromString(FSoundTraceProviderPrivate::GetEntryTypeDisplayName(Msg.EntryType));
			SoundWaveEntry->PinnedEntryType = (*OutActiveSoundEntry)->PinnedEntryType;
			SoundWaveEntry->ActorLabel = FText::FromString(*Msg.ActorLabel);
			SoundWaveEntry->AudioComponentName = FText::FromString(*Msg.AudioComponentName);

			if (PlottingSoundEntries.Contains(SoundWaveEntry->GetEntryID()))
			{
				SoundWaveEntry->bIsPlotActive = true;
			}

			RestoreCachedStateForEntry(*SoundWaveEntry);

			(*OutActiveSoundEntry)->Children.Add(SoundWaveEntry);
		};

		// Process messages
		ProcessMessageQueue<FSoundStartMessage>(TraceMessages.ActiveSoundStartMessages, GetActiveSoundEntryFromActiveStartMessage,
		[this, &ProcessStartMessage](const FSoundStartMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			ProcessStartMessage(Msg, OutActiveSoundEntry);
		});

		ProcessMessageQueue<FSoundWaveStartMessage>(TraceMessages.SoundWaveStartMessages, GetActiveSoundEntryFromSoundWaveStartMessage,
		[this, &ProcessSoundWaveStartMessage](const FSoundWaveStartMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			ProcessSoundWaveStartMessage(Msg, OutActiveSoundEntry);
		});

		ProcessMessageQueue<FSoundIsAlivePingMessage>(TraceMessages.ActiveSoundIsAlivePingMessages, CreateAndGetActiveSoundEntryFromPingIfNotFound,
		[this, &ProcessStartMessage](const FSoundIsAlivePingMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			ProcessStartMessage(Msg, OutActiveSoundEntry);
		});

		ProcessMessageQueue<FSoundWaveIsAlivePingMessage>(TraceMessages.SoundWaveIsAlivePingMessages, GetActiveSoundEntryFromSoundWaveStartMessage,
		[this, &ProcessSoundWaveStartMessage](const FSoundWaveIsAlivePingMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			ProcessSoundWaveStartMessage(Msg, OutActiveSoundEntry);
		});

		TArray<int32, TInlineAllocator<64>> EntriesWithPoppedDataPoints;

		ProcessMessageQueue<FSoundPriorityMessage>(TraceMessages.PriorityMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundPriorityMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.Priority, SoundEntry->PriorityDataPoint, SoundEntry->PriorityDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundDistanceMessage>(TraceMessages.DistanceMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundDistanceMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.Distance, SoundEntry->DistanceDataPoint, SoundEntry->DistanceDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundDistanceAttenuationMessage>(TraceMessages.DistanceAttenuationMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundDistanceAttenuationMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.DistanceAttenuation, SoundEntry->DistanceAttenuationDataPoint, SoundEntry->DistanceAttenuationDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundLPFFreqMessage>(TraceMessages.LPFFreqMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundLPFFreqMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.LPFFrequency, SoundEntry->LPFFreqDataPoint, SoundEntry->LPFFreqDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundHPFFreqMessage>(TraceMessages.HPFFreqMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundHPFFreqMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.HPFFrequency, SoundEntry->HPFFreqDataPoint, SoundEntry->HPFFreqDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundEnvelopeMessage>(TraceMessages.AmplitudeMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundEnvelopeMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.Envelope, SoundEntry->AmplitudeDataPoint, SoundEntry->AmplitudeDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundVolumeMessage>(TraceMessages.VolumeMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundVolumeMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.Volume, SoundEntry->VolumeDataPoint, SoundEntry->VolumeDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundPitchMessage>(TraceMessages.PitchMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundPitchMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.Pitch, SoundEntry->PitchDataPoint, SoundEntry->PitchDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundRelativeRenderCostMessage>(TraceMessages.RelativeRenderCostMessages, GetActiveSoundEntryFromParameterMessage,
		[this, &EntriesWithPoppedDataPoints](const FSoundRelativeRenderCostMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			FSoundDashboardEntry* SoundEntry = FSoundTraceProviderPrivate::FindWaveEntryInActiveSound(Msg.WaveInstancePlayOrder, OutActiveSoundEntry);
			if (SoundEntry == nullptr)
			{
				return;
			}

			FSoundTraceProviderPrivate::UpdateParameterEntry(Msg, Msg.RelativeRenderCost, SoundEntry->RelativeRenderCostDataPoint, SoundEntry->RelativeRenderCostDataRange, EntriesWithPoppedDataPoints);
		});
		EntriesWithPoppedDataPoints.Reset();

		ProcessMessageQueue<FSoundStopMessage>(TraceMessages.StopMessages, GetSoundEntryFromStopMessage,
		[this](const FSoundStopMessage& Msg, const TSharedPtr<FSoundDashboardEntry>* OutActiveSoundEntry)
		{
			if (OutActiveSoundEntry && OutActiveSoundEntry->IsValid())
			{
				if ((*OutActiveSoundEntry)->Timestamp < Msg.Timestamp)
				{
					const double TimeoutTimestamp = FPlatformTime::Seconds() + DashboardTimeoutTime;

					(*OutActiveSoundEntry)->TimeoutTimestamp = TimeoutTimestamp;
					for (TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry : (*OutActiveSoundEntry)->Children)
					{
						FSoundDashboardEntry& SoundWaveEntry = FSoundTraceProviderPrivate::CastEntry(*ChildEntry.Get());
						SoundWaveEntry.TimeoutTimestamp = TimeoutTimestamp;

						if (SoundWaveEntry.bIsPlotActive)
						{
							PlottingSoundEntries.FindOrAdd(SoundWaveEntry.GetEntryID());
						}
						else if (PlottingSoundEntries.Contains(SoundWaveEntry.GetEntryID()))
						{
							PlottingSoundEntries.Remove(SoundWaveEntry.GetEntryID());
						}
					}

					EntriesTimingOut.Push(SoundMessageIDs{ Msg.DeviceId, Msg.ActiveSoundPlayOrder });

					// Cache the bIsPlotActive flag in case this sound returns in future
					if ((*OutActiveSoundEntry)->bIsPlotActive)
					{
						PlottingSoundEntries.FindOrAdd((*OutActiveSoundEntry)->GetEntryID());
					}
					else if (PlottingSoundEntries.Contains((*OutActiveSoundEntry)->GetEntryID()))
					{
						PlottingSoundEntries.Remove((*OutActiveSoundEntry)->GetEntryID());
					}
				}
			}
			else
			{
				// Keep track of any messages that have sent stop but no start message
				// In rare cases these can be processed out of order
				SoundsStoppedBeforeStart.Add(Msg.ActiveSoundPlayOrder);
			}
		});

		UpdateAggregateActiveSoundData();

		return true;
	}

	bool FSoundTraceProvider::ProcessManuallyUpdatedEntries()
	{
		// Timeout old dashboard entries
		const double CurrentTimeStamp = FPlatformTime::Seconds();

		int32 NumToTrim = 0;
		for (const SoundMessageIDs SoundMessageIDs : EntriesTimingOut)
		{
			TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry = nullptr;
			GetActiveSoundEntryFromIDs(SoundMessageIDs.ActiveSoundPlayOrder, ActiveSoundEntry);

			if (!ActiveSoundEntry.IsValid())
			{
				NumToTrim++;
				continue;
			}

			if (ActiveSoundEntry->TimeoutTimestamp <= CurrentTimeStamp)
			{
				RemoveActiveSoundEntry(ActiveSoundEntry);
				NumToTrim++;
			}
			else
			{
				break;
			}
		}

		if (NumToTrim > 0)
		{
			EntriesTimingOut.RemoveAt(0, NumToTrim, EAllowShrinking::No);
			return true;
		}

		return false;
	}

	void FSoundTraceProvider::GetOrCreateActiveSoundEntry(const FSoundStartMessage& Msg, TSharedPtr<FSoundDashboardEntry>& OutReturnedSoundEntry)
	{
		OutReturnedSoundEntry = nullptr;
		
		// We cannot always guarentee the order of receiving start and stop messages
		// If the stop message preceeded the start message, do not create a new entry
		if (SoundsStoppedBeforeStart.Contains(Msg.ActiveSoundPlayOrder))
		{
			SoundsStoppedBeforeStart.Remove(Msg.ActiveSoundPlayOrder);
			return;
		}

		UpdateDeviceEntry(Msg.DeviceId, Msg.EntryType, [this, &OutReturnedSoundEntry, &Msg](TSharedPtr<FSoundDashboardEntry>& CategoryEntry)
		{
			if (!CategoryEntry.IsValid())
			{
				CategoryEntry = MakeShared<FSoundDashboardEntry>();
				CategoryEntry->DeviceId = Msg.DeviceId;
				CategoryEntry->EntryType = Msg.EntryType;
				CategoryEntry->SetName(FSoundTraceProviderPrivate::GetEntryTypeDisplayName(Msg.EntryType));
				CategoryEntry->bIsCategory = true;
			}

			TSharedPtr<IDashboardDataTreeViewEntry>* ActiveSoundEntry = CategoryEntry->Children.FindByPredicate([&Msg](TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry)
			{
				FSoundDashboardEntry& NewEntry = FSoundTraceProviderPrivate::CastEntry(*ChildEntry.Get());
				return NewEntry.ActiveSoundPlayOrder == Msg.ActiveSoundPlayOrder;
			});

			if (ActiveSoundEntry == nullptr || !ActiveSoundEntry->IsValid())
			{
				TSharedPtr<FSoundDashboardEntry> NewEntry = MakeShared<FSoundDashboardEntry>();
				NewEntry->DeviceId = Msg.DeviceId;
				NewEntry->ActiveSoundPlayOrder = Msg.ActiveSoundPlayOrder;

				CategoryEntry->Children.Add(NewEntry);

				ActiveSoundToEntryKeysMap.Add(Msg.ActiveSoundPlayOrder, SoundEntryKeys{ Msg.EntryType, Msg.DeviceId });

				OutReturnedSoundEntry = NewEntry;
			}
			else
			{
				TSharedPtr<FSoundDashboardEntry> NewEntry = StaticCastSharedPtr<FSoundDashboardEntry>(*ActiveSoundEntry);
				if (NewEntry->TimeoutTimestamp != INVALID_TIMEOUT)
				{
					NewEntry->TimeoutTimestamp = INVALID_TIMEOUT;

					EntriesTimingOut.RemoveAll([&Msg](const SoundMessageIDs& SoundMessageIDs)
					{
						return Msg.DeviceId == SoundMessageIDs.DeviceId && Msg.ActiveSoundPlayOrder == SoundMessageIDs.ActiveSoundPlayOrder;
					});

					// If this entry has SoundWaves, they will replay with new play order IDs, so we need to clear the old ones out
					NewEntry->Children.RemoveAll([](TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry)
					{
						FSoundDashboardEntry& SoundWaveEntry = FSoundTraceProviderPrivate::CastEntry(*ChildEntry.Get());
						return SoundWaveEntry.TimeoutTimestamp != INVALID_TIMEOUT;
					});
				}
				OutReturnedSoundEntry = NewEntry;
			}

			if (OutReturnedSoundEntry == nullptr)
			{
				return;
			}

			OutReturnedSoundEntry->Timestamp = Msg.Timestamp;
		});
	}

	void FSoundTraceProvider::CreateAndGetActiveSoundEntryIfNotFound(const FSoundStartMessage& Msg, TSharedPtr<FSoundDashboardEntry>& OutReturnedSoundEntry)
	{
		OutReturnedSoundEntry = nullptr;

		GetActiveSoundEntryFromIDs(Msg.ActiveSoundPlayOrder, OutReturnedSoundEntry);
		if (!OutReturnedSoundEntry.IsValid())
		{
			GetOrCreateActiveSoundEntry(Msg, OutReturnedSoundEntry);
		}
		else
		{
			OutReturnedSoundEntry.Reset();
		}
	}

	void FSoundTraceProvider::GetActiveSoundEntryFromIDs(const uint32 ActiveSoundPlayOrder, TSharedPtr<FSoundDashboardEntry>& OutActiveSoundEntry)
	{
		const SoundEntryKeys* SoundEntryKeys = ActiveSoundToEntryKeysMap.Find(ActiveSoundPlayOrder);
		if (SoundEntryKeys == nullptr)
		{
			return;
		}

		TSharedPtr<FSoundDashboardEntry>* CategoryEntry = FindDeviceEntry(SoundEntryKeys->DeviceId, SoundEntryKeys->EntryType);

		FSoundTraceProviderPrivate::FindActiveSoundEntryInCategory(CategoryEntry, ActiveSoundPlayOrder, OutActiveSoundEntry);
	}

	void FSoundTraceProvider::RemoveActiveSoundEntry(TSharedPtr<FSoundDashboardEntry> OutActiveSoundEntry)
	{
		if (!OutActiveSoundEntry.IsValid())
		{
			return;
		}

		const uint32 ActiveSoundPlayOrder = OutActiveSoundEntry->ActiveSoundPlayOrder;
		const SoundEntryKeys* SoundEntryKeys = ActiveSoundToEntryKeysMap.Find(ActiveSoundPlayOrder);

		if (SoundEntryKeys == nullptr)
		{
			return;
		}

		TSharedPtr<FSoundDashboardEntry>* CategoryEntry = FindDeviceEntry(SoundEntryKeys->DeviceId, SoundEntryKeys->EntryType);
		if (CategoryEntry == nullptr || !CategoryEntry->IsValid())
		{
			return;
		}

		(*CategoryEntry)->Children.Remove(OutActiveSoundEntry);

		if ((*CategoryEntry)->Children.Num() == 0)
		{
			RemoveDeviceEntry(SoundEntryKeys->DeviceId, OutActiveSoundEntry->EntryType);
		}

		ActiveSoundToEntryKeysMap.Remove(ActiveSoundPlayOrder);
	}

	void FSoundTraceProvider::UpdateAggregateActiveSoundData()
	{
		using namespace FSoundTraceProviderPrivate;

		for (const auto& [AudioDeviceID, DeviceData] : DeviceDataMap)
		{
			for (const auto& [EntryType, SoundDashboardEntry] : DeviceData)
			{
				if (!SoundDashboardEntry.IsValid() || !EntryTypeHasSoundWaveEntries(EntryType))
				{
					continue;
				}

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& ActiveSoundEntry : SoundDashboardEntry->Children)
				{
					if (ActiveSoundEntry.IsValid())
					{
						CollectAggregateData(CastEntry(*ActiveSoundEntry));
					}
				}
			}
		}
	}

	void FSoundTraceProvider::CollectAggregateData(FSoundDashboardEntry& ActiveSoundEntry)
	{
		using namespace FSoundTraceProviderPrivate;

		const int32 NumWaveInstances = ActiveSoundEntry.Children.Num();
		if (NumWaveInstances <= 0)
		{
			return;
		}

		struct FCumulativeDataPoint
		{
			float TotalAccumulationDataPoint = { 0.0f };
			int32 NumDataPoints = 0;
		};

		auto AccumulateDataPoints = [](const FCumulativeDataPoint& AccumulativeValue, const float DataPoint) -> FCumulativeDataPoint
		{
			return
			{
				AccumulativeValue.TotalAccumulationDataPoint + DataPoint,
				AccumulativeValue.NumDataPoints + 1
			};
		};

		auto AverageDataPoints = [](const FCumulativeDataPoint& DataPointsTotal) -> float
		{
			if (DataPointsTotal.NumDataPoints == 0)
			{
				return 0.0f;
			}

			return DataPointsTotal.TotalAccumulationDataPoint / static_cast<float>(DataPointsTotal.NumDataPoints);
		};

		auto AddSingleValueToBuffer = [](::Audio::TCircularAudioBuffer<FDataPoint>& To, const FDataPoint& Value)
		{
			if (To.Num() > 0)
			{
				To.Pop(1);
			}

			To.Push(Value);
		};

		float MaxVolume = 0.0f;
		float MinDistance = TNumericLimits<float>::Max();
		float MaxPriority = 0.0f;

		float AveragePeakAmp = 0.0f;
		FCumulativeDataPoint CumulativePeakAmplitude;

		float AveragePitch = 0.0f;
		FCumulativeDataPoint CumulativePitch;

		float AverageDistanceAttenuation = 0.0f;
		FCumulativeDataPoint CumulativeDistanceAttenuation;

		float AverageLPFFreq = 0.0f;
		FCumulativeDataPoint CumulativeLPFFreq;

		float AverageHPFFreq = 0.0f;
		FCumulativeDataPoint CumulativeHPFFreq;

		FCumulativeDataPoint CumulativeRelativeRenderCost;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& WaveInstanceEntry : ActiveSoundEntry.Children)
		{
			const FSoundDashboardEntry& WaveSoundEntry = CastEntry(*WaveInstanceEntry);

			MaxVolume = FMath::Max(MaxVolume, WaveSoundEntry.VolumeDataPoint);
			MinDistance = FMath::Min(MinDistance, WaveSoundEntry.DistanceDataPoint);
			MaxPriority = FMath::Max(MaxPriority, WaveSoundEntry.PriorityDataPoint);

			CumulativePeakAmplitude = AccumulateDataPoints(CumulativePeakAmplitude, WaveSoundEntry.AmplitudeDataPoint);
			CumulativePitch = AccumulateDataPoints(CumulativePitch, WaveSoundEntry.PitchDataPoint);
			CumulativeDistanceAttenuation = AccumulateDataPoints(CumulativeDistanceAttenuation, WaveSoundEntry.DistanceAttenuationDataPoint);
			CumulativeLPFFreq = AccumulateDataPoints(CumulativeLPFFreq, WaveSoundEntry.LPFFreqDataPoint);
			CumulativeHPFFreq = AccumulateDataPoints(CumulativeHPFFreq, WaveSoundEntry.HPFFreqDataPoint);
			CumulativeRelativeRenderCost = AccumulateDataPoints(CumulativeRelativeRenderCost, WaveSoundEntry.RelativeRenderCostDataPoint);
		}

		AveragePeakAmp = AverageDataPoints(CumulativePeakAmplitude);
		AveragePitch = AverageDataPoints(CumulativePitch);
		AverageDistanceAttenuation = AverageDataPoints(CumulativeDistanceAttenuation);
		AverageLPFFreq = AverageDataPoints(CumulativeLPFFreq);
		AverageHPFFreq = AverageDataPoints(CumulativeHPFFreq);

		AddSingleValueToBuffer(ActiveSoundEntry.VolumeDataRange, { ActiveSoundEntry.Timestamp, MaxVolume });
		ActiveSoundEntry.VolumeDataPoint = MaxVolume;

		AddSingleValueToBuffer(ActiveSoundEntry.DistanceDataRange, { ActiveSoundEntry.Timestamp, MinDistance });
		ActiveSoundEntry.DistanceDataPoint = MinDistance;

		AddSingleValueToBuffer(ActiveSoundEntry.DistanceAttenuationDataRange, { ActiveSoundEntry.Timestamp, AverageDistanceAttenuation });
		ActiveSoundEntry.DistanceAttenuationDataPoint = AverageDistanceAttenuation;

		AddSingleValueToBuffer(ActiveSoundEntry.LPFFreqDataRange, { ActiveSoundEntry.Timestamp, AverageLPFFreq });
		ActiveSoundEntry.LPFFreqDataPoint = AverageLPFFreq;

		AddSingleValueToBuffer(ActiveSoundEntry.HPFFreqDataRange, { ActiveSoundEntry.Timestamp, AverageHPFFreq });
		ActiveSoundEntry.HPFFreqDataPoint = AverageHPFFreq;

		AddSingleValueToBuffer(ActiveSoundEntry.PriorityDataRange, { ActiveSoundEntry.Timestamp, MaxPriority });
		ActiveSoundEntry.PriorityDataPoint = MaxPriority;

		AddSingleValueToBuffer(ActiveSoundEntry.AmplitudeDataRange, { ActiveSoundEntry.Timestamp, AveragePeakAmp });
		ActiveSoundEntry.AmplitudeDataPoint = AveragePeakAmp;

		AddSingleValueToBuffer(ActiveSoundEntry.PitchDataRange, { ActiveSoundEntry.Timestamp, AveragePitch });
		ActiveSoundEntry.PitchDataPoint = AveragePitch;

		AddSingleValueToBuffer(ActiveSoundEntry.RelativeRenderCostDataRange, { ActiveSoundEntry.Timestamp, CumulativeRelativeRenderCost.TotalAccumulationDataPoint });
		ActiveSoundEntry.RelativeRenderCostDataPoint = CumulativeRelativeRenderCost.TotalAccumulationDataPoint;
	}

	void FSoundTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		using namespace FSoundTraceProviderPrivate;

		auto InitActiveSoundEntry = [this](const FSoundStartMessage& Msg, FSoundDashboardEntry& OutActiveSoundEntry)
		{
			OutActiveSoundEntry.SetName(*Msg.Name);
			OutActiveSoundEntry.EntryType = Msg.EntryType;
			OutActiveSoundEntry.bForceKeepEntryAlive = false;
			OutActiveSoundEntry.ActorLabel = FText::FromString(*Msg.ActorLabel);
			OutActiveSoundEntry.AudioComponentName = FText::FromString(*Msg.AudioComponentName);

			if (!OutActiveSoundEntry.bIsPlotActive)
			{
				OutActiveSoundEntry.bIsPlotActive = PlottingSoundEntries.Contains(OutActiveSoundEntry.GetEntryID());
			}

			RestoreCachedStateForEntry(OutActiveSoundEntry);
		};

		auto CreateChildEntry = [this](const FSoundWaveStartMessage& Msg, const TSharedPtr<FSoundDashboardEntry>& OutParentSoundEntry)
		{
			TSharedPtr<FSoundDashboardEntry> ChildEntry = MakeShared<FSoundDashboardEntry>();
			ChildEntry->DeviceId = Msg.DeviceId;
			ChildEntry->WaveInstancePlayOrder = Msg.WaveInstancePlayOrder;
			ChildEntry->ActiveSoundPlayOrder = Msg.ActiveSoundPlayOrder;
			ChildEntry->Timestamp = Msg.Timestamp;
			ChildEntry->SetName(*Msg.Name);
			ChildEntry->ActorLabel = FText::FromString(*Msg.ActorLabel);
			ChildEntry->AudioComponentName = FText::FromString(*Msg.AudioComponentName);
			ChildEntry->EntryType = Msg.EntryType;
			ChildEntry->bForceKeepEntryAlive = false;

			if (!ChildEntry->bIsPlotActive)
			{
				ChildEntry->bIsPlotActive = PlottingSoundEntries.Contains(ChildEntry->GetEntryID());
			}

			RestoreCachedStateForEntry(*ChildEntry);

			OutParentSoundEntry->Children.Add(ChildEntry);
		};

		for (const auto& [AudioDeviceID, DeviceData] : DeviceDataMap)
		{
			for (const auto& [EntryType, SoundDashboardEntry] : DeviceData)
			{
				CacheIsPlottingFlagRecursive(SoundDashboardEntry);
				CachePinnedFlagRecursive(SoundDashboardEntry);
			}
		}

		TSet<uint32> AliveActiveSoundIDs;
		TSet<uint32> AliveWaveInstanceIDs;

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		// Collect all the start messages registered until this point in time
		auto ProcessStartMessage = [this, &AliveActiveSoundIDs, &InitActiveSoundEntry](const FSoundStartMessage& StartCachedMessage)
		{
			TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry = nullptr;
			GetOrCreateActiveSoundEntry(StartCachedMessage, ActiveSoundEntry);

			if (!ActiveSoundEntry.IsValid())
			{
				return;
			}

			AliveActiveSoundIDs.Add(StartCachedMessage.ActiveSoundPlayOrder);
			InitActiveSoundEntry(StartCachedMessage, *ActiveSoundEntry);
		};

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();
		CacheManager.IterateOverRange<FSoundStartMessage>(SoundMessageNames::SoundStart, RangeCheckStart, TimeMarker, [&ProcessStartMessage](const FSoundStartMessage& StartCachedMessage)
		{
			ProcessStartMessage(StartCachedMessage);
		});
			
		// Collect all the is alive ping messages registered until this point in time
		auto ProcessSoundIsAlivePingMessage = [this, &AliveActiveSoundIDs, &InitActiveSoundEntry](const FSoundIsAlivePingMessage& IsAlivePingCachedMessage)
		{
			AliveActiveSoundIDs.Add(IsAlivePingCachedMessage.ActiveSoundPlayOrder);

			TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry = nullptr;
			CreateAndGetActiveSoundEntryIfNotFound(IsAlivePingCachedMessage, ActiveSoundEntry);

			if (!ActiveSoundEntry.IsValid())
			{
				return;
			}

			InitActiveSoundEntry(IsAlivePingCachedMessage, *ActiveSoundEntry);
		};

		CacheManager.IterateOverRange<FSoundIsAlivePingMessage>(SoundMessageNames::SoundIsAlivePing, RangeCheckStart, TimeMarker, [&ProcessSoundIsAlivePingMessage](const FSoundIsAlivePingMessage& IsAlivePingCachedMessage)
		{
			ProcessSoundIsAlivePingMessage(IsAlivePingCachedMessage);
		});

		////////////////////////////////////////
		// Add all soundwave start messages registered until this point in time 
		auto ProcessSoundWaveStartMessage = [this, &AliveActiveSoundIDs, &AliveWaveInstanceIDs, &CreateChildEntry](const FSoundWaveStartMessage& SoundWaveStartCachedMessage)
		{
			TSharedPtr<FSoundDashboardEntry> ParentSoundEntryReturn = nullptr;
			GetActiveSoundEntryFromIDs(SoundWaveStartCachedMessage.ActiveSoundPlayOrder, ParentSoundEntryReturn);

			if (ParentSoundEntryReturn == nullptr || !ParentSoundEntryReturn->IsValid() || !FSoundTraceProviderPrivate::EntryTypeHasSoundWaveEntries(ParentSoundEntryReturn->EntryType))
			{
				return;
			}

			AliveActiveSoundIDs.Add(SoundWaveStartCachedMessage.ActiveSoundPlayOrder);
			AliveWaveInstanceIDs.Add(SoundWaveStartCachedMessage.WaveInstancePlayOrder);

			const bool bEntryExists = ParentSoundEntryReturn->Children.ContainsByPredicate([&SoundWaveStartCachedMessage](const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry)
			{
				return ChildEntry.IsValid() && SoundWaveStartCachedMessage.WaveInstancePlayOrder == CastEntry(*ChildEntry).WaveInstancePlayOrder;
			});

			if (bEntryExists)
			{
				return;
			}

			CreateChildEntry(SoundWaveStartCachedMessage, ParentSoundEntryReturn);
		};

		CacheManager.IterateOverRange<FSoundWaveStartMessage>(SoundMessageNames::SoundWaveStart, RangeCheckStart, TimeMarker, [&ProcessSoundWaveStartMessage](const FSoundWaveStartMessage& SoundWaveStartCachedMessage)
		{
			ProcessSoundWaveStartMessage(SoundWaveStartCachedMessage);
		});

		////////////////////////////////////////
		// Add all soundwave ping messages registered until this point in time 
		auto ProcessSoundWaveIsAlivePingMessage = [this, &AliveActiveSoundIDs, &AliveWaveInstanceIDs, &CreateChildEntry](const FSoundWaveIsAlivePingMessage& SoundWaveIsAlivePingCachedMessage)
		{
			TSharedPtr<FSoundDashboardEntry> ParentSoundIsAlivePingEntryReturn = nullptr;
			GetActiveSoundEntryFromIDs(SoundWaveIsAlivePingCachedMessage.ActiveSoundPlayOrder, ParentSoundIsAlivePingEntryReturn);

			if (ParentSoundIsAlivePingEntryReturn == nullptr || !ParentSoundIsAlivePingEntryReturn->IsValid() || !FSoundTraceProviderPrivate::EntryTypeHasSoundWaveEntries(ParentSoundIsAlivePingEntryReturn->EntryType))
			{
				return;
			}

			AliveActiveSoundIDs.Add(SoundWaveIsAlivePingCachedMessage.ActiveSoundPlayOrder);
			AliveWaveInstanceIDs.Add(SoundWaveIsAlivePingCachedMessage.WaveInstancePlayOrder);

			// Avoid creating duplicate entries when receiving the IsAlivePing message
			const bool bEntryExists = ParentSoundIsAlivePingEntryReturn->Children.ContainsByPredicate([&SoundWaveIsAlivePingCachedMessage](const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry)
			{
				return ChildEntry.IsValid() && SoundWaveIsAlivePingCachedMessage.WaveInstancePlayOrder == CastEntry(*ChildEntry).WaveInstancePlayOrder;
			});

			if (bEntryExists)
			{
				return;
			}

			CreateChildEntry(SoundWaveIsAlivePingCachedMessage, ParentSoundIsAlivePingEntryReturn);
		};

		CacheManager.IterateOverRange<FSoundWaveIsAlivePingMessage>(SoundMessageNames::SoundWaveIsAlivePing, RangeCheckStart, TimeMarker, [&ProcessSoundWaveIsAlivePingMessage](const FSoundWaveIsAlivePingMessage& SoundWaveIsAlivePingCachedMessage)
		{
			ProcessSoundWaveIsAlivePingMessage(SoundWaveIsAlivePingCachedMessage);
		});

		// Selectively remove start messages collected in the step above by knowing which sounds were stopped.
		// With this we will know what are the active sounds at this point in time.
		auto ProcessSoundStopMessageMessage = [this, &AliveWaveInstanceIDs](const FSoundStopMessage& StopCachedMessage)
		{
			TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry = nullptr;
			GetActiveSoundEntryFromIDs(StopCachedMessage.ActiveSoundPlayOrder, ActiveSoundEntry);

			if (ActiveSoundEntry.IsValid())
			{
				if (ActiveSoundEntry->Timestamp < StopCachedMessage.Timestamp)
				{
					if (ActiveSoundEntry->bIsPlotActive)
					{
						ActiveSoundEntry->bForceKeepEntryAlive = true;
					}
					else
					{
						// This active sound has been stopped, clean up any child Wave Instance play orders
						for (auto Iter = ActiveSoundEntry->Children.CreateConstIterator(); Iter; ++Iter)
						{
							const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry = *Iter;
							if (ChildEntry.IsValid())
							{
								AliveWaveInstanceIDs.Remove(CastEntry(*ChildEntry).WaveInstancePlayOrder);
							}
						}

						RemoveActiveSoundEntry(ActiveSoundEntry);
					}
				}
				else
				{
					// This active sound is alive, but may have virtualized/realized
					// Remove any old wave instances that are no longer alive
					for (auto Iter = ActiveSoundEntry->Children.CreateIterator(); Iter; ++Iter)
					{
						const TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry = *Iter;
						if (!ChildEntry.IsValid())
						{
							Iter.RemoveCurrent();
						}
						else if (CastEntry(*ChildEntry).Timestamp < StopCachedMessage.Timestamp)
						{
							AliveWaveInstanceIDs.Remove(CastEntry(*ChildEntry).WaveInstancePlayOrder);
							Iter.RemoveCurrent();
						}
					}
				}
			}
		};

		CacheManager.IterateOverRange<FSoundStopMessage>(SoundMessageNames::SoundStop, RangeCheckStart, TimeMarker, [&ProcessSoundStopMessageMessage](const FSoundStopMessage& StopCachedMessage)
		{
			ProcessSoundStopMessageMessage(StopCachedMessage);
		});

		TArray<uint32> ActiveSoundPlayOrderArray;
		ActiveSoundToEntryKeysMap.GetKeys(ActiveSoundPlayOrderArray);

		// Iterate through the remaining active sounds we have in the dashboard - check whether we detected a start or IsAlive ping message for that entry
		// If not, we assume this entry is no longer playing and remove it from the dashboard.
		for (int ReverseIndex = ActiveSoundPlayOrderArray.Num() - 1; ReverseIndex >= 0; --ReverseIndex)
		{
			const uint32 ActiveSoundPlayOrder = ActiveSoundPlayOrderArray[ReverseIndex];

			TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
			GetActiveSoundEntryFromIDs(ActiveSoundPlayOrder, ActiveSoundEntry);

			if (!ActiveSoundEntry.IsValid())
			{
				continue;
			}

			if (AliveActiveSoundIDs.Contains(ActiveSoundPlayOrder))
			{
				// This active sound is alive - check it's wave instances
				for (auto Iter = ActiveSoundEntry->Children.CreateIterator(); Iter; ++Iter)
				{
					const TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry = *Iter;
					if (!ChildEntry.IsValid() || !AliveWaveInstanceIDs.Contains(CastEntry(*ChildEntry).WaveInstancePlayOrder))
					{
						Iter.RemoveCurrent();
					}
				}
			}
			else
			{
				// This active sound is not alive - remove it and clean up any wave instances
				for (auto Iter = ActiveSoundEntry->Children.CreateConstIterator(); Iter; ++Iter)
				{
					const TSharedPtr<IDashboardDataTreeViewEntry> ChildEntry = *Iter;
					if (ChildEntry.IsValid())
					{
						AliveWaveInstanceIDs.Remove(CastEntry(*ChildEntry).WaveInstancePlayOrder);
					}
				}

				RemoveActiveSoundEntry(ActiveSoundEntry);
			}
		}

		CollectParamsForTimestamp(CacheManager, TimeMarker);

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);
	}

	void FSoundTraceProvider::OnTimeControlMethodReset()
	{
		for (const auto& [AudioDeviceID, DeviceData] : DeviceDataMap)
		{
			for (const auto& [EntryType, SoundDashboardEntry] : DeviceData)
			{
				CacheIsPlottingFlagRecursive(SoundDashboardEntry);
				CachePinnedFlagRecursive(SoundDashboardEntry);
			}
		}

		Reset();
	}

	void FSoundTraceProvider::CollectParamsForTimestamp(const FAudioInsightsCacheManager& CacheManager, const double TimeMarker)
	{
		using namespace FSoundTraceProviderPrivate;

		for (const auto& [AudioDeviceID, DeviceData] : DeviceDataMap)
		{
			for (const auto& [EntryType, SoundDashboardEntry] : DeviceData)
			{
				ResetDataBuffersRecursive(SoundDashboardEntry);
			}
		}

		enum class ESoundParamMessageType : uint8
		{
			Priority = 0u,
			Distance,
			DistanceAttenuation,
			LPFFreq,
			HPFFreq,
			Envelope,
			Volume,
			Pitch,
			RelativeRenderCost,

			/////
			MAX
		};

		constexpr uint32 NumMessageTypesToProcess = static_cast<uint32>(ESoundParamMessageType::MAX);

		const FAudioInsightsModule& AudioInsightsModule = UE::Audio::Insights::FAudioInsightsModule::GetChecked();
		const TRange<double> PlottingRange = AudioInsightsModule.GetTimingViewExtender().GetPlottingRange();

		// Add a small margin to either side to avoid gaps at the edges
		const double StartTime = FMath::Max(0.0f, PlottingRange.GetLowerBoundValue() - FAudioInsightsTimingViewExtender::PlottingMarginSeconds);
		const double EndTime = FMath::Max(0.0f, PlottingRange.GetUpperBoundValue() + FAudioInsightsTimingViewExtender::PlottingMarginSeconds);

		// Using ParallelFor to speed-up the cached messages retrieval, using a traditional for loop is unacceptably slower, specially in large traces.
		ParallelFor(NumMessageTypesToProcess, [&CacheManager, TimeMarker, StartTime, EndTime, this](const int32 Index)
		{
			switch (static_cast<ESoundParamMessageType>(Index))
			{
			case ESoundParamMessageType::Priority:
				CacheManager.IterateOverRange<FSoundPriorityMessage>(SoundMessageNames::PriorityParam, StartTime, EndTime, [this, TimeMarker](const FSoundPriorityMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);
					
					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->PriorityDataRange.Push({ Message.Timestamp, Message.Priority });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->PriorityDataPoint = Message.Priority;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::Distance:
				CacheManager.IterateOverRange<FSoundDistanceMessage>(SoundMessageNames::DistanceParam, StartTime, EndTime, [this, TimeMarker](const FSoundDistanceMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->DistanceDataRange.Push({ Message.Timestamp, Message.Distance });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->DistanceDataPoint = Message.Distance;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::DistanceAttenuation:
				CacheManager.IterateOverRange<FSoundDistanceAttenuationMessage>(SoundMessageNames::DistanceAttenuationParam, StartTime, EndTime, [this, TimeMarker](const FSoundDistanceAttenuationMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->DistanceAttenuationDataRange.Push({ Message.Timestamp, Message.DistanceAttenuation });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->DistanceAttenuationDataPoint = Message.DistanceAttenuation;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::LPFFreq:
				CacheManager.IterateOverRange<FSoundLPFFreqMessage>(SoundMessageNames::LPFFreqParam, StartTime, EndTime, [this, TimeMarker](const FSoundLPFFreqMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->LPFFreqDataRange.Push({ Message.Timestamp, Message.LPFFrequency });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->LPFFreqDataPoint = Message.LPFFrequency;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::HPFFreq:
				CacheManager.IterateOverRange<FSoundHPFFreqMessage>(SoundMessageNames::HPFFreqParam, StartTime, EndTime, [this, TimeMarker](const FSoundHPFFreqMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->HPFFreqDataRange.Push({ Message.Timestamp, Message.HPFFrequency });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->HPFFreqDataPoint = Message.HPFFrequency;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::Envelope:
				CacheManager.IterateOverRange<FSoundEnvelopeMessage>(SoundMessageNames::EnvelopeParam, StartTime, EndTime, [this, TimeMarker](const FSoundEnvelopeMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->AmplitudeDataRange.Push({ Message.Timestamp, Message.Envelope });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->AmplitudeDataPoint = Message.Envelope;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::Volume:
				CacheManager.IterateOverRange<FSoundVolumeMessage>(SoundMessageNames::VolumeParam, StartTime, EndTime, [this, TimeMarker](const FSoundVolumeMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->VolumeDataRange.Push({ Message.Timestamp, Message.Volume });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->VolumeDataPoint = Message.Volume;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::Pitch:
				CacheManager.IterateOverRange<FSoundPitchMessage>(SoundMessageNames::PitchParam, StartTime, EndTime, [this, TimeMarker](const FSoundPitchMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->PitchDataRange.Push({ Message.Timestamp, Message.Pitch });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->PitchDataPoint = Message.Pitch;
							}
						}
					}
				});
				break;

			case ESoundParamMessageType::RelativeRenderCost:
				CacheManager.IterateOverRange<FSoundRelativeRenderCostMessage>(SoundMessageNames::RelativeRenderCostParam, StartTime, EndTime, [this, TimeMarker](const FSoundRelativeRenderCostMessage& Message)
				{
					TSharedPtr<FSoundDashboardEntry> ActiveSoundEntry;
					GetActiveSoundEntryFromIDs(Message.ActiveSoundPlayOrder, ActiveSoundEntry);

					if (ActiveSoundEntry.IsValid())
					{
						if (FSoundDashboardEntry* SoundEntry = FindWaveEntryInActiveSound(Message.WaveInstancePlayOrder, &ActiveSoundEntry))
						{
							SoundEntry->RelativeRenderCostDataRange.Push({ Message.Timestamp, Message.RelativeRenderCost });
							SoundEntry->Timestamp = Message.Timestamp;

							if (Message.Timestamp <= TimeMarker)
							{
								SoundEntry->RelativeRenderCostDataPoint = Message.RelativeRenderCost;
							}
						}
					}
				});
				break;

			default:
				break;
			}
		});

		UpdateAggregateActiveSoundData();

		OnProcessPlotData.Broadcast();
	}

	void FSoundTraceProvider::CacheIsPlottingFlagRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		using namespace FSoundTraceProviderPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

		// Cache the bIsPlotActive flag and the entry color in case this sound returns in future
		if (SoundEntry.bIsPlotActive)
		{
			PlottingSoundEntries.FindOrAdd(SoundEntry.GetEntryID());
			PlottingEntryColors.Emplace(SoundEntry.GetEntryID(), SoundEntry.GetEntryColor());
		}
		else if (PlottingSoundEntries.Contains(SoundEntry.GetEntryID()))
		{
			PlottingSoundEntries.Remove(SoundEntry.GetEntryID());
			PlottingEntryColors.Remove(SoundEntry.GetEntryID());
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : SoundEntry.Children)
		{
			CacheIsPlottingFlagRecursive(Child);
		}
	}

	void FSoundTraceProvider::CachePinnedFlagRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		using namespace FSoundTraceProviderPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

		// Cache the PinnedEntryType=HiddenOriginalEntry flag in case this sound returns in future
		if (SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry)
		{
			PinnedSoundEntries.FindOrAdd(SoundEntry.GetEntryID());
		}
		else if (PinnedSoundEntries.Contains(SoundEntry.GetEntryID()))
		{
			PinnedSoundEntries.Remove(SoundEntry.GetEntryID());
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : SoundEntry.Children)
		{
			CachePinnedFlagRecursive(Child);
		}
	}

	void FSoundTraceProvider::ResetDataBuffersRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		using namespace FSoundTraceProviderPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

		constexpr uint32 DataPointsCapacity = 2048u;
		SoundEntry.ResetDataBuffers(DataPointsCapacity);

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : SoundEntry.Children)
		{
			ResetDataBuffersRecursive(Child);
		}
	}

	void FSoundTraceProvider::RestoreCachedStateForEntry(FSoundDashboardEntry& OutEntry)
	{
		const uint64 EntryID = OutEntry.GetEntryID();

		if (const FLinearColor* const CachedColor = PlottingEntryColors.Find(EntryID))
		{
			OutEntry.SetEntryColor(*CachedColor);
		}

		if (OutEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::None && PinnedSoundEntries.Contains(EntryID))
		{
			OutEntry.PinnedEntryType = FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry;
		}
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
