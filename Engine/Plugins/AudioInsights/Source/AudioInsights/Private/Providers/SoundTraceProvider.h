// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/SoundTraceMessages.h"
#include "UObject/NameTypes.h"

/**
 *	Trace provider for the "Sounds Dashboard"
 *	
 *	"Sounds Dashboard" displays active audio in the following tree structure:
 *	- Category											(i.e. The EntryType of the sound asset, e.g. SoundCue, MetaSound etc)
 *		- Active Sound									(i.e. The Active Sound instance playing)
 *			- SoundWave Instances (optional)			(i.e. If an entry type can have multiple SoundWaves playing simultaneously (e.g. SoundCue)
 *														, individual Soundwaves are shown here)
 */
namespace UE::Audio::Insights
{
	class FSoundTraceProvider
		: public TDeviceDataMapTraceProvider<ESoundDashboardEntryType, TSharedPtr<FSoundDashboardEntry>>
		, public TSharedFromThis<FSoundTraceProvider>
	{
	public:
		FSoundTraceProvider();
		virtual ~FSoundTraceProvider();

		static FName GetName_Static();

		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		virtual bool ProcessMessages() override;
		virtual bool ProcessManuallyUpdatedEntries() override;

		void SetDashboardTimeoutTime(const double InTimeOutTime) { DashboardTimeoutTime = InTimeOutTime; }

		bool HasCachedPinnedEntries() const { return !PinnedSoundEntries.IsEmpty(); }

		DECLARE_MULTICAST_DELEGATE(FOnProcessPlotData);
		FOnProcessPlotData OnProcessPlotData;

	private:
		void GetOrCreateActiveSoundEntry(const FSoundStartMessage& Msg, TSharedPtr<FSoundDashboardEntry>& OutReturnedSoundEntry);
		void CreateAndGetActiveSoundEntryIfNotFound(const FSoundStartMessage& Msg, TSharedPtr<FSoundDashboardEntry>& OutReturnedSoundEntry);
		void GetActiveSoundEntryFromIDs(const uint32 PlayOrderID, TSharedPtr<FSoundDashboardEntry>& OutSoundEntry);
		void RemoveActiveSoundEntry(TSharedPtr<FSoundDashboardEntry> OutEntry);

		void UpdateAggregateActiveSoundData();
		void CollectAggregateData(FSoundDashboardEntry& ActiveSoundEntry);

		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;
		virtual void OnTimeControlMethodReset() override;

		void CollectParamsForTimestamp(const FAudioInsightsCacheManager& CacheManager, const double TimeMarker);

		void CacheIsPlottingFlagRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);
		void CachePinnedFlagRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);
		void ResetDataBuffersRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);

		void RestoreCachedStateForEntry(FSoundDashboardEntry& OutEntry);

		FSoundMessages TraceMessages;

		struct SoundEntryKeys
		{
			const ESoundDashboardEntryType EntryType;
			const ::Audio::FDeviceId DeviceId;
		};

		struct SoundMessageIDs
		{
			const ::Audio::FDeviceId DeviceId;
			const uint32 ActiveSoundPlayOrder;
		};

		TMap<uint32, SoundEntryKeys> ActiveSoundToEntryKeysMap;
		TArray<SoundMessageIDs> EntriesTimingOut;
		TSet<uint32> SoundsStoppedBeforeStart;
		TSet<uint64> PlottingSoundEntries;
		TMap<uint64, FLinearColor> PlottingEntryColors;
		TSet<uint64> PinnedSoundEntries;

		double DashboardTimeoutTime = 3.0;

#if WITH_EDITOR
		FDelegateHandle OnReadSettingsHandle;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights
