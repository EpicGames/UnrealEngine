// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/SubmixTraceProvider.h"

#include "AudioInsightsModule.h"
#include "Async/ParallelFor.h"
#include "Cache/AudioInsightsCacheManager.h"

#if WITH_EDITOR
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#else
#include "Common/PagedArray.h"
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace FSubmixTraceProviderPrivate
	{
#if WITH_EDITOR
		TSharedPtr<FSubmixDashboardEntry> CreateEntryFromAsset(const FString& InAssetPath)
		{
			TSharedPtr<FSubmixDashboardEntry> Entry = MakeShared<FSubmixDashboardEntry>();

			Entry->Timestamp = FPlatformTime::Seconds() - GStartTime;
			Entry->SubmixId = GetTypeHash(InAssetPath);

			Entry->Name = InAssetPath;

			if (const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
				const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();

				if (const ::Audio::FMixerDevice* MixerDevice = static_cast<const ::Audio::FMixerDevice*>(AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId)))
				{
					Entry->AudioMeterInfo->NumChannels = MixerDevice->GetNumDeviceChannels();
				}
			}

			return Entry;
		}
#endif // WITH_EDITOR
	}

	FSubmixTraceProvider::FSubmixTraceProvider()
		: TDeviceDataMapTraceProvider<uint32, TSharedPtr<FSubmixDashboardEntry>>(GetName_Static())
	{
#if WITH_EDITOR
		SubmixAssetProvider.OnAssetAdded.BindRaw(this, &FSubmixTraceProvider::HandleOnSubmixAssetAdded);
		SubmixAssetProvider.OnAssetRemoved.BindRaw(this, &FSubmixTraceProvider::HandleOnSubmixAssetRemoved);
		SubmixAssetProvider.OnAssetListUpdated.BindRaw(this, &FSubmixTraceProvider::HandleOnSubmixAssetListUpdated);
#endif // WITH_EDITOR
	}

	FSubmixTraceProvider::~FSubmixTraceProvider()
	{
#if WITH_EDITOR
		SubmixAssetProvider.OnAssetAdded.Unbind();
		SubmixAssetProvider.OnAssetRemoved.Unbind();
		SubmixAssetProvider.OnAssetListUpdated.Unbind();
#endif // WITH_EDITOR
	}
	
	FName FSubmixTraceProvider::GetName_Static()
	{
		static const FLazyName SubmixTraceProviderName = "SubmixProvider";
		return SubmixTraceProviderName;
	}

	UE::Trace::IAnalyzer* FSubmixTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FSubmixTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FSubmixTraceAnalyzer(TSharedRef<FSubmixTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

				Builder.RouteEvent(RouteId_Loaded,                  "Audio", "SubmixLoaded");
				Builder.RouteEvent(RouteId_AlivePing,               "Audio", "SubmixAlivePing");
				Builder.RouteEvent(RouteId_HasActivity,             "Audio", "SubmixHasActivity");
				Builder.RouteEvent(RouteId_EnvelopeFollowerEnabled, "Audio", "SubmixEnvelopeFollowerEnabled");
				Builder.RouteEvent(RouteId_EnvelopeValues,          "Audio", "SubmixEnvelopeValues");
				Builder.RouteEvent(RouteId_Unloaded,                "Audio", "SubmixUnloaded");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FSubmixTraceAnalyzer"));

				FSubmixMessages& Messages = GetProvider<FSubmixTraceProvider>().TraceMessages;

				switch (RouteId)
				{
					case RouteId_Loaded:
					{
						CacheMessage<FSubmixLoadedMessage>(Context, Messages.LoadedMessages);
						break;
					}

					case RouteId_AlivePing:
					{
						CacheMessage<FSubmixAlivePingMessage>(Context, Messages.AlivePingMessages);
						break;
					}

					case RouteId_HasActivity:
					{
						CacheMessage<FSubmixHasActivityMessage>(Context, Messages.HasActivityMessages);
						break;
					}

					case RouteId_EnvelopeFollowerEnabled:
					{
						CacheMessage<FSubmixEnvelopeFollowerEnabledMessage>(Context, Messages.EnvelopeFollowerEnabledMessages);
						break;
					}

					case RouteId_EnvelopeValues:
					{
						CacheMessage<FSubmixEnvelopeValuesMessage>(Context, Messages.EnvelopeValuesMessages);
						break;
					}

					case RouteId_Unloaded:
					{
						CacheMessage<FSubmixUnloadedMessage>(Context, Messages.UnloadedMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_Loaded,
				RouteId_AlivePing,
				RouteId_HasActivity,
				RouteId_EnvelopeFollowerEnabled,
				RouteId_EnvelopeValues,
				RouteId_Unloaded
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FSubmixTraceAnalyzer(AsShared(), InSession);
	}

	void FSubmixTraceProvider::OnTraceChannelsEnabled()
	{
		DeviceDataMap.Empty();
		FTraceProviderBase::OnTraceChannelsEnabled();

#if WITH_EDITOR
		SubmixAssetProvider.RequestEntriesUpdate();
		OnSubmixListUpdated.ExecuteIfBound();
#endif // WITH_EDITOR
	}

	bool FSubmixTraceProvider::ProcessMessages()
	{
		// Helper lambdas
		auto CreateEntry = [this](const FSubmixMessageBase& Msg)
		{
			TSharedPtr<FSubmixDashboardEntry>* ToReturn = nullptr;
			
			UpdateDeviceEntry(Msg.DeviceId, Msg.SubmixId, [&ToReturn, &Msg](TSharedPtr<FSubmixDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FSubmixDashboardEntry>();
					Entry->SubmixId = Msg.SubmixId;
				}

				Entry->Timestamp = Msg.Timestamp;

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		auto GetEntry = [this](const FSubmixMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, Msg.SubmixId);
		};

		// Process messages
		ProcessMessageQueue<FSubmixLoadedMessage>(TraceMessages.LoadedMessages, CreateEntry,
		[this](const FSubmixLoadedMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.Name = Msg.Name;
				EntryRef.AudioMeterInfo->NumChannels = Msg.NumChannels;
				EntryRef.bIsMainSubmix = Msg.bIsMainSubmix;

				OnSubmixLoaded.ExecuteIfBound(Msg.SubmixId);
			}
		});

		ProcessMessageQueue<FSubmixAlivePingMessage>(TraceMessages.AlivePingMessages, CreateEntry,
		[this](const FSubmixAlivePingMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.Name = Msg.Name;
				EntryRef.AudioMeterInfo->NumChannels = Msg.NumChannels;
				EntryRef.bIsMainSubmix = Msg.bIsMainSubmix;
			}
		});

		ProcessMessageQueue<FSubmixHasActivityMessage>(TraceMessages.HasActivityMessages, GetEntry,
		[this](const FSubmixHasActivityMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.bHasActivity = Msg.bHasActivity;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FSubmixEnvelopeFollowerEnabledMessage>(TraceMessages.EnvelopeFollowerEnabledMessages, GetEntry,
		[this](const FSubmixEnvelopeFollowerEnabledMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.bEnvelopeFollowerEnabled = Msg.bEnvelopeFollowerEnabled;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FSubmixEnvelopeValuesMessage>(TraceMessages.EnvelopeValuesMessages, GetEntry,
		[this](const FSubmixEnvelopeValuesMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();

				EntryRef.AudioMeterInfo->EnvelopeValues = Msg.EnvelopeValues;
				EntryRef.Timestamp = Msg.Timestamp;
			}
		});

		ProcessMessageQueue<FSubmixUnloadedMessage>(TraceMessages.UnloadedMessages, GetEntry,
		[this](const FSubmixUnloadedMessage& Msg, TSharedPtr<FSubmixDashboardEntry>* OutEntry)
		{
			if (OutEntry && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
#if WITH_EDITOR
				// Turn off submix entry activity, do not remove it since we are in Editor mode.
				// We keep in the dashboard all registered submix assets from Content Browser
				FSubmixDashboardEntry& EntryRef = *OutEntry->Get();

				EntryRef.bHasActivity = false;

				TArray<float>& EnvelopeValues = EntryRef.AudioMeterInfo->EnvelopeValues;
				FMemory::Memzero(EnvelopeValues.GetData(), EnvelopeValues.Num() * sizeof(float));
#else
				OnSubmixRemoved.ExecuteIfBound(Msg.SubmixId);
				RemoveDeviceEntry(Msg.DeviceId, Msg.SubmixId);
#endif // WITH_EDITOR
			}
		});

#if WITH_EDITOR
		if (bAssetsUpdated)
		{
			OnSubmixListUpdated.ExecuteIfBound();
			bAssetsUpdated = false;
		}
#endif // WITH_EDITOR

		return true;
	}

#if WITH_EDITOR
	void FSubmixTraceProvider::RequestEntriesUpdate()
	{
		SubmixAssetProvider.RequestEntriesUpdate();
	}

	void FSubmixTraceProvider::HandleOnSubmixAssetAdded(const FString& InAssetPath)
	{
		using namespace FSubmixTraceProviderPrivate;

		if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			const TArray<FAudioDevice*> AudioDevices = DeviceManager->GetAudioDevices();

			for (const FAudioDevice* AudioDevice : AudioDevices)
			{
				if (AudioDevice == nullptr)
				{
					continue;
				}

				const uint32 SubmixId = GetTypeHash(InAssetPath);

				const TSharedPtr<FSubmixDashboardEntry>* FoundSubmixDashboardEntry = FindDeviceEntry(AudioDevice->DeviceID, SubmixId);

				if (FoundSubmixDashboardEntry == nullptr)
				{
					UpdateDeviceEntry(AudioDevice->DeviceID, SubmixId, [this, &InAssetPath](TSharedPtr<FSubmixDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = CreateEntryFromAsset(InAssetPath);
						}
					});
				}

				++LastUpdateId;
				OnSubmixAdded.ExecuteIfBound(SubmixId);
				bAssetsUpdated = true;
			}
		}
	}

	void FSubmixTraceProvider::HandleOnSubmixAssetRemoved(const FString& InAssetPath)
	{
		if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			const TArray<FAudioDevice*> AudioDevices = DeviceManager->GetAudioDevices();

			for (const FAudioDevice* AudioDevice : AudioDevices)
			{
				if (AudioDevice)
				{
					const uint32 SubmixId = GetTypeHash(InAssetPath);

					RemoveDeviceEntry(AudioDevice->DeviceID, SubmixId);

					++LastUpdateId;
					OnSubmixRemoved.ExecuteIfBound(SubmixId);
					bAssetsUpdated = true;
				}
			}
		}
	}

	void FSubmixTraceProvider::HandleOnSubmixAssetListUpdated(const TArray<FString>& InAssetPaths)
	{
		using namespace FSubmixTraceProviderPrivate;

		const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get();

		if (DeviceManager == nullptr)
		{
			return;
		}

		for (const FAudioDevice* AudioDevice : DeviceManager->GetAudioDevices())
		{
			if (AudioDevice == nullptr)
			{
				continue;
			}

			for (const FString& AssetPath : InAssetPaths)
			{
				const uint32 SubmixId = GetTypeHash(AssetPath);

				const TSharedPtr<FSubmixDashboardEntry>* FoundSubmixDashboardEntry = FindDeviceEntry(AudioDevice->DeviceID, SubmixId);

				if (FoundSubmixDashboardEntry == nullptr)
				{
					UpdateDeviceEntry(AudioDevice->DeviceID, SubmixId, [&AssetPath](TSharedPtr<FSubmixDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = CreateEntryFromAsset(AssetPath);
						}
					});
				}
			}
		}

		++LastUpdateId;
		OnSubmixListUpdated.ExecuteIfBound();
		bAssetsUpdated = true;
	}
#endif // WITH_EDITOR

	void FSubmixTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		using namespace FSubmixTraceProviderPrivate;

		DeviceDataMap.Empty();

		// Collect all the submix loaded messages registered until this point in time 
		auto ProcessSubmixLoadedMessage = [this](const FSubmixLoadedMessage& Msg)
		{
			UpdateDeviceEntry(Msg.DeviceId, Msg.SubmixId, [&Msg](TSharedPtr<FSubmixDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FSubmixDashboardEntry>();
					Entry->SubmixId = Msg.SubmixId;
				}

				Entry->Timestamp = Msg.Timestamp;

				Entry->Name = *Msg.Name;
				Entry->AudioMeterInfo->NumChannels = Msg.NumChannels;
				Entry->bIsMainSubmix = Msg.bIsMainSubmix;
			});
		};

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		CacheManager.IterateOverRange<FSubmixLoadedMessage>(SubmixMessageNames::Loaded, RangeCheckStart, TimeMarker, [&ProcessSubmixLoadedMessage](const FSubmixLoadedMessage& SubmixLoadedCachedMessage)
		{
			ProcessSubmixLoadedMessage(SubmixLoadedCachedMessage);
		});


		// Collect all the submix alive ping messages registered until this point in time 
		auto ProcessSubmixAlivePingMessage = [this](const FSubmixAlivePingMessage& Msg)
		{
			UpdateDeviceEntry(Msg.DeviceId, Msg.SubmixId, [&Msg](TSharedPtr<FSubmixDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FSubmixDashboardEntry>();
					Entry->SubmixId = Msg.SubmixId;
				}

				Entry->Timestamp = Msg.Timestamp;

				Entry->Name = *Msg.Name;
				Entry->AudioMeterInfo->NumChannels = Msg.NumChannels;
				Entry->bIsMainSubmix = Msg.bIsMainSubmix;
			});
		};

		CacheManager.IterateOverRange<FSubmixAlivePingMessage>(SubmixMessageNames::IsAlivePing, RangeCheckStart, TimeMarker, [&ProcessSubmixAlivePingMessage](const FSubmixAlivePingMessage& SubmixAlivePingCachedMessage)
		{
			ProcessSubmixAlivePingMessage(SubmixAlivePingCachedMessage);
		});

		// Selectively remove loaded messages collected in the step above by knowing which submixes were unloaded.
		// With this we will know what are the available submixes at this point in time.
		auto ProcessSubmixUnloadedMessage = [this](const FSubmixUnloadedMessage& Msg)
		{
			auto* OutEntry = FindDeviceEntry(Msg.DeviceId, Msg.SubmixId);

			if (OutEntry && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
				RemoveDeviceEntry(Msg.DeviceId, Msg.SubmixId);
			}
		};

		CacheManager.IterateOverRange<FSubmixUnloadedMessage>(SubmixMessageNames::Unloaded, RangeCheckStart, TimeMarker, [&ProcessSubmixUnloadedMessage](const FSubmixUnloadedMessage& SubmixUnloadedCachedMessage)
		{
			ProcessSubmixUnloadedMessage(SubmixUnloadedCachedMessage);
		});

		const FDeviceData* DeviceData = FindFilteredDeviceData();
		if (DeviceData)
		{
			// Collect has activity messages from submixes (based on active submixes SubmixId)
			struct CachedEntryInfo
			{
				TOptional<FSubmixHasActivityMessage> HasActivityMessage;
				TOptional<FSubmixEnvelopeFollowerEnabledMessage> EnvelopeFollowerEnabledMessage;
				TOptional<FSubmixEnvelopeValuesMessage> EnvelopeValuesMessage;
			};

			TArray<uint32> SubmixIdArray;
			(*DeviceData).GenerateKeyArray(SubmixIdArray);

			TArray<CachedEntryInfo> CachedEntryInfos;
			CachedEntryInfos.SetNum(SubmixIdArray.Num());

			ParallelFor(SubmixIdArray.Num(), [&SubmixIdArray, &CachedEntryInfos, &CacheManager, TimeMarker, this](const int32 Index)
			{
				const uint32 SubmixId = SubmixIdArray[Index];

				// HasActivity
				const FSubmixHasActivityMessage* FoundHasActivityCachedMessage = CacheManager.FindClosestMessage<FSubmixHasActivityMessage>(SubmixMessageNames::HasActivity, TimeMarker, SubmixId);
				if (FoundHasActivityCachedMessage)
				{
					CachedEntryInfos[Index].HasActivityMessage = *FoundHasActivityCachedMessage;
				}

				// EnvelopeFollowerEnabled
				const FSubmixEnvelopeFollowerEnabledMessage* FoundEnvelopeFollowerEnabledCachedMessage = CacheManager.FindClosestMessage<FSubmixEnvelopeFollowerEnabledMessage>(SubmixMessageNames::EnvelopeFollowerEnabled, TimeMarker, SubmixId);
				if (FoundEnvelopeFollowerEnabledCachedMessage)
				{
					CachedEntryInfos[Index].EnvelopeFollowerEnabledMessage = *FoundEnvelopeFollowerEnabledCachedMessage;
				}

				// EnvelopeValues
				const FSubmixEnvelopeValuesMessage* FoundEnvelopeValuesCachedMessage = CacheManager.FindClosestMessage<FSubmixEnvelopeValuesMessage>(SubmixMessageNames::EnvelopeValues, TimeMarker, SubmixId);
				if (FoundEnvelopeValuesCachedMessage)
				{
					CachedEntryInfos[Index].EnvelopeValuesMessage = *FoundEnvelopeValuesCachedMessage;
				}
			});

			// Update the device entries with the collected info
			for (const CachedEntryInfo& CachedEntryInfo : CachedEntryInfos)
			{
				if (CachedEntryInfo.HasActivityMessage.IsSet())
				{
					const FSubmixHasActivityMessage& HasActivityMessage = CachedEntryInfo.HasActivityMessage.GetValue();

					UpdateDeviceEntry(HasActivityMessage.DeviceId, HasActivityMessage.SubmixId, [&HasActivityMessage](TSharedPtr<FSubmixDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FSubmixDashboardEntry>();
							Entry->SubmixId = HasActivityMessage.SubmixId;
						}

						Entry->Timestamp = HasActivityMessage.Timestamp;

						Entry->bHasActivity = HasActivityMessage.bHasActivity;
					});
				}

				if (CachedEntryInfo.EnvelopeFollowerEnabledMessage.IsSet())
				{
					const FSubmixEnvelopeFollowerEnabledMessage& EnvelopeFollowerEnabledMessage = CachedEntryInfo.EnvelopeFollowerEnabledMessage.GetValue();

					UpdateDeviceEntry(EnvelopeFollowerEnabledMessage.DeviceId, EnvelopeFollowerEnabledMessage.SubmixId, [&EnvelopeFollowerEnabledMessage](TSharedPtr<FSubmixDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FSubmixDashboardEntry>();
							Entry->SubmixId = EnvelopeFollowerEnabledMessage.SubmixId;
						}

						Entry->Timestamp = EnvelopeFollowerEnabledMessage.Timestamp;

						Entry->bEnvelopeFollowerEnabled = EnvelopeFollowerEnabledMessage.bEnvelopeFollowerEnabled;
					});
				}

				if (CachedEntryInfo.EnvelopeValuesMessage.IsSet())
				{
					const FSubmixEnvelopeValuesMessage& EnvelopeValuesMessage = CachedEntryInfo.EnvelopeValuesMessage.GetValue();

					UpdateDeviceEntry(EnvelopeValuesMessage.DeviceId, EnvelopeValuesMessage.SubmixId, [&EnvelopeValuesMessage](TSharedPtr<FSubmixDashboardEntry>& Entry)
					{
						if (!Entry.IsValid())
						{
							Entry = MakeShared<FSubmixDashboardEntry>();
							Entry->SubmixId = EnvelopeValuesMessage.SubmixId;
						}

						Entry->Timestamp = EnvelopeValuesMessage.Timestamp;

						Entry->AudioMeterInfo->NumChannels = EnvelopeValuesMessage.EnvelopeValues.Num();
						Entry->AudioMeterInfo->EnvelopeValues = EnvelopeValuesMessage.EnvelopeValues;
					});
				}
			}
		}

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);

		OnTimeMarkerUpdated.ExecuteIfBound();
	}

#if WITH_EDITOR
	void FSubmixTraceProvider::OnTimeControlMethodReset()
	{
		SubmixAssetProvider.RequestEntriesUpdate();
		OnSubmixListUpdated.ExecuteIfBound();
	}
#endif // WITH_EDITOR
} // namespace UE::Audio::Insights
