// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/SubmixTraceMessages.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "Delegates/Delegate.h"
#include "Providers/AssetProvider.h"
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	class FSubmixTraceProvider : public TDeviceDataMapTraceProvider<uint32, TSharedPtr<FSubmixDashboardEntry>>, public TSharedFromThis<FSubmixTraceProvider>
	{
	public:
		FSubmixTraceProvider();
		virtual ~FSubmixTraceProvider();

		static FName GetName_Static();
		
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;
		void RequestEntriesUpdate();

		DECLARE_DELEGATE_OneParam(FOnSubmixChanged, const uint32 /*SubmixId*/);
		FOnSubmixChanged OnSubmixAdded;
		FOnSubmixChanged OnSubmixRemoved;
		FOnSubmixChanged OnSubmixLoaded;

		DECLARE_DELEGATE(FOnSubmixListUpdated);
		FOnSubmixListUpdated OnSubmixListUpdated;

		DECLARE_DELEGATE(FOnTimeMarkerUpdated);
		FOnTimeMarkerUpdated OnTimeMarkerUpdated;

	protected:
		virtual void OnTraceChannelsEnabled() override;

	private:
		virtual bool ProcessMessages() override;

#if WITH_EDITOR
		virtual void OnTimeControlMethodReset() override;

		void HandleOnSubmixAssetAdded(const FString& InAssetPath);
		void HandleOnSubmixAssetRemoved(const FString& InAssetPath);
		void HandleOnSubmixAssetListUpdated(const TArray<FString>& InAssetPaths);
#endif // WITH_EDITOR

		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;

		FSubmixMessages TraceMessages;

#if WITH_EDITOR
		TAssetProvider<USoundSubmix> SubmixAssetProvider;
		bool bAssetsUpdated = false;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights
