// Copyright Epic Games, Inc. All Rights Reserved.
#include "Providers/VirtualLoopTraceProvider.h"

#include "Async/ParallelFor.h"
#include "AudioInsightsModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

#if !WITH_EDITOR
#include "Common/PagedArray.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	FName FVirtualLoopTraceProvider::GetName_Static()
	{
		return "AudioVirtualLoopProvider";
	}

	void FVirtualLoopTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		// Helper lambdas
		auto ProcessVirtualizeMessage = [this](const FVirtualLoopVirtualizeMessage& VirtualizeCachedMessage)
		{
			UpdateDeviceEntry(VirtualizeCachedMessage.DeviceId, VirtualizeCachedMessage.PlayOrder, [&VirtualizeCachedMessage](TSharedPtr<FVirtualLoopDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FVirtualLoopDashboardEntry>();
					Entry->DeviceId  = VirtualizeCachedMessage.DeviceId;
					Entry->PlayOrder = VirtualizeCachedMessage.PlayOrder;
				}
				
				Entry->Timestamp = VirtualizeCachedMessage.Timestamp;

				Entry->Name        = *VirtualizeCachedMessage.Name;
				Entry->ComponentId = VirtualizeCachedMessage.ComponentId;
			});
		};

		auto ProcessStopOrRealizeMessage = [this](const FVirtualLoopStopOrRealizeMessage& StopOrRealizeCachedMessage)
		{
			auto* OutEntry = FindDeviceEntry(StopOrRealizeCachedMessage.DeviceId, StopOrRealizeCachedMessage.PlayOrder);
			
			if (OutEntry && (*OutEntry)->Timestamp < StopOrRealizeCachedMessage.Timestamp)
			{
				RemoveDeviceEntry(StopOrRealizeCachedMessage.DeviceId, StopOrRealizeCachedMessage.PlayOrder);
			}
		};

		// Process cached messages
		DeviceDataMap.Empty();

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();

		// IsAlive pings are sent every second - therefore we should only need to check for alive nodes from the last second
		// Double this to 2s to ensure we do catch all alive sounds
		constexpr double MaxLookbackTime = 2.0;
		const double RangeCheckStart = FMath::Max(TimeMarker - MaxLookbackTime, 0.0);

		// Virtualize cached messages
		CacheManager.IterateOverRange<FVirtualLoopVirtualizeMessage>(VirtualLoopMessageNames::Virtualize, RangeCheckStart, TimeMarker, [&ProcessVirtualizeMessage](const FVirtualLoopVirtualizeMessage& VirtualizeCachedMessage)
		{
			ProcessVirtualizeMessage(VirtualizeCachedMessage);
		});

		// Stop or realize cached messages
		CacheManager.IterateOverRange<FVirtualLoopStopOrRealizeMessage>(VirtualLoopMessageNames::StopOrRealize, RangeCheckStart, TimeMarker, [&ProcessStopOrRealizeMessage](const FVirtualLoopStopOrRealizeMessage& StopOrRealizeCachedMessage)
		{
			ProcessStopOrRealizeMessage(StopOrRealizeCachedMessage);
		});

		CollectParamsForTimestamp(TimeMarker);

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);
	}

	void FVirtualLoopTraceProvider::CollectParamsForTimestamp(const double InTimeMarker)
	{
		const FDeviceData* DeviceData = FindFilteredDeviceData();
		if (DeviceData == nullptr)
		{
			return;
		}

		// Collect update messages from virtualized sounds (based on active sounds's PlayOrder)
		struct CachedEntryInfo
		{
			TOptional<FVirtualLoopUpdateMessage> UpdateMessage;
		};

		TArray<uint32> PlayOrderArray;
		DeviceData->GenerateKeyArray(PlayOrderArray);

		TArray<CachedEntryInfo> CachedEntryInfos;
		CachedEntryInfos.SetNum(PlayOrderArray.Num());

		const FAudioInsightsCacheManager& CacheManager = FAudioInsightsModule::GetChecked().GetCacheManager();

		ParallelFor(PlayOrderArray.Num(), [&PlayOrderArray, &CachedEntryInfos, &CacheManager, InTimeMarker, this](const int32 Index)
		{
			const uint32 PlayOrder = PlayOrderArray[Index];

			const FVirtualLoopUpdateMessage* FoundUpdateCachedMessage = CacheManager.FindClosestMessage<FVirtualLoopUpdateMessage>(VirtualLoopMessageNames::Update, InTimeMarker, PlayOrder);
			if (FoundUpdateCachedMessage)
			{
				CachedEntryInfos[Index].UpdateMessage = *FoundUpdateCachedMessage;
			}
		});

		// Update the device entries with the collected info
		for (const CachedEntryInfo& CachedEntryInfo : CachedEntryInfos)
		{
			if (CachedEntryInfo.UpdateMessage.IsSet())
			{
				const FVirtualLoopUpdateMessage& VirtualLoopUpdateMessage = CachedEntryInfo.UpdateMessage.GetValue();

				UpdateDeviceEntry(VirtualLoopUpdateMessage.DeviceId, VirtualLoopUpdateMessage.PlayOrder, [&VirtualLoopUpdateMessage](TSharedPtr<FVirtualLoopDashboardEntry>& Entry)
				{
					if (ensure(Entry.IsValid()))
					{
						Entry->Timestamp = VirtualLoopUpdateMessage.Timestamp;

						Entry->PlaybackTime    = VirtualLoopUpdateMessage.PlaybackTime;
						Entry->TimeVirtualized = VirtualLoopUpdateMessage.TimeVirtualized;
						Entry->UpdateInterval  = VirtualLoopUpdateMessage.UpdateInterval;
						Entry->Location        = FVector{ VirtualLoopUpdateMessage.LocationX, VirtualLoopUpdateMessage.LocationY, VirtualLoopUpdateMessage.LocationZ };
						Entry->Rotator         = FRotator{ VirtualLoopUpdateMessage.RotatorPitch, VirtualLoopUpdateMessage.RotatorYaw, VirtualLoopUpdateMessage.RotatorRoll };
					}
				});
			}
		}
	}

	bool FVirtualLoopTraceProvider::ProcessMessages()
	{
		auto RemoveEntryFunc = [this](const FVirtualLoopStopOrRealizeMessage& Msg, TSharedPtr<FVirtualLoopDashboardEntry>* OutEntry)
		{
			if (OutEntry && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
				RemoveDeviceEntry(Msg.DeviceId, Msg.PlayOrder);
			}
		};

		auto GetEntryFunc = [this](const FVirtualLoopMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, Msg.PlayOrder);
		};

		auto BumpEntryFunc = [this](const FVirtualLoopMessageBase& Msg)
		{
			TSharedPtr<FVirtualLoopDashboardEntry>* ToReturn = nullptr;
			UpdateDeviceEntry(Msg.DeviceId, Msg.PlayOrder, [&ToReturn, &Msg](TSharedPtr<FVirtualLoopDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FVirtualLoopDashboardEntry>();
					Entry->DeviceId  = Msg.DeviceId;
					Entry->PlayOrder = Msg.PlayOrder;
				}
				Entry->Timestamp = Msg.Timestamp;

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		ProcessMessageQueue<FVirtualLoopVirtualizeMessage>(TraceMessages.VirtualizeMessages,
		BumpEntryFunc,
		[this](const FVirtualLoopVirtualizeMessage& Msg, TSharedPtr<FVirtualLoopDashboardEntry>* OutEntry)
		{
			FVirtualLoopDashboardEntry& EntryRef = *OutEntry->Get();
			EntryRef.Name = Msg.Name;
			EntryRef.ComponentId = Msg.ComponentId;
		});

		ProcessMessageQueue<FVirtualLoopUpdateMessage>(TraceMessages.UpdateMessages,
		GetEntryFunc,
		[this](const FVirtualLoopUpdateMessage& Msg, TSharedPtr<FVirtualLoopDashboardEntry>* OutEntry)
		{
			if (OutEntry)
			{
				FVirtualLoopDashboardEntry& EntryRef = *OutEntry->Get();

				EntryRef.Timestamp = Msg.Timestamp;

				EntryRef.PlaybackTime = Msg.PlaybackTime;
				EntryRef.TimeVirtualized = Msg.TimeVirtualized;
				EntryRef.UpdateInterval = Msg.UpdateInterval;
				EntryRef.Location = FVector{ Msg.LocationX, Msg.LocationY, Msg.LocationZ };
				EntryRef.Rotator = FRotator{ Msg.RotatorPitch, Msg.RotatorYaw, Msg.RotatorRoll };
			}
		});

		ProcessMessageQueue<FVirtualLoopStopOrRealizeMessage>(TraceMessages.StopOrRealizeMessages, GetEntryFunc, RemoveEntryFunc);

		return true;
	}

	UE::Trace::IAnalyzer* FVirtualLoopTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FVirtualLoopTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FVirtualLoopTraceAnalyzer(TSharedRef<FVirtualLoopTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
				Builder.RouteEvent(RouteId_Stop, "Audio", "VirtualLoopStopOrRealize");
				Builder.RouteEvent(RouteId_Update, "Audio", "VirtualLoopUpdate");
				Builder.RouteEvent(RouteId_Virtualize, "Audio", "VirtualLoopVirtualize");
				Builder.RouteEvent(RouteId_IsStillVirtualizedPing, "Audio", "VirtualLoopIsVirtualizedPing");
			}

			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FVirtualLoopTraceAnalyzer"));

				FVirtualLoopMessages& Messages = GetProvider<FVirtualLoopTraceProvider>().TraceMessages;
				switch (RouteId)
				{
					case RouteId_Virtualize:
					case RouteId_IsStillVirtualizedPing:
					{
						CacheMessage<FVirtualLoopVirtualizeMessage>(Context, Messages.VirtualizeMessages);
						break;
					}

					case RouteId_Stop:
					{
						CacheMessage<FVirtualLoopStopOrRealizeMessage>(Context, Messages.StopOrRealizeMessages);
						break;
					}

					case RouteId_Update:
					{
						CacheMessage<FVirtualLoopUpdateMessage>(Context, Messages.UpdateMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_Virtualize,
				RouteId_Update,
				RouteId_Stop,
				RouteId_IsStillVirtualizedPing
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FVirtualLoopTraceAnalyzer(AsShared(), InSession);
	}
} // namespace UE::Audio::Insights
