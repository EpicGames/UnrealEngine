// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/VirtualLoopTraceMessages.h"


namespace UE::Audio::Insights
{
	class FVirtualLoopTraceProvider
		: public TDeviceDataMapTraceProvider<uint32, TSharedPtr<FVirtualLoopDashboardEntry>>
		, public TSharedFromThis<FVirtualLoopTraceProvider>
	{
	public:
		FVirtualLoopTraceProvider()
			: TDeviceDataMapTraceProvider<uint32, TSharedPtr<FVirtualLoopDashboardEntry>>(GetName_Static())
		{
			
		}

		virtual ~FVirtualLoopTraceProvider() = default;
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		static FName GetName_Static();

	private:
		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;
		void CollectParamsForTimestamp(const double InTimeMarker);

		virtual bool ProcessMessages() override;

		FVirtualLoopMessages TraceMessages;
	};
} // namespace UE::Audio::Insights
