// Copyright Epic Games, Inc. All Rights Reserved.
#include "Settings/AudioEventLogSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioEventLogSettings)

#if WITH_EDITOR

FAudioEventLogSettings::FOnReadEventLogSettings FAudioEventLogSettings::OnReadSettings;
FAudioEventLogSettings::FOnWriteEventLogSettings FAudioEventLogSettings::OnWriteSettings;
FAudioEventLogSettings::FOnRequestReadEventLogSettings FAudioEventLogSettings::OnRequestReadSettings;
FAudioEventLogSettings::FOnRequestWriteEventLogSettings FAudioEventLogSettings::OnRequestWriteSettings;

#endif // WITH_EDITOR
