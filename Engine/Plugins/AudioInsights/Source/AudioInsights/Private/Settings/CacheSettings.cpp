// Copyright Epic Games, Inc. All Rights Reserved.
#include "Settings/CacheSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheSettings)

FCacheSettings::FOnCacheSizeSettingsChanged FCacheSettings::OnCacheSizeSettingsChanged;
