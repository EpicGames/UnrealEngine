// Copyright Epic Games, Inc. All Rights Reserved.
#include "Settings/SignalFlowSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SignalFlowSettings)

#if WITH_EDITOR

FSignalFlowSettings::FOnReadSignalFlowSettings FSignalFlowSettings::OnReadSettings;
FSignalFlowSettings::FOnWriteSignalFlowSettings FSignalFlowSettings::OnWriteSettings;
FSignalFlowSettings::FOnRequestReadSignalFlowSettings FSignalFlowSettings::OnRequestReadSettings;
FSignalFlowSettings::FOnRequestWriteSignalFlowSettings FSignalFlowSettings::OnRequestWriteSettings;

#endif // WITH_EDITOR

bool FSignalFlowNodeDetailFilterSettings::GetParameterIsVisible(const UE::Audio::Insights::ESignalFlowNodeDetailParam Param) const
{
	using namespace UE::Audio::Insights;

	switch (Param)
	{
		case ESignalFlowNodeDetailParam::Amplitude:
			return bAmplitude;

		case ESignalFlowNodeDetailParam::Volume:
			return bVolume;

		case ESignalFlowNodeDetailParam::Pitch:
			return bPitch;

		case ESignalFlowNodeDetailParam::LPFFreq:
			return bLPFFreq;

		case ESignalFlowNodeDetailParam::HPFFreq:
			return bHPFFreq;

		case ESignalFlowNodeDetailParam::Priority:
			return bPriority;

		case ESignalFlowNodeDetailParam::Distance:
			return bDistance;

		case ESignalFlowNodeDetailParam::Attenuation:
			return bDistanceOcclusionAttenuation;

		case ESignalFlowNodeDetailParam::RelativeRenderCost:
			return bRelativeRenderCost;

		case ESignalFlowNodeDetailParam::AudioComponentName:
			return bAudioComponentName;

		case ESignalFlowNodeDetailParam::SendOutputVolume:
			return bSendOutputVolume;

		default:
			ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
			break;
	}

    return false;
}

int32 FSignalFlowNodeDetailFilterSettings::GetNumVisibleParams(const TSet<UE::Audio::Insights::ESignalFlowNodeDetailParam>& IgnoredParams /* = TSet<UE::Audio::Insights::ESignalFlowNodeDetailParam>()*/) const
{
	using namespace UE::Audio::Insights;

	int32 NumVisibleParams = 0;

	const int32 NumFilters = static_cast<int32>(ESignalFlowNodeDetailParam::MAX);
	for (uint8 FilterID = 0u; FilterID < NumFilters; ++FilterID)
	{
		const ESignalFlowNodeDetailParam ParamType = static_cast<ESignalFlowNodeDetailParam>(FilterID);
		if (!IgnoredParams.Contains(ParamType) && GetParameterIsVisible(ParamType))
		{
			NumVisibleParams++;
		}
	}

	return NumVisibleParams;
}

ECheckBoxState FSignalFlowNodeDetailFilterSettings::GetShowAllNodeDetailFiltersCheckboxState() const
{
	using namespace UE::Audio::Insights;

	const int32 NumVisibleParams = GetNumVisibleParams();

	if (NumVisibleParams == 0)
	{
		return ECheckBoxState::Unchecked;
	}

	const int32 NumFilters = static_cast<int32>(ESignalFlowNodeDetailParam::MAX);
	if (NumVisibleParams == NumFilters)
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

void FSignalFlowNodeDetailFilterSettings::ToggleEnableAllNodeDetailFilters()
{
	using namespace UE::Audio::Insights;

	const bool bAllFiltersAreEnabled = GetShowAllNodeDetailFiltersCheckboxState() == ECheckBoxState::Checked;

	const int32 NumFilters = static_cast<int32>(ESignalFlowNodeDetailParam::MAX);

	for (int32 FilterID = 0; FilterID < NumFilters; ++FilterID)
	{
		SetParameterVisibility(static_cast<ESignalFlowNodeDetailParam>(FilterID), !bAllFiltersAreEnabled);
	}
}

void FSignalFlowNodeDetailFilterSettings::SetParameterVisibility(const UE::Audio::Insights::ESignalFlowNodeDetailParam Param, const bool bIsVisible)
{
	using namespace UE::Audio::Insights;

	switch (Param)
	{
		case ESignalFlowNodeDetailParam::Amplitude:
			bAmplitude = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::Volume:
			bVolume = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::Pitch:
			bPitch = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::LPFFreq:
			bLPFFreq = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::HPFFreq:
			bHPFFreq = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::Priority:
			bPriority = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::Distance:
			bDistance = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::Attenuation:
			bDistanceOcclusionAttenuation = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::RelativeRenderCost:
			bRelativeRenderCost = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::AudioComponentName:
			bAudioComponentName = bIsVisible;
			break;

		case ESignalFlowNodeDetailParam::SendOutputVolume:
			bSendOutputVolume = bIsVisible;
			break;

		default:
			ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
			break;
	}
}