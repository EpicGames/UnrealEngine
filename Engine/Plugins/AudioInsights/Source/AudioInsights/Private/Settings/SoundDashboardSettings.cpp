// Copyright Epic Games, Inc. All Rights Reserved.
#include "Settings/SoundDashboardSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundDashboardSettings)

#if WITH_EDITOR

FSoundDashboardSettings::FOnReadSoundDashboardSettings FSoundDashboardSettings::OnReadSettings;
FSoundDashboardSettings::FOnWriteSoundDashboardSettings FSoundDashboardSettings::OnWriteSettings;
FSoundDashboardSettings::FOnRequestReadSoundDashboardSettings FSoundDashboardSettings::OnRequestReadSettings;
FSoundDashboardSettings::FOnRequestWriteSoundDashboardSettings FSoundDashboardSettings::OnRequestWriteSettings;

#endif // WITH_EDITOR
