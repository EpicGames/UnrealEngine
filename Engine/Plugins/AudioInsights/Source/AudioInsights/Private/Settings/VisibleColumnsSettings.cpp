// Copyright Epic Games, Inc. All Rights Reserved.
#include "Settings/VisibleColumnsSettings.h"

bool FVisibleColumnsSettings::GetIsVisible(const FName& ColumnId) const
{
	const FName PropertyName(TEXT("b") + ColumnId.ToString());
	if (const FProperty* Property = FindProperty(PropertyName))
	{
		if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			return BoolProperty->GetPropertyValue_InContainer(this);
		}
		else
		{
			ensureMsgf(false, TEXT("VisibleColumns setting for ColumnId %s found, but is incorrectly typed (should be a boolean)."), *ColumnId.ToString());
		}
	}
	else
	{
		ensureMsgf(false, TEXT("No VisibleColumns setting for ColumnId: %s"), *ColumnId.ToString());
	}
	
	return false;
}

void FVisibleColumnsSettings::SetIsVisible(const FName& ColumnId, const bool bIsVisible)
{
	const FName PropertyName(TEXT("b") + ColumnId.ToString());
	if (const FProperty* Property = FindProperty(PropertyName))
	{
		if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			BoolProperty->SetPropertyValue_InContainer(this, bIsVisible);
		}
		else
		{
			ensureMsgf(false, TEXT("VisibleColumns setting for ColumnId %s found, but is incorrectly typed (should be a boolean)."), *ColumnId.ToString());
		}
	}
	else
	{
		ensureMsgf(false, TEXT("No VisibleColumns setting for ColumnId: %s"), *ColumnId.ToString());
	}
}