// Copyright Epic Games, Inc. All Rights Reserved.
#include "SignalFlowEditorCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FSignalFlowEditorCommands::FSignalFlowEditorCommands()
		: TCommands<FSignalFlowEditorCommands>("SignalFlowEditorCommands", LOCTEXT("SignalFlowEditorCommands_ContextDescText", "Signal Flow Editor Commands"), NAME_None, UE::Audio::Insights::FSlateStyle::GetStyleName())
	{
	}

	void FSignalFlowEditorCommands::RegisterCommands()
	{
		UI_COMMAND(ToggleHorizontalView, "Horizontal Flow", "Toggles between horizontally/vertically orienting the Signal Flow graph.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::H, EModifierKey::Control));

		UI_COMMAND(JustifyEdge, "Edge", "Position nodes towards one edge of the graph.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::E, EModifierKey::Control));
		UI_COMMAND(JustifyCenter, "Center", "Position nodes around the center of the graph.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::C, EModifierKey::Control));

		UI_COMMAND(ShowNodeDetails, "Show Node Details", "Toggles visibility of node details.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::D, EModifierKey::Control));
		UI_COMMAND(PauseOnSelect, "Pause on Select", "Toggles whether selecting a node should pause processing trace data, freezing audio insights at the latest timestamp.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::P, EModifierKey::Control));
		UI_COMMAND(ToggleAnimateWires, "Animate Wires", "Toggles whether connection wire animations driven by amplitude envelope values are enabled in the Signal Flow graph.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
		UI_COMMAND(CenterView, "Center View", "Focus the graph view on the selected node.", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
		UI_COMMAND(ResetTimestampPause, "Reset Inspect Timestamp Signal Flow", "Resumes signal flow graph to showing the latest trace data if previously paused.", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	}

} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
