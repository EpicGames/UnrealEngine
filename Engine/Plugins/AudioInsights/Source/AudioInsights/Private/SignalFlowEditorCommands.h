// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

namespace UE::Audio::Insights
{
	class FSignalFlowEditorCommands : public TCommands<FSignalFlowEditorCommands>
	{
	public:
		FSignalFlowEditorCommands();

		virtual void RegisterCommands() override;

		// Toggles
		TSharedPtr<const FUICommandInfo> GetToggleHorizontalViewCommand() const { return ToggleHorizontalView; }
		TSharedPtr<const FUICommandInfo> GetJustifyEdgeCommand() const { return JustifyEdge; }
		TSharedPtr<const FUICommandInfo> GetJustifyCenterCommand() const { return JustifyCenter; }
		TSharedPtr<const FUICommandInfo> GetShowNodeDetailsCommand() const { return ShowNodeDetails; }
		TSharedPtr<const FUICommandInfo> GetPauseOnSelectCommand() const { return PauseOnSelect; }
		TSharedPtr<const FUICommandInfo> GetToggleAnimateWiresCommand() const { return ToggleAnimateWires; }

		// Actions
		TSharedPtr<const FUICommandInfo> GetCenterViewCommand() const { return CenterView; }
		TSharedPtr<const FUICommandInfo> GetResetTimestampPauseCommand() const { return ResetTimestampPause; }

	private:
		// Toggles
		TSharedPtr<FUICommandInfo> ToggleHorizontalView;

		TSharedPtr<FUICommandInfo> JustifyEdge;
		TSharedPtr<FUICommandInfo> JustifyCenter;

		TSharedPtr<FUICommandInfo> ShowNodeDetails;
		TSharedPtr<FUICommandInfo> PauseOnSelect;
		TSharedPtr<FUICommandInfo> ToggleAnimateWires;

		// Actions
		TSharedPtr<FUICommandInfo> CenterView;
		TSharedPtr<FUICommandInfo> ResetTimestampPause;
	};
} // namespace UE::Audio::Insights
