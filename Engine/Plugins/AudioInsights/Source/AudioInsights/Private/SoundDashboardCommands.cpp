// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundDashboardCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioInsights.SoundDashboardCommands"

namespace UE::Audio::Insights
{
	FSoundDashboardCommands::FSoundDashboardCommands()
		: TCommands<FSoundDashboardCommands>("SoundDashboardCommands", LOCTEXT("SoundDashboardCommands_ContextDescText", "Sound Dashboard Commands"), NAME_None, FSlateStyle::GetStyleName())
	{

	}
	
	void FSoundDashboardCommands::RegisterCommands()
	{
		UI_COMMAND(Pin, "Pin", "Pins the selected sound in the Pinned category.", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Control));
		UI_COMMAND(Unpin, "Unpin", "Removes the selected sound from the Pinned category.", EUserInterfaceActionType::Button, FInputChord(EKeys::U, EModifierKey::Control));
		UI_COMMAND(Browse, "Browse To Asset", "Browses to the selected sound asset in the content browser.", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Control));
		UI_COMMAND(Edit, "Edit", "Opens the selected sound for edit.", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Control));

		UI_COMMAND(ViewFullTree, "Tree View", "Organize sounds into categories.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F1, EModifierKey::Control));
		UI_COMMAND(ViewActiveSounds, "Active Sounds", "Organize sounds into Active Sounds.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F2, EModifierKey::Control));
		UI_COMMAND(ViewFlatList, "Flat List", "Display sounds as individual waves in a flat list.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F3, EModifierKey::Control));

		UI_COMMAND(AutoExpandCategories, "Categories", "Auto-expand new categories.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F1, EModifierKey::Shift));
		UI_COMMAND(AutoExpandEverything, "Everything", "Auto-expand all new categories and sounds.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F2, EModifierKey::Shift));
		UI_COMMAND(AutoExpandNothing, "Nothing", "Don't auto-expand anything.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::F3, EModifierKey::Shift));
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
