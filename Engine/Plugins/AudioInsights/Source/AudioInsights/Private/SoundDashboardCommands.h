// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

namespace UE::Audio::Insights
{
	class FSoundDashboardCommands : public TCommands<FSoundDashboardCommands>
	{
	public:
		FSoundDashboardCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<const FUICommandInfo> GetPinCommand() const    { return Pin; }
		TSharedPtr<const FUICommandInfo> GetUnpinCommand() const  { return Unpin; }
		TSharedPtr<const FUICommandInfo> GetBrowseCommand() const { return Browse; }
		TSharedPtr<const FUICommandInfo> GetEditCommand() const   { return Edit; }
		TSharedPtr<const FUICommandInfo> GetViewFullTreeCommand() const { return ViewFullTree; }
		TSharedPtr<const FUICommandInfo> GetViewActiveSoundsCommand() const { return ViewActiveSounds; }
		TSharedPtr<const FUICommandInfo> GetViewFlatList() const { return ViewFlatList; }
		TSharedPtr<const FUICommandInfo> GetAutoExpandCategoriesCommand() const { return AutoExpandCategories; }
		TSharedPtr<const FUICommandInfo> GetAutoExpandEverythingCommand() const { return AutoExpandEverything; }
		TSharedPtr<const FUICommandInfo> GetAutoExpandNothingCommand() const { return AutoExpandNothing; }
		
	private:
		TSharedPtr<FUICommandInfo> Pin;
		TSharedPtr<FUICommandInfo> Unpin;
		TSharedPtr<FUICommandInfo> Browse;
		TSharedPtr<FUICommandInfo> Edit;
		TSharedPtr<FUICommandInfo> ViewFullTree;
		TSharedPtr<FUICommandInfo> ViewActiveSounds;
		TSharedPtr<FUICommandInfo> ViewFlatList;
		TSharedPtr<FUICommandInfo> AutoExpandCategories;
		TSharedPtr<FUICommandInfo> AutoExpandEverything;
		TSharedPtr<FUICommandInfo> AutoExpandNothing;
	};
} // namespace UE::Audio::Insights
