// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundPlotsWidgetCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FSoundPlotsWidgetCommands::FSoundPlotsWidgetCommands()
		: TCommands<FSoundPlotsWidgetCommands>("SoundPlotsWidgetCommands", LOCTEXT("SoundPlotsWidgetCommands_ContextDescText", "Sound Plots Widget Commands"), NAME_None, UE::Audio::Insights::FSlateStyle::GetStyleName())
	{
	}

	void FSoundPlotsWidgetCommands::RegisterCommands()
	{
		UI_COMMAND(ResetInspectTimestampPlots, "Reset Inspect Timestamp Plots", "Resumes gathering the latest audio data and auto-scrolling in the Plots tab if previously paused.", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	}

} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
