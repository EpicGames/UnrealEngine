// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

namespace UE::Audio::Insights
{
	class FSoundPlotsWidgetCommands : public TCommands<FSoundPlotsWidgetCommands>
	{
	public:
		FSoundPlotsWidgetCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<const FUICommandInfo> GetResetInspectTimestampCommand() const { return ResetInspectTimestampPlots; }

	private:
		TSharedPtr<FUICommandInfo> ResetInspectTimestampPlots;
	};
} // namespace UE::Audio::Insights
