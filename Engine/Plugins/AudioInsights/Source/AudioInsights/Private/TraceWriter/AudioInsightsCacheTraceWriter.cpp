// Copyright Epic Games, Inc. All Rights Reserved.
#include "TraceWriter/AudioInsightsCacheTraceWriter.h"

#include "AudioInsightsLog.h"
#include "BuildSettings.h"
#include "Cache/AudioCachedMessageChunk.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Misc/App.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Trace/OutDataStream.h"
#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	namespace FAudioInsightsCacheTraceWriterPrivate
	{
		struct FResolvedHandler
		{
			FCacheWriteHandler Handler;
			uint32 EventId = 0;
		};

		struct FFrameEventContext
		{
			uint32 GameThreadId      = 0;
			uint32 BeginFrameEventId = 0;
			uint32 EndFrameEventId   = 0;
		};

		struct FTimestampContext
		{
			double CacheStartTimestamp = 0.0;
			uint64 BaseTimestamp       = 0;
			uint64 TimeFrequency       = 0;
		};

		uint64 CacheTimestampToTraceCycles(double InTimestampSeconds, const FTimestampContext& InContext)
		{
			const double SecondsFromCacheStart = FMath::Max(0.0, InTimestampSeconds - InContext.CacheStartTimestamp);
			return InContext.BaseTimestamp + static_cast<uint64>(SecondsFromCacheStart * InContext.TimeFrequency);
		}

		TMap<FName, FResolvedHandler> DeclareAudioEvents(UE::Trace::FTraceWriter& InWriter, const FAudioInsightsCacheManager& InCacheManager)
		{
			TMap<FName, FResolvedHandler> Handlers;

			for (const auto& [MessageName, Message] : InCacheManager.GetMessageTypeRegistry())
			{
				FCacheWriteHandler Handler = Message->GetCacheWriteHandler();

				if (Handler.IsValid())
				{
					const uint32 EventId = Handler.DeclareEvent(InWriter);
					Handlers.Emplace(MessageName, FResolvedHandler{ MoveTemp(Handler), EventId });
				}
			}

			return Handlers;
		}

		FFrameEventContext DeclareFrameEvents(UE::Trace::FTraceWriter& InWriter)
		{
			FFrameEventContext Context;

			Context.GameThreadId = InWriter.RegisterThread(ANSITEXTVIEW("GameThread"));

			Context.BeginFrameEventId = InWriter.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("BeginFrame"))
				.Field(ANSITEXTVIEW("Cycle"),     UE::Trace::ETraceWriterFieldType::Uint64)
				.Field(ANSITEXTVIEW("FrameType"), UE::Trace::ETraceWriterFieldType::Uint8)
				.End();

			Context.EndFrameEventId = InWriter.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("EndFrame"))
				.Field(ANSITEXTVIEW("Cycle"),     UE::Trace::ETraceWriterFieldType::Uint64)
				.Field(ANSITEXTVIEW("FrameType"), UE::Trace::ETraceWriterFieldType::Uint8)
				.End();

			return Context;
		}

		void WriteDiagnosticsSessionEvent(UE::Trace::FTraceWriter& InWriter)
		{
			constexpr int32 InstanceIdSize = 4;

			FGuid InstanceGuid = FApp::GetInstanceId();
			uint32 InstanceId[InstanceIdSize];

			for (int32 Index = 0; Index < InstanceIdSize; ++Index)
			{
				InstanceId[Index] = InstanceGuid[Index];
			}

			const uint32 Session2EventId = InWriter.DeclareDiagnosticsSession2Event();

			InWriter.SetCurrentThreadImportants();

			InWriter.WriteEvent(Session2EventId)
				.Field(ANSITEXTVIEW("ConfigurationType"), static_cast<uint8>(FApp::GetBuildConfiguration()))
				.Field(ANSITEXTVIEW("TargetType"), static_cast<uint8>(FApp::GetBuildTargetType()))
				.Field(ANSITEXTVIEW("Changelist"), BuildSettings::GetCurrentChangelist())
				.Field(ANSITEXTVIEW("InstanceId"), TConstArrayView<uint32>(InstanceId, InstanceIdSize))
				.Field(ANSITEXTVIEW("Platform"), FAnsiStringView(UE_STRINGIZE(UBT_COMPILED_PLATFORM)))
				.Field(ANSITEXTVIEW("AppName"), UE_APP_NAME)
				.Field(ANSITEXTVIEW("ProjectName"), FApp::GetProjectName())
				.Field(ANSITEXTVIEW("Branch"), FStringView(*FApp::GetBranchName()))
				.Field(ANSITEXTVIEW("BuildVersion"), FApp::GetBuildVersion())
				.Field(ANSITEXTVIEW("EngineVersion"), BuildSettings::GetEngineVersionString())
				.End();
		}

		void WriteFrameEvents(UE::Trace::FTraceWriter& InWriter, const FAudioInsightsCacheManager& InCacheManager, const FFrameEventContext& InFrameContext, const FTimestampContext& InTimestampContext)
		{
			if (InTimestampContext.TimeFrequency == 0)
			{
				return;
			}

			const uint64 CacheDurationCycles = static_cast<uint64>(InCacheManager.GetCacheDuration() * InTimestampContext.TimeFrequency);
			constexpr uint64 FrameIntervalMs = 16; // ~60fps synthetic frame interval
			const uint64 FrameDuration = (FrameIntervalMs * InTimestampContext.TimeFrequency) / 1000;

			if (FrameDuration == 0)
			{
				return;
			}
			const uint8 GameFrameType  = static_cast<uint8>(TraceFrameType_Game);

			InWriter.SetCurrentThread(InFrameContext.GameThreadId);

			for (uint64 Offset = 0; Offset < CacheDurationCycles; Offset += FrameDuration)
			{
				const uint64 Duration = FMath::Min(FrameDuration, CacheDurationCycles - Offset);

				InWriter.WriteEvent(InFrameContext.BeginFrameEventId)
					.Field(0, InTimestampContext.BaseTimestamp + Offset)
					.Field(1, GameFrameType)
					.End();

				InWriter.WriteEvent(InFrameContext.EndFrameEventId)
					.Field(0, InTimestampContext.BaseTimestamp + Offset + Duration)
					.Field(1, GameFrameType)
					.End();
			}
		}

		void WriteAudioEvents(UE::Trace::FTraceWriter& InWriter, const FAudioInsightsCacheManager& InCacheManager, const TMap<FName, FResolvedHandler>& InHandlers, const FTimestampContext& InTimestampContext)
		{
			uint32 TotalMessagesWritten = 0;
			TSet<FName> UnhandledMessageTypes;

			for (uint32 ChunkIndex = 0; ChunkIndex < InCacheManager.GetNumUsedChunks(); ++ChunkIndex)
			{
				const FAudioCachedMessageChunk* const Chunk = InCacheManager.GetChunk(ChunkIndex);

				if (Chunk == nullptr || !Chunk->HasAnyData())
				{
					continue;
				}

				for (const auto& [MessageName, MessageArray] : Chunk->GetAllChunkMessages())
				{
					const FResolvedHandler* ResolvedHandler = InHandlers.Find(MessageName);

					if (ResolvedHandler == nullptr)
					{
						UnhandledMessageTypes.Emplace(MessageName);
						continue;
					}

					for (uint64 MessageIndex = 0; MessageIndex < MessageArray->Num(); ++MessageIndex)
					{
						const TSharedPtr<IAudioCachedMessage>& AudioCachedMessage = (*MessageArray)[MessageIndex];

						if (AudioCachedMessage.IsValid())
						{
							const uint64 TimestampCycles = CacheTimestampToTraceCycles(AudioCachedMessage->Timestamp, InTimestampContext);

							ResolvedHandler->Handler.WriteEvent(InWriter, ResolvedHandler->EventId, *AudioCachedMessage, TimestampCycles);
							++TotalMessagesWritten;
						}
					}
				}
			}

			UE_LOG(LogAudioInsights, Log, TEXT("Audio events written: %u, unhandled message types: %d"), TotalMessagesWritten, UnhandledMessageTypes.Num());
		}
	} // namespace FAudioInsightsCacheTraceWriterPrivate

	bool FAudioInsightsCacheTraceWriter::WriteCacheToFile(const FAudioInsightsCacheManager& InCacheManager, const FString& InFilePath)
	{
		using namespace FAudioInsightsCacheTraceWriterPrivate;

		// Setup data stream
		UE::Trace::FFileOutDataStream DataStream;

		if (!DataStream.Open(*InFilePath))
		{
			UE_LOG(LogAudioInsights, Warning, TEXT("Failed to open file for cache trace writing: %s"), *InFilePath);
			return false;
		}

		// Setup writer
		UE::Trace::FTraceWriter Writer(DataStream);
		Writer.Begin();

		// Setup timestamp conversion context
		const FTimestampContext TimestampContext
		{
			.CacheStartTimestamp = InCacheManager.GetCacheStartTimeStamp(),
			.BaseTimestamp       = Writer.GetStartTime(),
			.TimeFrequency       = Writer.GetTimeFrequency()
		};

		// Declare events
		FFrameEventContext FrameEventContext = DeclareFrameEvents(Writer);

		TMap<FName, FResolvedHandler> Handlers = DeclareAudioEvents(Writer, InCacheManager);

		if (Handlers.IsEmpty())
		{
			UE_LOG(LogAudioInsights, Warning, TEXT("No messages registered cache write handlers. Nothing to write."));
			Writer.End();

			return false;
		}

		// Write events
		WriteDiagnosticsSessionEvent(Writer);
		WriteFrameEvents(Writer, InCacheManager, FrameEventContext, TimestampContext);
		WriteAudioEvents(Writer, InCacheManager, Handlers, TimestampContext);

		Writer.End();

		UE_LOG(LogAudioInsights, Log, TEXT("Cache trace written to %s"), *InFilePath);

		return true;
	}

} // namespace UE::Audio::Insights
