// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/AssetEditorContextMenuHelper.h"

#if WITH_EDITOR
#include "AssetEditorCommands.h"
#include "AudioInsightsStyle.h"
#include "Editor.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Events.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Views/TableDashboardViewFactory.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FAssetEditorContextMenuHelper::FAssetEditorContextMenuHelper()
	{
		BindCommands();
	}

	TSharedPtr<SWidget> FAssetEditorContextMenuHelper::ContructContextMenuOptions()
	{
		if (GetEditableAsset() == nullptr)
		{
			return SNullWidget::NullWidget;
		}

		const FAssetEditorCommands& Commands = FAssetEditorCommands::Get();
		FMenuBuilder MenuBuilder(true /*bInShouldCloseWindowAfterMenuSelection*/, CommandList);

		MenuBuilder.BeginSection("AssetEditorActions", LOCTEXT("AudioDashboard_AssetEditorActions_RightClick_HeaderText", "Asset options"));
		{
			MenuBuilder.AddMenuEntry(Commands.GetBrowseCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Browse"));
			MenuBuilder.AddMenuEntry(Commands.GetEditCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Edit"));
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	bool FAssetEditorContextMenuHelper::ProcessCommandBindings(const FKeyEvent& InKeyEvent) const
	{
		return CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent);
	}

	void FAssetEditorContextMenuHelper::SetAssetEntry(const TSharedPtr<IObjectDashboardEntry>& InEntry)
	{
		if (InEntry.IsValid())
		{
			Entry = InEntry.ToWeakPtr();
		}
	}

	void FAssetEditorContextMenuHelper::ResetAssetEntry()
	{
		Entry.Reset();
	}

	void FAssetEditorContextMenuHelper::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FAssetEditorCommands& Commands = FAssetEditorCommands::Get();

		CommandList->MapAction(
			Commands.GetBrowseCommand(),
			FExecuteAction::CreateLambda([this]() { BrowseToAsset(); }),
			FCanExecuteAction::CreateLambda([this]() { return GetEditableAsset() != nullptr; }));

		CommandList->MapAction(
			Commands.GetEditCommand(),
			FExecuteAction::CreateLambda([this]() { OpenAsset(); }),
			FCanExecuteAction::CreateLambda([this]() { return GetEditableAsset() != nullptr; }));
	}

	bool FAssetEditorContextMenuHelper::BrowseToAsset() const
	{
		if (GEditor)
		{
			if (const TObjectPtr<UObject> Asset = GetEditableAsset())
			{
				GEditor->SyncBrowserToObject(Asset);
				return true;
			}
		}

		return false;
	}

	bool FAssetEditorContextMenuHelper::OpenAsset() const
	{
		if (GEditor)
		{
			if (const TObjectPtr<UObject> Asset = GetEditableAsset())
			{
				if (const TObjectPtr<UAssetEditorSubsystem> AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					return AssetSubsystem->OpenEditorForAsset(Asset);
				}
			}
		}

		return false;
	}

	TObjectPtr<UObject> FAssetEditorContextMenuHelper::GetEditableAsset() const
	{
		const TSharedPtr<IObjectDashboardEntry> EntryPinned = Entry.Pin();
		if (EntryPinned.IsValid())
		{
			const TObjectPtr<UObject> ObjectPtr = EntryPinned->GetObject();
			if (ObjectPtr && ObjectPtr->IsAsset())
			{
				return ObjectPtr;
			}
		}

		return nullptr;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR