// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/AudioBusDashboardViewFactory.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTimingViewExtender.h"
#include "IAudioInsightsModule.h"
#include "Internationalization/Text.h"
#include "Providers/AudioMeterProvider.h"
#include "Providers/AudioBusTraceProvider.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#else
#include "AudioInsightsComponent.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FAudioBusDashboardViewFactoryPrivate
	{
		const FAudioBusDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FAudioBusDashboardEntry&>(InData);
		};

		void SendAudioBusEnvelopeFollowerCVar(const bool bInIsAudioBusChecked, const uint32 InAudioBusId)
		{
#if WITH_EDITOR
			const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
#else
			const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
#endif // WITH_EDITOR

			const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();

			const FString AudioBusCommandStr = FString::Printf(TEXT("au.%sAudioBusEnvelopeFollower_AD%d %u"), bInIsAudioBusChecked ? TEXT("Start") : TEXT("Stop"), AudioDeviceId, InAudioBusId);
			
			IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();
			AudioInsightsTraceModule.ExecuteConsoleCommand(AudioBusCommandStr);
		}

		const FSlateColor AudioBusMeterTextColor(FColor(80, 200, 255));
	}

	FAudioBusDashboardViewFactory::FAudioBusDashboardViewFactory()
	{
		IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();

		AudioBusProvider = MakeShared<FAudioBusTraceProvider>();

		AudioInsightsTraceModule.AddTraceProvider(AudioBusProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			AudioBusProvider
		};

		SortByColumn = "Name";
		SortMode     = EColumnSortMode::Ascending;

		AudioBusProvider->OnAudioBusAdded.BindRaw(this, &FAudioBusDashboardViewFactory::HandleOnAudioBusAdded);
		AudioBusProvider->OnAudioBusRemoved.BindRaw(this, &FAudioBusDashboardViewFactory::HandleOnAudioBusRemoved);
		AudioBusProvider->OnAudioBusStarted.BindRaw(this, &FAudioBusDashboardViewFactory::HandleOnAudioBusStarted);
		AudioBusProvider->OnAudioBusListUpdated.BindRaw(this, &FAudioBusDashboardViewFactory::RequestListRefresh);
		AudioBusProvider->OnTimeMarkerUpdated.BindRaw(this, &FAudioBusDashboardViewFactory::HandleOnTimeMarkerUpdated);
	}

	FAudioBusDashboardViewFactory::~FAudioBusDashboardViewFactory()
	{
		AudioBusProvider->OnAudioBusAdded.Unbind();
		AudioBusProvider->OnAudioBusRemoved.Unbind();
		AudioBusProvider->OnAudioBusStarted.Unbind();
		AudioBusProvider->OnAudioBusListUpdated.Unbind();
		AudioBusProvider->OnTimeMarkerUpdated.Unbind();
	}

	FName FAudioBusDashboardViewFactory::GetName() const
	{
		return "AudioBus";
	}

	FText FAudioBusDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_AudioBus_DisplayName", "AudioBuses");
	}
	
	FSlateIcon FAudioBusDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon");
	}

	EDefaultDashboardTabStack FAudioBusDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Analysis;
	}

	TSharedRef<SWidget> FAudioBusDashboardViewFactory::GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& InColumnName)
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		if (InColumnName == "Active")
		{
			static const FLinearColor DarkGreen(0.027f, 0.541f, 0.22f);
			static const float Radius = 4.0f;
			static const FVector2f Size(7.0f, 7.0f);
			static const FSlateRoundedBoxBrush GreenRoundedBrush(DarkGreen, Radius, Size);

			return SNew(SBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Padding(6.0f)
				.Visibility_Lambda([InRowData]()
				{ 
					const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(InRowData.Get());
					return AudioBusDashboardEntry.bHasActivity ? EVisibility::Visible : EVisibility::Hidden;
				})
				[
					SNew(SImage)
					.Image(&GreenRoundedBrush)
				];
		}
		else if (InColumnName == "Name")
		{
			const FColumnData& ColumnData = GetColumns()[InColumnName];
			const FText ValueText = ColumnData.GetDisplayValue(InRowData.Get());

			if (ValueText.IsEmpty())
			{
				return SNullWidget::NullWidget;
			}

			const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*InRowData);

			if (ShouldUpdateAudioMeter(AudioBusDashboardEntry) && !AudioBusDashboardEntry.AudioMeterInfo->EnvelopeValues.IsEmpty())
			{
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName());
			}

			return SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsEnabled_Lambda([this, InRowData]()
					{
#if WITH_EDITOR
						return AudioBusProvider && AudioBusProvider->GetMessageCacheAndProcessingStatus() == ECacheAndProcess::Latest;
#else
						return IAudioInsightsModule::IsLiveSession();
#endif // WITH_EDITOR
					})
					.IsChecked_Lambda([this, InRowData]()
					{
						const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*InRowData);

						if (CheckedAudioBuses.Contains(AudioBusDashboardEntry.AudioBusId))
						{
							return ECheckBoxState::Checked;
						}

#if !WITH_EDITOR
						if (!IAudioInsightsModule::IsLiveSession())
						{
							return AudioBusDashboardEntry.bEnvelopeFollowerEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
#endif // !WITH_EDITOR

						return ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this, InRowData](ECheckBoxState NewState)
					{
						const bool bIsChecked = NewState == ECheckBoxState::Checked;

						// When the clicked row is part of the current multi-selection, the new state propagates to
						// every selected row (bulk-enable after Shift+Click). Otherwise it only affects the clicked row.
						TArray<TSharedPtr<IDashboardDataViewEntry>> EntriesToToggle;

						if (FilteredEntriesListView.IsValid())
						{
							const TArray<TSharedPtr<IDashboardDataViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

							if (SelectedItems.Contains(InRowData.ToSharedPtr()))
							{
								EntriesToToggle = SelectedItems;
							}
						}

						// Clicked row is not part of the current selection: toggle only that one row.
						if (EntriesToToggle.IsEmpty())
						{
							EntriesToToggle.Emplace(InRowData.ToSharedPtr());
						}

						for (const TSharedPtr<IDashboardDataViewEntry>& Entry : EntriesToToggle)
						{
							if (Entry.IsValid())
							{
								const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*Entry);
								SetAudioBusEnvelopeFollowerEnabled(AudioBusDashboardEntry, bIsChecked);
							}
						}
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(5.0f)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(ValueText)
					.ToolTipText(ValueText)
					.ColorAndOpacity(AudioBusDashboardEntry.AudioBusType == EAudioBusType::CodeGenerated ? AudioBusMeterTextColor : FSlateColor::UseForeground())
					.OnDoubleClicked_Lambda([this, InRowData](const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
					{
#if WITH_EDITOR
						if (GEditor)
						{
							const TSharedPtr<IObjectDashboardEntry> ObjectData = StaticCastSharedPtr<IObjectDashboardEntry>(InRowData.ToSharedPtr());
							if (ObjectData.IsValid())
							{
								const TObjectPtr<UObject> Object = ObjectData->GetObject();
								if (Object && Object->IsAsset())
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
									return FReply::Handled();
								}
							}
						}
#endif // WITH_EDITOR

						return FReply::Unhandled();
					})
				];
		}

		return SNullWidget::NullWidget;
	}

	void FAudioBusDashboardViewFactory::ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason)
	{
		FilterByAudioBusName();
		FilterByAudioBusType();

		if (bShouldRebuildAudioMeters)
		{
			RebuildAudioMeters();
			bShouldRebuildAudioMeters = false;
		}
		else if (bListRefreshed)
		{
			UpdateAudioMeters();
			bListRefreshed = false;
		}
		else if (bShouldReactivateAudioMeters)
		{
			ReactivateAudioMeters();
			bShouldReactivateAudioMeters = false;
		}
	}

	TSharedRef<SWidget> FAudioBusDashboardViewFactory::MakeAudioBusTypeFilterWidget()
	{
		if (AudioBusTypes.IsEmpty())
		{
			AudioBusTypes.Emplace(MakeShared<FComboboxSelectionItem>(EAudioBusTypeComboboxSelection::AssetBased,    LOCTEXT("AudioBusDashboard_AudioBusTypeAssetBased",    "Asset")));
			AudioBusTypes.Emplace(MakeShared<FComboboxSelectionItem>(EAudioBusTypeComboboxSelection::CodeGenerated, LOCTEXT("AudioBusDashboard_AudioBusTypeCodeGenerated", "Code Generated")));
			AudioBusTypes.Emplace(MakeShared<FComboboxSelectionItem>(EAudioBusTypeComboboxSelection::All,           LOCTEXT("AudioBusDashboard_AudioBusTypeAll",           "All")));

			SelectedAudioBusType = AudioBusTypes[0];
		}

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Margin(FMargin(0.0, 2.0, 0.0, 0.0))
				.Text(LOCTEXT("AudioBusesDashboard_TypeFilterText", "Type Filter:"))
			]
			+ SHorizontalBox::Slot()
			.MaxWidth(2.0f)
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(SComboBox<TSharedPtr<FComboboxSelectionItem>>)
				.OptionsSource(&AudioBusTypes)
				.OnGenerateWidget_Lambda([this](const TSharedPtr<FComboboxSelectionItem>& AudioBusTypePtr)
				{
					const FText AudioBusTypeDisplayName = AudioBusTypePtr.IsValid() ? AudioBusTypePtr->Value /*DisplayName*/ : FText::GetEmpty();

					return SNew(STextBlock)
						.Text(AudioBusTypeDisplayName);
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<FComboboxSelectionItem> InSelectedAudioBusTypePtr, ESelectInfo::Type)
				{
					if (InSelectedAudioBusTypePtr.IsValid())
					{
						SelectedAudioBusType = InSelectedAudioBusTypePtr;
						UpdateFilterReason = EProcessReason::FilterUpdated;
					}
				})
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						const int32 FoundIndex = AudioBusTypes.Find(SelectedAudioBusType);
						if (AudioBusTypes.IsValidIndex(FoundIndex) && AudioBusTypes[FoundIndex].IsValid())
						{
							return AudioBusTypes[FoundIndex]->Value;
						}

						return FText::GetEmpty();
					})
				]
			];
	}

	TSharedRef<SWidget> FAudioBusDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			AudioInsightsModule.GetTimingViewExtender().OnTimeControlMethodReset.AddSP(this, &FAudioBusDashboardViewFactory::HandleOnTimeControlMethodReset);

			SAssignNew(DashboardWidget, SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 6.0f)
			[
				MakeAudioBusTypeFilterWidget()
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs)
			];
		}
#if WITH_EDITOR
		else
		{
			if (AudioBusProvider.IsValid())
			{
				AudioBusProvider->RequestEntriesUpdate();
			}
		}
#endif // WITH_EDITOR

		// Update audio meters (if any has been checked previously)
		UpdateAudioMeters();

		return DashboardWidget->AsShared();
	}

	void FAudioBusDashboardViewFactory::SetAudioBusEnvelopeFollowerEnabled(const FAudioBusDashboardEntry& InAudioBusDashboardEntry, const bool bInEnabled)
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		SendAudioBusEnvelopeFollowerCVar(bInEnabled, InAudioBusDashboardEntry.AudioBusId);

		if (bInEnabled)
		{
			CheckedAudioBuses.Emplace(InAudioBusDashboardEntry.AudioBusId);
			FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(InAudioBusDashboardEntry.AudioBusId, InAudioBusDashboardEntry.AudioMeterInfo, InAudioBusDashboardEntry.GetDisplayName(), AudioBusMeterTextColor);
		}
		else
		{
			CheckedAudioBuses.Remove(InAudioBusDashboardEntry.AudioBusId);
			FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(InAudioBusDashboardEntry.AudioBusId);
		}
	}

	const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& FAudioBusDashboardViewFactory::GetColumns() const
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		auto CreateColumnData = []()
		{
			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					"Active",
					{
						LOCTEXT("AudioBusDashboard_ActiveDisplayName", "Active"),
						[](const IDashboardDataViewEntry& InData) { return FText::GetEmpty(); },
						nullptr,	/* GetIconName */
						false,		/* bDefaultHidden */
						0.08f,		/* FillWidth */
						EHorizontalAlignment::HAlign_Center
					}
				},
				{
					"Name",
					{
						LOCTEXT("AudioBusDashboard_NameColumnDisplayName", "Name"),
						[](const IDashboardDataViewEntry& InData) { return CastEntry(InData).GetDisplayName(); },
						nullptr,	/* GetIconName */
						false,		/* bDefaultHidden */
						0.92f,		/* FillWidth */
						EHorizontalAlignment::HAlign_Left
					}
				}
			};
		};
		
		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		
		return ColumnData;
	}

	void FAudioBusDashboardViewFactory::SortTable()
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		if (SortByColumn == "Active")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioBusDashboardEntry& AData = CastEntry(*A.Get());
					const FAudioBusDashboardEntry& BData = CastEntry(*B.Get());

					return !AData.bHasActivity && BData.bHasActivity;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioBusDashboardEntry& AData = CastEntry(*A.Get());
					const FAudioBusDashboardEntry& BData = CastEntry(*B.Get());

					return !BData.bHasActivity && AData.bHasActivity;
				});
			}
		}
		else if (SortByColumn == "Name")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioBusDashboardEntry& AData = CastEntry(*A.Get());
					const FAudioBusDashboardEntry& BData = CastEntry(*B.Get());

					return AData.GetDisplayName().CompareToCaseIgnored(BData.GetDisplayName()) < 0;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioBusDashboardEntry& AData = CastEntry(*A.Get());
					const FAudioBusDashboardEntry& BData = CastEntry(*B.Get());

					return BData.GetDisplayName().CompareToCaseIgnored(AData.GetDisplayName()) < 0;
				});
			}
		}
	}

	void FAudioBusDashboardViewFactory::FilterByAudioBusName()
	{
		const FString FilterString = GetSearchFilterText().ToString();
		
		FTraceTableDashboardViewFactory::FilterEntries<FAudioBusTraceProvider>([&FilterString](const IDashboardDataViewEntry& Entry)
		{
			const FAudioBusDashboardEntry& AudioBusEntry = static_cast<const FAudioBusDashboardEntry&>(Entry);
			
			return AudioBusEntry.GetDisplayName().ToString().Contains(FilterString);
		});
	}

	void FAudioBusDashboardViewFactory::FilterByAudioBusType()
	{
		using namespace UE::Audio::Insights;
		using namespace FAudioBusDashboardViewFactoryPrivate;

		TArray<TSharedPtr<IDashboardDataViewEntry>> EntriesToFilterOut;

		const EAudioBusTypeComboboxSelection SelectedAudioBusTypeEnum = SelectedAudioBusType.IsValid()? SelectedAudioBusType->Key : EAudioBusTypeComboboxSelection::All;

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			if (Entry.IsValid())
			{
				const FAudioBusDashboardEntry& AudioBusAssetDashboardEntry = CastEntry(*Entry);

				if (SelectedAudioBusTypeEnum != EAudioBusTypeComboboxSelection::All)
				{
					if ((SelectedAudioBusTypeEnum == EAudioBusTypeComboboxSelection::AssetBased    && AudioBusAssetDashboardEntry.AudioBusType != EAudioBusType::AssetBased) ||
						(SelectedAudioBusTypeEnum == EAudioBusTypeComboboxSelection::CodeGenerated && AudioBusAssetDashboardEntry.AudioBusType != EAudioBusType::CodeGenerated))
					{
						EntriesToFilterOut.Emplace(Entry);
					}
				}
			}
		}

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : EntriesToFilterOut)
		{
			DataViewEntries.Remove(Entry);
		}
	}

	void FAudioBusDashboardViewFactory::RebuildAudioMeters() const
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*Entry);

			if (ShouldUpdateAudioMeter(AudioBusDashboardEntry))
			{
				FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName(), AudioBusMeterTextColor);

				if (!AudioBusDashboardEntry.AudioMeterInfo->EnvelopeValues.IsEmpty())
				{
					FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName());
				}
			}
			else
			{
				FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId);
			}
		}
	}

	bool FAudioBusDashboardViewFactory::ShouldUpdateAudioMeter(const FAudioBusDashboardEntry& InAudioBusDashboardEntry) const
	{
#if WITH_EDITOR
		if (!AudioBusProvider.IsValid())
		{
			return false;
		}
#endif // WITH_EDITOR

		if (CheckedAudioBuses.Contains(InAudioBusDashboardEntry.AudioBusId))
		{
			return true;
		}

#if !WITH_EDITOR
		if (!IAudioInsightsModule::IsLiveSession())
		{
			return InAudioBusDashboardEntry.bEnvelopeFollowerEnabled;
		}
#endif // !WITH_EDITOR

		return false;
	}

	void FAudioBusDashboardViewFactory::UpdateAudioMeters() const
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*Entry);

			if (CheckedAudioBuses.Find(AudioBusDashboardEntry.AudioBusId) != nullptr)
			{
				SendAudioBusEnvelopeFollowerCVar(true, AudioBusDashboardEntry.AudioBusId);
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName());
			}
		}
	}

	void FAudioBusDashboardViewFactory::ReactivateAudioMeters()
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		for (auto It = PendingToReactivateAudioBusIds.CreateIterator(); It; ++It)
		{
			const TSharedPtr<IDashboardDataViewEntry>* FoundEntry = DataViewEntries.FindByPredicate([&AudioBusId = *It](const TSharedPtr<IDashboardDataViewEntry>& Entry)
				{
					const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*Entry);

					return AudioBusDashboardEntry.AudioBusId == AudioBusId;
				});

			if (FoundEntry)
			{
				const TSharedPtr<IDashboardDataViewEntry> Entry = *FoundEntry;
				const FAudioBusDashboardEntry& AudioBusDashboardEntry = CastEntry(*Entry);

				SendAudioBusEnvelopeFollowerCVar(true, AudioBusDashboardEntry.AudioBusId);
				FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName(), AudioBusMeterTextColor);
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(AudioBusDashboardEntry.AudioBusId, AudioBusDashboardEntry.AudioMeterInfo, AudioBusDashboardEntry.GetDisplayName());

				It.RemoveCurrent();
			}
		}
	}

	void FAudioBusDashboardViewFactory::HandleOnAudioBusAdded(const uint32 InAudioBusId)
	{
		RequestListRefresh();
	}

	void FAudioBusDashboardViewFactory::HandleOnAudioBusRemoved(const uint32 InAudioBusId)
	{
		using namespace FAudioBusDashboardViewFactoryPrivate;

		SendAudioBusEnvelopeFollowerCVar(false, InAudioBusId);
		FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(InAudioBusId);
		RequestListRefresh();
	}

	void FAudioBusDashboardViewFactory::HandleOnAudioBusStarted(const uint32 InAudioBusId)
	{
		if (CheckedAudioBuses.Contains(InAudioBusId))
		{
			PendingToReactivateAudioBusIds.Emplace(InAudioBusId);
			bShouldReactivateAudioMeters = true;
		}
	}

	void FAudioBusDashboardViewFactory::HandleOnTimeMarkerUpdated()
	{
		bShouldRebuildAudioMeters = true;
	}

	void FAudioBusDashboardViewFactory::HandleOnTimeControlMethodReset()
	{
		bShouldRebuildAudioMeters = true;
	}

	void FAudioBusDashboardViewFactory::RequestListRefresh()
	{
		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->RequestListRefresh();
		}

		bListRefreshed = true;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
