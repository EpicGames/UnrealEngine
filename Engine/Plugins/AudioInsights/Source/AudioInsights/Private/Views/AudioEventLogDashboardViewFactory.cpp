// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/AudioEventLogDashboardViewFactory.h"

#include "AudioEventLogEditorCommands.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsSettings.h"
#include "AudioInsightsStyle.h"
#include "AudioMixerTrace.h"
#include "Features/IModularFeatures.h"
#include "Messages/AudioEventLogTraceMessages.h"
#include "Providers/AudioEventLogTraceProvider.h"
#include "Widgets/Layout/SBox.h"

#if !WITH_EDITOR
#include "AudioInsightsComponent.h"
#endif // !WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsightsEventLog"

namespace UE::Audio::Insights
{
	/////////////////////////////////////////////////////////////////////////////////////////
	// AudioEventLogPrivate
	namespace AudioEventLogPrivate
	{
		const FAudioEventLogDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FAudioEventLogDashboardEntry&>(InData);
		};

		FAudioEventLogDashboardEntry& CastEntry(IDashboardDataViewEntry& InData)
		{
			return static_cast<FAudioEventLogDashboardEntry&>(InData);
		};

		bool CanClearLog()
		{
#if WITH_EDITOR
			return true;
#else
			return IAudioInsightsModule::IsLiveSession();
#endif // WITH_EDITOR
		}

		const FString CategorySoundActivity = TEXT("Sound Activity");
		const FString CategoryVirtualization = TEXT("Virtualization");
		const FString CategoryPlayRequests = TEXT("Play Requests");
		const FString CategoryStopRequests = TEXT("Stop Requests");
		const FString CategoryPlayErrors = TEXT("Play Errors");
		const FString CategoryMessages = TEXT("Messages");
		const FString CategoryPIESession = TEXT("PIE Session");
		const FString CategoryCustom = TEXT("Custom");

		const FName CacheStatusColumnID = "CacheStatus";
		const FName MessageIDColumnID = "MessageID";
		const FName TimestampColumnID = "Timestamp";
		const FName PlayOrderColumnID = "PlayOrder";
		const FName EventColumnID = "Event";
		const FName AssetNameColumnID = "Asset";
		const FName ActorColumnID = "Actor";
		const FName AudioComponentColumnID = "AudioComponent";
		const FName CategoryColumnID = "Category";

		const FText ClearLogText = LOCTEXT("AudioEventLog_ClearLog_Text", "Clear Log");
		const FText ClearLogTooltipText = LOCTEXT("AudioEventLog_ClearLog_Tooltip", "Clears the event log for the currently selected world.");
		const FText CacheStatusColumnDisplayName = LOCTEXT("AudioEventLog_CacheStatus", "Cache Status");

		FText FindLocalizedName(const FString& Key, const TMap<FString, FText>& InBuiltinNames, const TMap<FString, FText>& InExternalNames)
		{
			const FText* const Found = InBuiltinNames.Find(Key);
			if (Found != nullptr)
			{
				return *Found;
			}

			const FText* const ExternalFound = InExternalNames.Find(Key);
			if (ExternalFound != nullptr)
			{
				return *ExternalFound;
			}

			return FText::FromString(Key);
		}

		FText GetLocalizedEventTypeName(const FString& EventType, const TMap<FString, FText>& InExternalDisplayNames)
		{
			using namespace ::Audio::Trace::EventLog;

			static const TMap<FString, FText> LocalizedNames =
			{
				{ ID::SoundStart,                     LOCTEXT("EventLogTraceMessage_Playing", "Playing") },
				{ ID::SoundStop,                      LOCTEXT("EventLogTraceMessage_Stopped", "Stopped") },
				{ ID::SoundVirtualized,               LOCTEXT("EventLogTraceMessage_Virtualize", "Virtualized") },
				{ ID::SoundRealized,                  LOCTEXT("EventLogTraceMessage_Realize", "Realized") },
				{ ID::PlayRequestSoundHandle,         LOCTEXT("EventLogTraceMessage_PlayRequestedSoundHandle", "Play Request : Sound Handle") },
				{ ID::StopRequestedSoundHandle,       LOCTEXT("EventLogTraceMessage_StopRequestedSoundHandle", "Stop Request : Sound Handle") },
				{ ID::PlayRequestAudioComponent,      LOCTEXT("EventLogTraceMessage_PlayRequestedAudioComponent", "Play Request : Audio Component") },
				{ ID::StopRequestAudioComponent,      LOCTEXT("EventLogTraceMessage_StopRequestedAudioComponent", "Stop Request : Audio Component") },
				{ ID::PlayRequestOneShot,             LOCTEXT("EventLogTraceMessage_PlayRequestedOneShot", "Play Request : One shot") },
				{ ID::PlayRequestSoundAtLocation,     LOCTEXT("EventLogTraceMessage_PlayRequestedSoundAtLocation", "Play Request : Sound at location") },
				{ ID::PlayRequestSound2D,             LOCTEXT("EventLogTraceMessage_PlayRequested2DSound", "Play Request : Play Sound 2D") },
				{ ID::PlayRequestSlateSound,          LOCTEXT("EventLogTraceMessage_PlayRequestedSlateSound", "Play Request : Slate Sound") },
				{ ID::StopRequestActiveSound,         LOCTEXT("EventLogTraceMessage_StopRequestedActiveSound", "Stop Request : Active Sound") },
				{ ID::StopRequestSoundsUsingResource, LOCTEXT("EventLogTraceMessage_StopRequestedSoundsUsingResource", "Stop Request : Sounds using resource") },
				{ ID::StopRequestConcurrency,         LOCTEXT("EventLogTraceMessage_StopRequestedConcurrency", "Stop Request : Concurrency") },
				{ ID::PauseSoundRequested,            LOCTEXT("EventLogTraceMessage_PauseSoundRequested", "Paused") },
				{ ID::ResumeSoundRequested,           LOCTEXT("EventLogTraceMessage_ResumeSoundRequested", "Resumed") },
				{ ID::StopAllRequested,               LOCTEXT("EventLogTraceMessage_StopAllRequested", "Stop All") },
				{ ID::FlushAudioDeviceRequested,      LOCTEXT("EventLogTraceMessage_FlushAudioDeviceRequested", "Flush Audio Device") },
				{ ID::PlayFailedNotPlayable,          LOCTEXT("EventLogTraceMessage_PlayFailedNotPlayable", "Play Failed : Not playable") },
				{ ID::PlayFailedOutOfRange,           LOCTEXT("EventLogTraceMessage_PlayFailedOutOfRange", "Play Failed : Out of range") },
				{ ID::PlayFailedDebugFiltered,        LOCTEXT("EventLogTraceMessage_PlayFailedDebugFiltered", "Play Failed : Debug filtered") },
				{ ID::PlayFailedConcurrency,          LOCTEXT("EventLogTraceMessage_PlayFailedConcurrency", "Play Failed : Concurrency") },
				{ ID::PIEStarted,                     LOCTEXT("EventLogTraceMessage_PIEStarted", "PIE Started") },
				{ ID::PIEStopped,                     LOCTEXT("EventLogTraceMessage_PIEStopped", "PIE Stopped") },
			};

			return FindLocalizedName(EventType, LocalizedNames, InExternalDisplayNames);
		}

		FText GetLocalizedCategoryName(const FString& Category, const TMap<FString, FText>& InExternalDisplayNames)
		{
			static const TMap<FString, FText> LocalizedNames =
			{
				{ CategorySoundActivity, LOCTEXT("EventLogCategory_SoundActivity", "Sound Activity") },
				{ CategoryVirtualization, LOCTEXT("EventLogCategory_Virtualization", "Virtualization") },
				{ CategoryPlayRequests,  LOCTEXT("EventLogCategory_PlayRequests", "Play Requests") },
				{ CategoryStopRequests,  LOCTEXT("EventLogCategory_StopRequests", "Stop Requests") },
				{ CategoryPlayErrors,    LOCTEXT("EventLogCategory_PlayErrors", "Play Errors") },
				{ CategoryMessages,      LOCTEXT("EventLogCategory_Messages", "Messages") },
				{ CategoryPIESession,    LOCTEXT("EventLogCategory_PIESession", "PIE Session") },
				{ CategoryCustom,        LOCTEXT("EventLogCategory_Custom", "Custom") },
			};

			return FindLocalizedName(Category, LocalizedNames, InExternalDisplayNames);
		}
	} // namespace AudioEventLogPrivate

	const TMap<FString, TSet<FString>> FAudioEventLogDashboardViewFactory::GetInitEventTypeFilters()
	{
		using namespace AudioEventLogPrivate;
		using namespace ::Audio::Trace::EventLog;

		return
		{
			{
				CategorySoundActivity,
				{
					ID::SoundStart,
					ID::SoundStop,
					ID::PauseSoundRequested,
					ID::ResumeSoundRequested
				}
			},

			{
				CategoryVirtualization,
				{
					ID::SoundVirtualized,
					ID::SoundRealized
				}
			},

			{
				CategoryPlayRequests,
				{
					ID::PlayRequestSoundHandle,
					ID::PlayRequestAudioComponent,
					ID::PlayRequestOneShot,
					ID::PlayRequestSoundAtLocation,
					ID::PlayRequestSound2D,
					ID::PlayRequestSlateSound
				}
			},

			{
				CategoryStopRequests,
				{
					ID::StopAllRequested,
					ID::StopRequestedSoundHandle,
					ID::StopRequestAudioComponent,
					ID::StopRequestActiveSound,
					ID::StopRequestSoundsUsingResource,
					ID::StopRequestConcurrency
				}
			},

			{
				CategoryPlayErrors,
				{
					ID::PlayFailedNotPlayable,
					ID::PlayFailedOutOfRange,
					ID::PlayFailedDebugFiltered,
					ID::PlayFailedConcurrency,
				}
			},

			{
				CategoryMessages,
				{
					ID::FlushAudioDeviceRequested
				}
			},

			{
				CategoryPIESession,
				{
					ID::PIEStarted,
					ID::PIEStopped
				}
			},

			{
				CategoryCustom,
				{
					// Auto-populates with custom events from users which haven't been categorized 
				}
			}
		};
	}

	/////////////////////////////////////////////////////////////////////////////////////////
	// FAudioEventLogDashboardViewFactory
	FAudioEventLogDashboardViewFactory::FAudioEventLogDashboardViewFactory()
		: InitEventTypeFilters(GetInitEventTypeFilters())
	{
		using namespace AudioEventLogPrivate;

		FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
		AudioInsightsTraceModule.OnAnalysisStarting.AddRaw(this, &FAudioEventLogDashboardViewFactory::OnAnalysisStarting);

		AudioEventLogTraceProvider = MakeShared<FAudioEventLogTraceProvider>();

		AudioInsightsTraceModule.AddTraceProvider(AudioEventLogTraceProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			AudioEventLogTraceProvider
		};

		SortByColumn = TimestampColumnID;
		SortMode = EColumnSortMode::Ascending;

		FAudioEventLogEditorCommands::Register();
		BindCommands();

#if WITH_EDITOR
		FAudioEventLogSettings::OnReadSettings.AddRaw(this, &FAudioEventLogDashboardViewFactory::OnReadEditorSettings);
		FAudioEventLogSettings::OnWriteSettings.AddRaw(this, &FAudioEventLogDashboardViewFactory::OnWriteEditorSettings);
#endif // WITH_EDITOR
	}

	FAudioEventLogDashboardViewFactory::~FAudioEventLogDashboardViewFactory()
	{
		if (FModuleManager::Get().IsModuleLoaded("AudioInsights"))
		{
			if (IModularFeatures::Get().IsModularFeatureAvailable(TraceServices::ModuleFeatureName))
			{
				FTraceModule& TraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
				TraceModule.OnAnalysisStarting.RemoveAll(this);
			}
			
		}

#if WITH_EDITOR
		FAudioEventLogSettings::OnReadSettings.RemoveAll(this);
		FAudioEventLogSettings::OnWriteSettings.RemoveAll(this);
#endif // WITH_EDITOR

		FAudioEventLogEditorCommands::Unregister();
	}

	void FAudioEventLogDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FAudioEventLogEditorCommands& Commands = FAudioEventLogEditorCommands::Get();

		CommandList->MapAction(Commands.GetResetInspectTimestampCommand(), FExecuteAction::CreateLambda([this]() { ResetInspectTimestamp(); }));

#if WITH_EDITOR
		CommandList->MapAction(Commands.GetBrowseCommand(), FExecuteAction::CreateLambda([this]() { BrowseToAsset(); }));
		CommandList->MapAction(Commands.GetEditCommand(), FExecuteAction::CreateLambda([this]() { OpenAsset(); }));
#endif // WITH_EDITOR
	}

	void FAudioEventLogDashboardViewFactory::ClearInspectState()
	{
		FocusedItem.Reset();
		bAutoScroll = true;

		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->ClearSelection();
			FilteredEntriesListView->ClearHighlightedItems();
		}
	}

	void FAudioEventLogDashboardViewFactory::ResetInspectTimestamp()
	{
		ClearInspectState();

		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().ResumeTimeMarker();
	}

	FName FAudioEventLogDashboardViewFactory::GetName() const
	{
		return "EventLog";
	}

	FText FAudioEventLogDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_AudioEventLog_Name", "Event Log");
	}

	EDefaultDashboardTabStack FAudioEventLogDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Log;
	}

	FSlateIcon FAudioEventLogDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Log");
	}

	FReply FAudioEventLogDashboardViewFactory::OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const
	{
		return (CommandList && CommandList->ProcessCommandBindings(KeyEvent)) ? FReply::Handled() : FReply::Unhandled();
	}

	TSharedRef<SWidget> FAudioEventLogDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().OnTimingViewTimeMarkerChanged.AddSP(this, &FAudioEventLogDashboardViewFactory::OnTimingViewTimeMarkerChanged);
		AudioInsightsModule.GetTimingViewExtender().OnTimeControlMethodReset.AddSP(this, &FAudioEventLogDashboardViewFactory::OnTimeControlMethodReset);

		if (!DashboardWidget.IsValid())
		{
			DashboardWidget = FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs);

			if (FilteredEntriesListView.IsValid())
			{
				FilteredEntriesListView->SetSelectionMode(ESelectionMode::SingleToggle);
				FilteredEntriesListView->OnMouseWheelDetected.AddSP(this, &FAudioEventLogDashboardViewFactory::OnUpdateAutoScroll);
			}

			if (HeaderRowWidget.IsValid())
			{
				VisibleColumnsSettingsMenu = MakeShared<FVisibleColumnsSettingsMenu<FAudioEventLogVisibleColumns>>(HeaderRowWidget.ToSharedRef(), FAudioEventLogVisibleColumns());

				VisibleColumnsSettingsMenu->OnVisibleColumnsSettingsUpdated.AddSPLambda(this, []()
				{
#if WITH_EDITOR
					FAudioEventLogSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
				});
			}
		}

#if WITH_EDITOR
		// Read the editor settings after the widget has been created
		FAudioEventLogSettings::OnRequestReadSettings.Broadcast();
#endif // WITH_EDITOR

		return DashboardWidget->AsShared();
	}

	void FAudioEventLogDashboardViewFactory::RefreshFilteredEntriesListView()
	{
		FTraceObjectTableDashboardViewFactory::RefreshFilteredEntriesListView();

		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		if (FilteredEntriesListView->IsUserScrolling())
		{
			return;
		}

		const TSharedPtr<IDashboardDataViewEntry> FocusedItemShared = FocusedItem.Pin();
		if (FocusedItemShared.IsValid())
		{
			FilteredEntriesListView->RequestScrollIntoView(FocusedItemShared);
		}
		else if (bAutoScroll)
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				FilteredEntriesListView->ScrollToBottom();
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				FilteredEntriesListView->ScrollToTop();
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason)
	{
		using namespace AudioEventLogPrivate;

		const FString FilterString = GetSearchFilterText().ToString();
		FTraceTableDashboardViewFactory::FilterEntries<FAudioEventLogTraceProvider>([this, &FilterString](const IDashboardDataViewEntry& Entry)
		{
			UpdateCustomEventLogFilters(Entry);

			const FAudioEventLogDashboardEntry& EventLogEntry = static_cast<const FAudioEventLogDashboardEntry&>(Entry);

			const FEventLogFilterID* FilterID = EventFilterTypes.Find(EventLogEntry.EventName);

			const bool bPassesEventTypeFilter = FilterID ? PassesEventTypeFilter(*FilterID) : true;

			if (bPassesEventTypeFilter)
			{
				if (FilterString.IsNumeric() && LexToString(EventLogEntry.PlayOrder).Equals(FilterString))
				{
					return true;
				}
				else if (EventLogEntry.GetDisplayName().ToString().Contains(FilterString) || EventLogEntry.ActorLabel.Contains(FilterString) || EventLogEntry.AudioComponentName.Contains(FilterString))
				{
					return true;
				}
			}

			return false;
		});

		if (Reason == EProcessReason::EntriesUpdated && FilteredEntriesListView.IsValid())
		{
			// Reset bCacheStatusIsDirty flag to push icon update to the UI
			for (const TSharedPtr<IDashboardDataViewEntry>& Entry : FilteredEntriesListView->GetItems())
			{
				if (!Entry.IsValid())
				{
					continue;
				}

				FAudioEventLogDashboardEntry& EventLogEntry = CastEntry(*Entry);
				if (EventLogEntry.CachedState == EAudioEventCacheState::NextToBeDeleted)
				{
					EventLogEntry.bCacheStatusIsDirty = false;
				}
				else
				{
					// Only the earliest items in the list will be marked as NextToBeDeleted - no need to process anymore items
					break;
				}
			}
		}
	}

	const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& FAudioEventLogDashboardViewFactory::GetColumns() const
	{
		using namespace AudioEventLogPrivate;

		auto CreateColumnData = [this]()
		{
			FAudioEventLogVisibleColumns VisibleColumns;
			if (VisibleColumnsSettingsMenu.IsValid())
			{
				VisibleColumnsSettingsMenu->WriteToSettings(VisibleColumns);
			}

			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					CacheStatusColumnID,
					{
						CacheStatusColumnDisplayName,
						[](const IDashboardDataViewEntry& InData) { return FText::GetEmpty(); }, /* GetDisplayValue */
						[](const IDashboardDataViewEntry& InData) -> FName
						{
							return FName("AudioInsights.Icon.EventLog.MarkedForDelete");
						},
						!VisibleColumns.GetIsVisible(CacheStatusColumnID) /* bDefaultHidden */,
						0.1f /* FillWidth */,
						HAlign_Center /* Alignment */,
						[this](const IDashboardDataViewEntry& InData) -> FLinearColor /* GetIconColor */
						{
							const FAudioEventLogDashboardEntry& Entry = CastEntry(InData);

							switch (Entry.CachedState)
							{
								case EAudioEventCacheState::NextToBeDeleted:
									return Entry.bCacheStatusIsDirty ? FLinearColor::Transparent : FLinearColor::White;
								default:
									break;
							}

							return FLinearColor::Transparent;
						},
						[](const IDashboardDataViewEntry& InData) -> FText /* GetIconTooltip */
						{
							const FAudioEventLogDashboardEntry& Entry = CastEntry(InData);

							switch (Entry.CachedState)
							{
								case EAudioEventCacheState::NextToBeDeleted:
									return LOCTEXT("AudioEventLog_NextToBeDeleted", "This event has been marked for deletion the next time the cache is full.");
								default:
									break;
							}

							return FText::GetEmpty();
						},
						FName("AudioInsights.Icon.EventLog.MarkedForDelete"), /* GetHeaderRowIconName */
						CacheStatusColumnDisplayName, /* HeaderRowTooltip */
						false /* bShowDisplayName */
					}
				},
				{
					MessageIDColumnID,
					{
						LOCTEXT("AudioEventLog_MessageIDColumnDisplayName", "ID"),
						[](const IDashboardDataViewEntry& InData) { return FText::AsNumber(CastEntry(InData).MessageID); },
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(MessageIDColumnID) /* bDefaultHidden */,
						0.1f /* FillWidth */
					}
				},
				{
					TimestampColumnID,
					{
						LOCTEXT("AudioEventLog_TimestampColumnDisplayName", "Timestamp"),
						[this](const IDashboardDataViewEntry& InData) { return FSlateStyle::Get().FormatTimestamp(GetTimestampRelativeToAnalysisStart(CastEntry(InData).Timestamp)); },
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(TimestampColumnID) /* bDefaultHidden */,
						0.2f /* FillWidth */
					}
				},
				{
					PlayOrderColumnID,
					{
						LOCTEXT("AudioEventLog_PlayOrderColumnDisplayName", "Play Order"),
						[](const IDashboardDataViewEntry& InData)
						{ 
							const uint32 PlayOrder = AudioEventLogPrivate::CastEntry(InData).PlayOrder;
							if (PlayOrder == INDEX_NONE)
							{
								return FText::GetEmpty();
							}

							return FText::AsNumber(AudioEventLogPrivate::CastEntry(InData).PlayOrder);
						},
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(PlayOrderColumnID) /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					EventColumnID,
					{
						LOCTEXT("AudioEventLog_EventColumnDisplayName", "Event"),
						[this](const IDashboardDataViewEntry& InData) { return GetLocalizedEventTypeName(CastEntry(InData).EventName, ExternalDisplayNames); },
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(EventColumnID) /* bDefaultHidden */,
						0.2f  /* FillWidth */
					}
				},
				{
					AssetNameColumnID,
					{
						LOCTEXT("AudioEventLog_AssetColumnDisplayName", "Asset"),
						[](const IDashboardDataViewEntry& InData) { return CastEntry(InData).GetDisplayName(); },
						[](const IDashboardDataViewEntry& InData) -> FName
						{
							const FAudioEventLogDashboardEntry& Entry = CastEntry(InData);

							switch (Entry.Category)
							{
								case EAudioEventLogSoundCategory::MetaSound:
									return FName("AudioInsights.Icon.SoundDashboard.MetaSound");
								case EAudioEventLogSoundCategory::SoundCue:
									return FName("AudioInsights.Icon.SoundDashboard.SoundCue");
								case EAudioEventLogSoundCategory::ProceduralSource:
									return FName("AudioInsights.Icon.SoundDashboard.ProceduralSource");
								case EAudioEventLogSoundCategory::SoundWave:
									return FName("AudioInsights.Icon.SoundDashboard.SoundWave");
								case EAudioEventLogSoundCategory::SoundCueTemplate:
									return FName("AudioInsights.Icon.SoundDashboard.SoundCue");
								case EAudioEventLogSoundCategory::None:
								default:
									break;
							}

							return NAME_None;
						},
						!VisibleColumns.GetIsVisible(AssetNameColumnID) /* bDefaultHidden */,
						0.3f  /* FillWidth */
					}
				},
				{
					ActorColumnID,
					{
						LOCTEXT("AudioEventLog_ActorColumnDisplayName", "Actor"),
						[](const IDashboardDataViewEntry& InData) { return FText::FromString(CastEntry(InData).ActorLabel); },
						[](const IDashboardDataViewEntry& InData) -> FName
						{
							return CastEntry(InData).ActorIconName;
						},
						!VisibleColumns.GetIsVisible(ActorColumnID) /* bDefaultHidden */,
						0.3f  /* FillWidth */
					}
				},
				{
					AudioComponentColumnID,
					{
						LOCTEXT("AudioEventLog_AudioComponentColumnDisplayName", "Audio Component"),
						[](const IDashboardDataViewEntry& InData) { return FText::FromString(CastEntry(InData).AudioComponentName); },
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(AudioComponentColumnID) /* bDefaultHidden */,
						0.25f /* FillWidth */
					}
				},
				{
					CategoryColumnID,
					{
						LOCTEXT("AudioEventLog_CategoryColumnDisplayName", "Category"),
						[](const IDashboardDataViewEntry& InData) { return FText::FromString(CastEntry(InData).CategoryName); },
						nullptr /* GetIconName */,
						!VisibleColumns.GetIsVisible(CategoryColumnID) /* bDefaultHidden */,
						0.3f  /* FillWidth */
					}
				}
			};
		};

		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		return ColumnData;
	}

	TSharedRef<SWidget> FAudioEventLogDashboardViewFactory::GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column)
	{
		// Wrap each column in an SBox with a small margin on the right
		// This helps distinguish between columns where the contents
		// overflow the column width
		return SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				FTraceObjectTableDashboardViewFactory::GenerateWidgetForColumn(InRowData, Column)
			];
	}

	void FAudioEventLogDashboardViewFactory::SortTable()
	{
		using namespace AudioEventLogPrivate;

		auto SortByMessageID = [](const FAudioEventLogDashboardEntry& First, const FAudioEventLogDashboardEntry& Second)
		{
			return First.MessageID < Second.MessageID;
		};

		auto SortByTimestamp = [&SortByMessageID](const FAudioEventLogDashboardEntry& First, const FAudioEventLogDashboardEntry& Second)
		{
			const double ComparisonDiff = First.Timestamp - Second.Timestamp;

			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByMessageID(First, Second);
			}

			return ComparisonDiff < 0.0;
		};

		SortByPredicate(SortByTimestamp);
	}

	bool FAudioEventLogDashboardViewFactory::IsColumnSortable(const FName& ColumnId) const
	{
		using namespace AudioEventLogPrivate;

		return ColumnId == TimestampColumnID;
	}

	void FAudioEventLogDashboardViewFactory::OnHiddenColumnsListChanged()
	{
		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->OnHiddenColumnsListChanged();
		}
	}

	TSharedPtr<SWidget> FAudioEventLogDashboardViewFactory::CreateFilterBarButtonWidget()
	{
		if (!FilterBarButton.IsValid())
		{
			CreateFilterBarWidget();
			FilterBarButton = SBasicFilterBar<FEventLogFilterID>::MakeAddFilterButton(StaticCastSharedPtr<SAudioFilterBar<FEventLogFilterID>>(FilterBar).ToSharedRef()).ToSharedPtr();
		}

		return FilterBarButton;
	}

	TSharedRef<SWidget> FAudioEventLogDashboardViewFactory::CreateFilterBarWidget()
	{
		using namespace AudioEventLogPrivate;

		if (!FilterBar.IsValid())
		{
			TArray<TSharedRef<FFilterBase<FEventLogFilterID>>> Filters;

			for (const auto& [Category, EventTypes] : InitEventTypeFilters)
			{
				const TSharedPtr<FFilterCategory>& FilterCategory = FindOrAddFilterCategory(Category);
				for (const FString& EventType : EventTypes)
				{
					Filters.Add(CreateNewEventFilterType(FilterCategory, EventType));
				}
			}

			for (const auto& [Category, EventTypes] : ExternalEventTypeFilters)
			{
				const TSharedPtr<FFilterCategory>& FilterCategory = FindOrAddFilterCategory(Category);
				for (const FString& EventType : EventTypes)
				{
					Filters.Add(CreateNewEventFilterType(FilterCategory, EventType));
				}
			}

			SAssignNew(FilterBar, SAudioFilterBar<FEventLogFilterID>)
			.CustomFilters(Filters)
			.UseSectionsForCategories(false)
			.Visibility(EVisibility::Collapsed) // Filter bar buttons intentionally hidden in UI
			.OnFilterChanged_Lambda([this]()
			{
				UpdateFilterReason = EProcessReason::FilterUpdated;
			});
		}

		return FilterBar.ToSharedRef();
	}

	TSharedRef<SWidget> FAudioEventLogDashboardViewFactory::CreateSettingsButtonWidget()
	{
		if (!SettingsAreaWidget.IsValid())
		{
			constexpr FLinearColor StatTitleColor(1.0f, 1.0f, 1.0f, 0.5f);
			SAssignNew(SettingsAreaWidget, SHorizontalBox)

			// Clear log Button
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(15.0f, 0.0f, 6.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(AudioEventLogPrivate::ClearLogTooltipText)
				.IsEnabled_Lambda([]()
				{
					return AudioEventLogPrivate::CanClearLog();
				})
				.OnClicked_Lambda([this]()
				{
					ClearActiveAudioDeviceEntries();

					return FReply::Handled();
				})
				[
					SNew(SBox)
					[
						SNew(SHorizontalBox)
						// Clear log icon
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.ClearLog"))
						]
						// Clear log text
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(6.0f, 1.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text(AudioEventLogPrivate::ClearLogText)
						]
					]
				]
			]

			// Settings button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Right)
			[
				FTraceObjectTableDashboardViewFactory::CreateSettingsButtonWidget()
			];
		}

		return SettingsAreaWidget.ToSharedRef();
	}

	TSharedRef<SWidget> FAudioEventLogDashboardViewFactory::OnGetSettingsMenuContent()
	{
		FMenuBuilder MenuBuilder(false /*bShouldCloseWindowAfterMenuSelection*/, CommandList);

		MenuBuilder.BeginSection("AudioEventLogSettingActions", LOCTEXT("AudioEventLog_Settings_HeaderText", "Settings"));
		{
			// Visible columns
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("AudioEventLog_Settings_VisibleColumns", "Visible Columns"),
				LOCTEXT("AudioEventLog_Settings_VisibleColumnsTooltip", "Show/hide columns"),
				FNewMenuDelegate::CreateRaw(this, &FAudioEventLogDashboardViewFactory::BuildVisibleColumnsMenuContent)
			);
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	FText FAudioEventLogDashboardViewFactory::OnGetSettingsMenuToolTip()
	{
		return LOCTEXT("AudioEventLog_Settings_TooltipText", "Event log settings");
	}

	TSharedPtr<SWidget> FAudioEventLogDashboardViewFactory::OnConstructContextMenu()
	{
		if (DataViewEntries.IsEmpty())
		{
			return nullptr;
		}

		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

#if WITH_EDITOR
		const bool bHasSelectedAsset = FilteredEntriesListView.IsValid()
			&& (FilteredEntriesListView->GetSelectedItems().Num() > 0)
			&& !GetSelectedEditableAssets().IsEmpty();

		if (bHasSelectedAsset)
		{
			const FAudioEventLogEditorCommands& Commands = FAudioEventLogEditorCommands::Get();

			MenuBuilder.BeginSection("EventLogDashboardActions", LOCTEXT("EventLogDashboard_Actions_HeaderText", "Editor Options"));
			{
				MenuBuilder.AddMenuEntry(Commands.GetBrowseCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Browse"));
				MenuBuilder.AddMenuEntry(Commands.GetEditCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Edit"));
			}

			MenuBuilder.EndSection();

			MenuBuilder.AddSeparator();
		}
#endif // WITH_EDITOR

		MenuBuilder.AddMenuEntry(
			AudioEventLogPrivate::ClearLogText,
			AudioEventLogPrivate::ClearLogTooltipText,
			FSlateStyle::Get().CreateIcon("AudioInsights.Icon.ClearLog"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAudioEventLogDashboardViewFactory::ClearActiveAudioDeviceEntries),
				FCanExecuteAction::CreateLambda([]()
				{
					return AudioEventLogPrivate::CanClearLog();
				})
			)
		);

		return MenuBuilder.MakeWidget();
	}

	void FAudioEventLogDashboardViewFactory::ClearActiveAudioDeviceEntries()
	{
		if (AudioEventLogTraceProvider.IsValid())
		{
			AudioEventLogTraceProvider->ClearActiveAudioDeviceEntries();
			DataViewEntries.Reset();

			if (FilteredEntriesListView.IsValid())
			{
				RefreshFilteredEntriesListView();
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::OnUpdateAutoScroll()
	{
		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		const TArrayView<const TSharedPtr<IDashboardDataViewEntry>> FilteredItems = FilteredEntriesListView->GetItems();
		const int32 NumItems = FilteredItems.Num();
		if (NumItems == 0)
		{
			return;
		}

		if (SortMode == EColumnSortMode::Ascending)
		{
			bAutoScroll = FilteredEntriesListView->IsItemVisible(FilteredItems[NumItems - 1]);
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			bAutoScroll = FilteredEntriesListView->IsItemVisible(FilteredItems[0]);
		}
	}

	double FAudioEventLogDashboardViewFactory::GetTimestampRelativeToAnalysisStart(const double Timestamp) const
	{
		return Timestamp - BeginTimestamp;
	}

	void FAudioEventLogDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		using namespace AudioEventLogPrivate;

		if (SelectInfo != ESelectInfo::Type::Direct)
		{
			FTraceObjectTableDashboardViewFactory::OnSelectionChanged(SelectedItem, SelectInfo);
		}

		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		if (!SelectedItem.IsValid())
		{
			ResetInspectTimestamp();
			return;
		}

		FocusedItem = SelectedItem;

		const FAudioEventLogDashboardEntry& SelectedEventLogEntry = CastEntry(*SelectedItem);
		if (SelectedEventLogEntry.PlayOrder == INDEX_NONE)
		{
			FilteredEntriesListView->ClearHighlightedItems();
		}
		else
		{
			for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
			{
				if (!Entry.IsValid())
				{
					continue;
				}

				FilteredEntriesListView->SetItemHighlighted(Entry, CastEntry(*Entry).PlayOrder == SelectedEventLogEntry.PlayOrder);
			}
		}

		if (SelectInfo != ESelectInfo::Type::Direct)
		{
			FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			AudioInsightsModule.GetTimingViewExtender().PauseTimeMarker(SelectedEventLogEntry.Timestamp, ESystemControllingTimeMarker::EventLog);
		}
	}

	void FAudioEventLogDashboardViewFactory::OnListViewScrolled(double InScrollOffset)
	{
		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		if (FilteredEntriesListView->IsUserScrolling())
		{
			OnUpdateAutoScroll();
		}
	}

	void FAudioEventLogDashboardViewFactory::OnFinishedScrolling()
	{
		if (FilteredEntriesListView.IsValid() && FilteredEntriesListView->GetNumItemsBeingObserved() > 0)
		{
			const TSharedPtr<IDashboardDataViewEntry> FocusedItemShared = FocusedItem.Pin();
			
			// Only set a focused item if the user has previously focused on a row
			if (FocusedItemShared.IsValid())
			{
				for (const TSharedPtr<IDashboardDataViewEntry>& Entry : FilteredEntriesListView->GetItems())
				{
					if (FilteredEntriesListView->IsItemVisible(Entry))
					{
						FocusedItem = Entry;
						break;
					}
				}
			}
		}
	}

	const FTableRowStyle* FAudioEventLogDashboardViewFactory::GetRowStyle() const
	{
		return &FSlateStyle::Get().GetWidgetStyle<FTableRowStyle>("TreeDashboard.TableViewRow");
	}

	void FAudioEventLogDashboardViewFactory::OnAnalysisStarting(const double Timestamp)
	{
#if WITH_EDITOR
		BeginTimestamp = Timestamp - GStartTime;
#else
		BeginTimestamp = 0.0;
#endif // WITH_EDITOR
	}

	void FAudioEventLogDashboardViewFactory::UpdateCustomEventLogFilters(const IDashboardDataViewEntry& Entry)
	{
		using namespace AudioEventLogPrivate;

		const FAudioEventLogDashboardEntry& EventLogEntry = CastEntry(Entry);

		if (!EventFilterTypes.Contains(EventLogEntry.EventName))
		{
			CreateFilterBarWidget();

			const TSharedPtr<FFilterCategory>& CustomCategory = FindOrAddFilterCategory(CategoryCustom);
			FilterBar->AddFilter(CreateNewEventFilterType(CustomCategory, EventLogEntry.EventName));
		}
	}

	TSharedRef<FAudioEventLogFilter> FAudioEventLogDashboardViewFactory::CreateNewEventFilterType(const TSharedPtr<FFilterCategory>& FilterCategory, const FString& EventType)
	{
		using namespace AudioEventLogPrivate;

		static FEventLogFilterID FilterID = 0;

		TSharedPtr<FAudioEventLogFilter> Filter = MakeShared<FAudioEventLogFilter>(FilterID,
																				   EventType,
																				   GetLocalizedEventTypeName(EventType, ExternalDisplayNames),
																				   NAME_None,
																				   FText::GetEmpty(),
																				   FLinearColor::MakeRandomColor(),
																				   FilterCategory);
		EventFilterTypes.Add(EventType, FilterID++);
		return Filter.ToSharedRef();
	}

	const TSharedPtr<FFilterCategory>& FAudioEventLogDashboardViewFactory::FindOrAddFilterCategory(const FString& CategoryName)
	{
		using namespace AudioEventLogPrivate;

		return FilterCategories.FindOrAdd(CategoryName, MakeShared<FFilterCategory>(GetLocalizedCategoryName(CategoryName, ExternalDisplayNames), FText::GetEmpty()));
	}

	bool FAudioEventLogDashboardViewFactory::PassesEventTypeFilter(const FEventLogFilterID EventLogID) const
	{
		if (!FilterBar.IsValid())
		{
			return true;
		}

		TSharedPtr<TFilterCollection<FEventLogFilterID>> ActiveFilters = FilterBar->GetAllActiveFilters();
		
		if (!ActiveFilters.IsValid() || ActiveFilters->Num() == 0)
		{
			return true;
		}
		
		for (const TSharedPtr<IFilter<FEventLogFilterID>>& Filter : *ActiveFilters.Get())
		{
			if (Filter->PassesFilter(EventLogID))
			{
				return true;
			}
		}

		return false;
	}

	void FAudioEventLogDashboardViewFactory::BuildVisibleColumnsMenuContent(FMenuBuilder& MenuBuilder)
	{
		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->BuildVisibleColumnsMenuContent(MenuBuilder);
		}
	}

	void FAudioEventLogDashboardViewFactory::RegisterExternalEventTypes(const TMap<FString, TSet<FString>>& InCategoriesToEvents)
	{
		using namespace AudioEventLogPrivate;

		check(IsInGameThread());

		for (const auto& [CategoryName, EventTypes] : InCategoriesToEvents)
		{
			ExternalEventTypeFilters.FindOrAdd(CategoryName).Append(EventTypes);

			if (FilterBar.IsValid() && !EventTypes.IsEmpty())
			{
				const TSharedPtr<FFilterCategory>& FilterCategory = FindOrAddFilterCategory(CategoryName);

				for (const FString& EventType : EventTypes)
				{
					if (!EventFilterTypes.Contains(EventType))
					{
						FilterBar->AddFilter(CreateNewEventFilterType(FilterCategory, EventType));
					}
				}
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::RegisterExternalDisplayNames(const TMap<FString, FText>& InDisplayNames)
	{
		check(IsInGameThread());

		ExternalDisplayNames.Append(InDisplayNames);
	}

	void FAudioEventLogDashboardViewFactory::SortByPredicate(TFunctionRef<bool(const FAudioEventLogDashboardEntry&, const FAudioEventLogDashboardEntry&)> Predicate)
	{
		using namespace AudioEventLogPrivate;

		auto SortDashboardEntries = [this](const TSharedPtr<IDashboardDataViewEntry>& First, const TSharedPtr<IDashboardDataViewEntry>& Second, TFunctionRef<bool(const FAudioEventLogDashboardEntry&, const FAudioEventLogDashboardEntry&)> Predicate)
		{
			return Predicate(CastEntry(*First), CastEntry(*Second));
		};

		if (SortMode == EColumnSortMode::Ascending)
		{
			DataViewEntries.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
			{
				return SortDashboardEntries(A, B, Predicate);
			});
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			DataViewEntries.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
			{
				return SortDashboardEntries(B, A, Predicate);
			});
		}
	}

#if WITH_EDITOR
	void FAudioEventLogDashboardViewFactory::OnReadEditorSettings(const FAudioEventLogSettings& InSettings)
	{
		if (FilterBar.IsValid())
		{
			RemoveDeletedCustomEvents(InSettings.CustomCategoriesToEvents);
			AddNewCustomEvents(InSettings.CustomCategoriesToEvents);

			CachedCustomEventSettings = InSettings.CustomCategoriesToEvents;
			
			RefreshFilterBarFromSettings(InSettings.EventFilters);
		}

		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->ReadFromSettings(InSettings.VisibleColumns);
		}
	}

	void FAudioEventLogDashboardViewFactory::OnWriteEditorSettings(FAudioEventLogSettings& OutSettings)
	{
		if (FilterBar.IsValid())
		{
			TSharedPtr<TFilterCollection<FEventLogFilterID>> ActiveFilters = FilterBar->GetAllActiveFilters();

			if (!ActiveFilters.IsValid() || ActiveFilters->Num() == 0)
			{
				OutSettings.EventFilters.Empty();
			}
			else
			{
				for (const auto& [EventName, FilterID] : EventFilterTypes)
				{
					if (PassesEventTypeFilter(FilterID))
					{
						OutSettings.EventFilters.Add(EventName);
					}
					else
					{
						OutSettings.EventFilters.Remove(EventName);
					}
				}
			}
		}

		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->WriteToSettings(OutSettings.VisibleColumns);
		}
	}

	void FAudioEventLogDashboardViewFactory::RemoveDeletedCustomEvents(const TMap<FString, FAudioEventLogCustomEvents>& CustomEventsFromSettings)
	{
		if (!FilterBar.IsValid())
		{
			return;
		}

		// Run through our cached custom events, remove any that are not in the settings
		for (const auto& [CachedCategory, CachedEventTypes] : CachedCustomEventSettings)
		{
			// Can be null if the category has been deleted
			const FAudioEventLogCustomEvents* FoundEventTypesFromSettingsCategory = CustomEventsFromSettings.Find(CachedCategory);

			int32 NumRemoved = 0;
			for (const FString& CachedEvent : CachedEventTypes.EventNames)
			{
				if (FoundEventTypesFromSettingsCategory == nullptr || !FoundEventTypesFromSettingsCategory->EventNames.Contains(CachedEvent))
				{
					// The category or event has been destroyed, remove it here
					EventFilterTypes.Remove(CachedEvent);
					NumRemoved++;

					if (const TSharedPtr<FFilterBase<FEventLogFilterID>> Filter = FilterBar->GetFilter(CachedEvent))
					{
						FilterBar->DeleteFromFilter(Filter.ToSharedRef());
					}
				}
			}

			// If every event in this category has been deleted, remove the category too
			if (NumRemoved == CachedEventTypes.EventNames.Num())
			{
				if (const TSharedPtr<FFilterCategory>* FilterCategory = FilterCategories.Find(CachedCategory))
				{
					FilterBar->DeleteCategory(FilterCategory->ToSharedRef());
					FilterCategories.Remove(CachedCategory);
				}
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::AddNewCustomEvents(const TMap<FString, FAudioEventLogCustomEvents>& CustomEventsFromSettings)
	{
		using namespace AudioEventLogPrivate;

		if (!FilterBar.IsValid())
		{
			return;
		}

		// Run through all of the events in the settings - detect any new ones and add them to the filter
		for (const auto& [SettingsCategoryName, SettingsEventTypes] : CustomEventsFromSettings)
		{
			if (SettingsEventTypes.EventNames.IsEmpty())
			{
				// Do not re-add the category if the event name collection is empty
				continue;
			}

			// Can be null if the category does not exist yet
			const FAudioEventLogCustomEvents* FoundEventTypesFromCachedCategory = CachedCustomEventSettings.Find(SettingsCategoryName);

			// Find the category if it exists, or create a new one if it doesn't
			const TSharedPtr<FFilterCategory>& FilterCategory = FindOrAddFilterCategory(SettingsCategoryName);

			for (const FString& SettingsEvent : SettingsEventTypes.EventNames)
			{
				if (FoundEventTypesFromCachedCategory == nullptr || !FoundEventTypesFromCachedCategory->EventNames.Contains(SettingsEvent))
				{
					// This is a new category or event, add the event to the filter
					FilterBar->AddFilter(CreateNewEventFilterType(FilterCategory, SettingsEvent));
				}
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::RefreshFilterBarFromSettings(const TSet<FString>& EventFilters)
	{
		if (!FilterBar.IsValid())
		{
			return;
		}

		if (EventFilters.IsEmpty())
		{
			FilterBar->RemoveAllFilters();
		}
		else
		{
			for (const auto& [EventName, FilterID] : EventFilterTypes)
			{
				const bool bFilterIsActive = EventFilters.Contains(EventName);
				FilterBar->SetFilterCheckState(FilterBar->GetFilter(EventName), bFilterIsActive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
			}
		}
	}
#endif // WITH_EDITOR


	void FAudioEventLogDashboardViewFactory::OnTimingViewTimeMarkerChanged(double InTimeMarker)
	{
		using namespace AudioEventLogPrivate;

		bAutoScroll = false;

		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		// We assume the data entries are in timestamp order (earliest to latest)
		for (int32 Index = 0; Index < DataViewEntries.Num(); ++Index)
		{
			const TSharedPtr<IDashboardDataViewEntry>& Entry = DataViewEntries[Index];
			if (!Entry.IsValid())
			{
				continue;
			}

			const FAudioEventLogDashboardEntry& EventLogEntry = CastEntry(*Entry);
			if (EventLogEntry.Timestamp == InTimeMarker)
			{
				FilteredEntriesListView->SetSelection(Entry);
				break;
			}
			else if (EventLogEntry.Timestamp > InTimeMarker)
			{
				const int32 SelectedIndex = Index - 1;
				if (SelectedIndex < 0)
				{
					FilteredEntriesListView->ClearSelection();
				}
				else
				{
					FilteredEntriesListView->SetSelection(DataViewEntries[SelectedIndex]);
				}

				break;
			}
			// If we've reached the last entry, the timemarker must be beyond the final entry
			// Select the last entry in the list in this case
			else if (Index == DataViewEntries.Num() - 1)
			{
				FilteredEntriesListView->SetSelection(Entry);
			}
		}
	}

	void FAudioEventLogDashboardViewFactory::OnTimeControlMethodReset()
	{
		ClearInspectState();
	}

} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
