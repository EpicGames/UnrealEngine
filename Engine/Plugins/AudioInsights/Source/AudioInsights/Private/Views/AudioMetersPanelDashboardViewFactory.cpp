// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/AudioMetersPanelDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "Providers/AudioMeterProvider.h"
#include "Views/SAudioMeterView.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FAudioMetersPanelDashboardViewFactory::FAudioMetersPanelDashboardViewFactory()
	{
		FAudioMeterProvider::OnAddAudioMeter.BindRaw(this, &FAudioMetersPanelDashboardViewFactory::HandleOnAddAudioMeter);
		FAudioMeterProvider::OnRemoveAudioMeter.BindRaw(this, &FAudioMetersPanelDashboardViewFactory::HandleOnRemoveAudioMeter);
		FAudioMeterProvider::OnUpdateAudioMeterInfo.BindRaw(this, &FAudioMetersPanelDashboardViewFactory::HandleOnUpdateAudioMeterInfo);
	}

	FAudioMetersPanelDashboardViewFactory::~FAudioMetersPanelDashboardViewFactory()
	{
		FAudioMeterProvider::OnAddAudioMeter.Unbind();
		FAudioMeterProvider::OnRemoveAudioMeter.Unbind();
		FAudioMeterProvider::OnUpdateAudioMeterInfo.Unbind();
	}

	FName FAudioMetersPanelDashboardViewFactory::GetName() const
	{
		return "AudioMetersPanel";
	}

	FText FAudioMetersPanelDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_AudioMetersPanelTab_DisplayName", "Audio Meters");
	}

	EDefaultDashboardTabStack FAudioMetersPanelDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::AudioMeters;
	}

	FSlateIcon FAudioMetersPanelDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Submix");
	}

	void FAudioMetersPanelDashboardViewFactory::AddAudioMeterView(const TSharedRef<SAudioMeterView>& InAudioMeterView)
	{
		if (AudioMeterWidgetsContainer.IsValid())
		{
			AudioMeterWidgetsContainer->AddSlot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 10.0f, 0.0f)
			[
				InAudioMeterView
			];
		}
	}

	TSharedRef<SWidget> FAudioMetersPanelDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!AudioMeterWidgetsScrollBox.IsValid())
		{
			SAssignNew(AudioMeterWidgetsContainer, SHorizontalBox);

			// Add any AudioMeterView that could have been added in HandleOnAddAudioMeter while the dashboard was not opened
			for (const auto& [EntryId, AudioMeterView] : AudioMeterViews)
			{
				AddAudioMeterView(AudioMeterView);
			}

			SAssignNew(AudioMeterWidgetsScrollBox, SScrollBox)
			.Orientation(Orient_Horizontal)
			+ SScrollBox::Slot()
			[
				AudioMeterWidgetsContainer.ToSharedRef()
			];
		}

		return AudioMeterWidgetsScrollBox.ToSharedRef();
	}

	void FAudioMetersPanelDashboardViewFactory::HandleOnAddAudioMeter(const uint32 InEntryId, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo, const FText& InName, const FSlateColor& InNameColor)
	{
		if (AudioMeterViews.Contains(InEntryId))
		{
			return;
		}

		const TSharedRef<SAudioMeterView>& AudioMeterView = AudioMeterViews.Emplace(InEntryId, SNew(SAudioMeterView, InAudioMeterInfo)
			.NameText(InName)
			.NameTextColor(InNameColor));

		AddAudioMeterView(AudioMeterView);

		if (AudioMeterWidgetsScrollBox.IsValid())
		{
			AudioMeterWidgetsScrollBox->Invalidate(EInvalidateWidget::Layout);
		}
	}

	void FAudioMetersPanelDashboardViewFactory::HandleOnRemoveAudioMeter(const uint32 InEntryId)
	{
		const TSharedRef<SAudioMeterView>* FoundAudioMeterView = AudioMeterViews.Find(InEntryId);

		if (FoundAudioMeterView == nullptr)
		{
			return;
		}

		if (AudioMeterWidgetsContainer.IsValid())
		{
			AudioMeterWidgetsContainer->RemoveSlot(*FoundAudioMeterView);
		}

		AudioMeterViews.Remove(InEntryId);

		if (AudioMeterWidgetsScrollBox.IsValid())
		{
			AudioMeterWidgetsScrollBox->Invalidate(EInvalidateWidget::Layout);
		}
	}

	void FAudioMetersPanelDashboardViewFactory::HandleOnUpdateAudioMeterInfo(const uint32 InEntryId, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo, const FText& InName)
	{
		if (TSharedRef<SAudioMeterView>* FoundAudioMeterView = AudioMeterViews.Find(InEntryId))
		{
			(*FoundAudioMeterView)->SetAudioMeterChannelInfo(InAudioMeterInfo);
			(*FoundAudioMeterView)->SetName(InName);

			if (AudioMeterWidgetsScrollBox.IsValid())
			{
				AudioMeterWidgetsScrollBox->Invalidate(EInvalidateWidget::Layout);
			}
		}
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
