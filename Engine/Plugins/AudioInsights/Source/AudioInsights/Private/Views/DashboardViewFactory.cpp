// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/DashboardViewFactory.h"

namespace UE::Audio::Insights
{
	FTraceDashboardViewFactoryBase::FTraceDashboardViewFactoryBase()
	{
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("TraceDashboardViewFactoryBase"), 0.0f, [this](float DeltaTime)
		{
			Tick(DeltaTime);
			return true;
		});
	}

	FTraceDashboardViewFactoryBase::~FTraceDashboardViewFactoryBase()
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

	void FTraceDashboardViewFactoryBase::Tick(float InElapsed)
	{
		for (const TSharedPtr<FTraceProviderBase>& Provider : Providers)
		{
			if (!Provider.IsValid())
			{
				continue;
			}

			if (Provider->ShouldForceUpdate())
			{
				Provider->ResetShouldForceUpdate();
 
				UpdateFilterReason = EProcessReason::EntriesUpdated;
			}
			else if (const uint64* CurrentUpdateId = UpdateIds.Find(Provider->GetName()))
			{
				if (*CurrentUpdateId != Provider->GetLastUpdateId())
				{
					UpdateFilterReason = EProcessReason::EntriesUpdated;
				}
			}
			else
			{
				UpdateFilterReason = EProcessReason::EntriesUpdated;
			}
		}

		if (UpdateFilterReason != EProcessReason::None)
		{
			ProcessEntries(UpdateFilterReason);
			if (UpdateFilterReason == EProcessReason::EntriesUpdated)
			{
				for (TSharedPtr<FTraceProviderBase> Provider : Providers)
				{
					if (!Provider.IsValid())
					{
						continue;
					}

					const FName ProviderName = Provider->GetName();
					const uint64 LastUpdateId = Provider->GetLastUpdateId();
					UpdateIds.FindOrAdd(ProviderName) = LastUpdateId;
				}
			}

			RefreshFilteredEntriesListView();
			UpdateFilterReason = EProcessReason::None;
		}

#if WITH_EDITOR
		UpdateDebugDraw(InElapsed);
#endif // WITH_EDITOR
	}
}