// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/LogDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "OutputLogModule.h"

#define LOCTEXT_NAMESPACE "AudioInsights"


namespace UE::Audio::Insights
{
	namespace LogDashboardViewFactoryPrivate
	{
		static const TSet<FString> DefaultCategoryKeywords = { TEXT("Audio"), TEXT("Sound") };

		bool IsDefaultDashboardCategory(const FName InCategoryName)
		{
			const FString CategoryNameStr = InCategoryName.ToString();
			for (const FString& Keyword : DefaultCategoryKeywords)
			{
				if (CategoryNameStr.Contains(Keyword))
				{
					return true;
				}
			}

			return false;
		}
	} // namespace LogDashboardViewFactoryPrivate

	FLogDashboardViewFactory::FLogDashboardViewFactory() = default;
	FLogDashboardViewFactory::~FLogDashboardViewFactory() = default;

	FName FLogDashboardViewFactory::GetName() const
	{
		return "Log";
	}

	FText FLogDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_LogTab_DisplayName", "Log");
	}

	EDefaultDashboardTabStack FLogDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Log;
	}

	FSlateIcon FLogDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Log");
	}

	TSharedRef<SWidget> FLogDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		using namespace LogDashboardViewFactoryPrivate;

		FOutputLogCreationParams Params;
		Params.SettingsMenuCreationFlags = EOutputLogSettingsMenuFlags::SkipClearOnPie
			| EOutputLogSettingsMenuFlags::SkipOpenSourceButton
			| EOutputLogSettingsMenuFlags::SkipEnableWordWrapping;

		Params.DefaultCategorySelection = CategoryCollector.GetCollectedCategories();
		Params.AllowAsInitialLogCategory = FAllowLogCategoryCallback::CreateStatic(&IsDefaultDashboardCategory);

		FModuleManager::LoadModuleChecked<FOutputLogModule>("OutputLog");
		return FOutputLogModule::Get().MakeOutputLogWidget(Params);
	}

	FLogDashboardViewFactory::FLogCategoryCollector::FLogCategoryCollector()
	{
		if (GLog)
		{
			GLog->AddOutputDevice(this);
		}
	}

	FLogDashboardViewFactory::FLogCategoryCollector::~FLogCategoryCollector()
	{
		if (GLog)
		{
			GLog->RemoveOutputDevice(this);
		}
	}

	FDefaultCategorySelectionMap FLogDashboardViewFactory::FLogCategoryCollector::GetCollectedCategories() const
	{
		FScopeLock CollectionLock(&CollectionCritSec);
		return CollectedCategories;
	}

	void FLogDashboardViewFactory::FLogCategoryCollector::Serialize(const TCHAR* InMsg, ELogVerbosity::Type Verbosity, const FName& InCategory)
	{
		using namespace LogDashboardViewFactoryPrivate;

		FScopeLock CollectionLock(&CollectionCritSec);

		const bool bIsDefaultCategory = IsDefaultDashboardCategory(InCategory);
		CollectedCategories.FindOrAdd(InCategory) = bIsDefaultCategory;
	}

	bool FLogDashboardViewFactory::FLogCategoryCollector::IsMemoryOnly() const
	{
		return true;
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
