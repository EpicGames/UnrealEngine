// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/OutputMeterDashboardViewFactory.h"

#include "Algo/Find.h"
#include "AudioDefines.h"
#include "AudioInsightsComponent.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsSettings.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceModule.h"
#include "DSP/Dsp.h"
#include "LoudnessMeterSettings.h"
#include "Providers/AudioMeterProvider.h"
#include "Providers/OutputMeterTraceProvider.h"
#include "Providers/SubmixTraceProvider.h"
#include "SAudioMeterWidget.h"
#include "Views/SubmixDashboardViewFactory.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FOutputMeterDashboardViewFactoryPrivate
	{
		const FOutputMeterDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FOutputMeterDashboardEntry&>(InData);
		};
	} // namespace FOutputMeterDashboardViewFactoryPrivate


	FOutputMeterDashboardViewFactory::FOutputMeterDashboardViewFactory(TSharedRef<FSubmixDashboardViewFactory> SubmixDashboard)
		: FTraceDashboardViewFactoryBase()
		, SubmixProvider(SubmixDashboard->FindProvider<FSubmixTraceProvider>())
		, AudioMeterInfo(MakeUnique<FAudioMeterInfo>())
	{
		FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());

		OutputMeterProvider = MakeShared<FOutputMeterTraceProvider>();

		AudioInsightsTraceModule.AddTraceProvider(OutputMeterProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			OutputMeterProvider
		};
	}

	FOutputMeterDashboardViewFactory::~FOutputMeterDashboardViewFactory() = default;

	EDefaultDashboardTabStack FOutputMeterDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::OutputMetering;
	}

	FText FOutputMeterDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_DashboardsOutputMeteringTab_DisplayName", "Output Metering");
	}

	FName FOutputMeterDashboardViewFactory::GetName() const
	{
		return "OutputMeter";
	}

	FSlateIcon FOutputMeterDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon");
	}

	FLoudnessMeterSettings& FOutputMeterDashboardViewFactory::GetLoudnessMeterSettings() const
	{
		return GetMutableDefault<UAudioInsightsSettings>()->OutputMeterSettings;
	}

	FOutputMeterDashboardSettings& FOutputMeterDashboardViewFactory::GetOutputMeterDashboardSettings()
	{
		return GetMutableDefault<UAudioInsightsSettings>()->OutputMeterDashboardSettings;
	}

	void FOutputMeterDashboardViewFactory::SaveOutputMeterDashboardSettings()
	{
		const FProperty* Property = UAudioInsightsSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAudioInsightsSettings, OutputMeterDashboardSettings));

		UAudioInsightsSettings* AudioInsightsSettings = GetMutableDefault<UAudioInsightsSettings>();
		AudioInsightsSettings->UpdateSinglePropertyInConfigFile(Property, UAudioInsightsSettings::GetAudioInsightsConfigFilename());
	}

	void FOutputMeterDashboardViewFactory::InitLoudnessMeterWidgetView()
	{
		using namespace AudioWidgetsCore;

		auto SaveSettings = [this]()
		{
			const FProperty* Property = UAudioInsightsSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAudioInsightsSettings, OutputMeterSettings));

			UAudioInsightsSettings* Settings = GetMutableDefault<UAudioInsightsSettings>();
			Settings->UpdateSinglePropertyInConfigFile(Property, UAudioInsightsSettings::GetAudioInsightsConfigFilename());
		};

		// Migrate loudness meter default colors (v0 -> v1)
		{
			FLoudnessMeterSettings& MeterSettings = GetLoudnessMeterSettings();

			if (MeterSettings.ConfigVersion < 1)
			{
				MeterSettings.LoudnessRange.Color = FLoudnessMeterSettings::DefaultLoudnessRangeColor;
				MeterSettings.TruePeak.Color = FLoudnessMeterSettings::DefaultTruePeakColor;
				MeterSettings.ConfigVersion = 1;
				SaveSettings();
			}
		}

		// TimerPanelParams
		FLoudnessMeterWidgetView::FTimerPanelParams TimerPanelParams
		{
			.AnalysisTime = TAttribute<FTimespan>::CreateSP(this, &FOutputMeterDashboardViewFactory::GetAnalysisTime),
			.OnResetButtonClicked = FOnClicked::CreateSP(this, &FOutputMeterDashboardViewFactory::HandleResetButtonClicked),
			.bIsVisible = true
		};

		TimerPanelParams.bIsResetButtonEnabled.BindLambda([]()
		{
			return IAudioInsightsModule::IsLiveSession();
		});

		TimerPanelParams.bIsVisible.BindLambda([this]() { return GetLoudnessMeterSettings().bDisplayAnalysisTimer; });
		TimerPanelParams.OnVisibilityToggleRequested.BindLambda([this, SaveSettings]()
		{
			FLoudnessMeterSettings& LoudnessMeterSettings = GetLoudnessMeterSettings();
			LoudnessMeterSettings.bDisplayAnalysisTimer = !LoudnessMeterSettings.bDisplayAnalysisTimer;
			SaveSettings();
		});

		// LoudnessScaleParams
		FLoudnessMeterWidgetView::FLoudnessScaleParams LoudnessScaleParams;

		LoudnessScaleParams.Range.BindLambda([this]() { return GetLoudnessMeterSettings().LoudnessScaleRange; });
		LoudnessScaleParams.OnRangeValueChanged.BindLambda([this](int32 Value) { GetLoudnessMeterSettings().LoudnessScaleRange = Value; });
		LoudnessScaleParams.OnRangeValueCommitted.BindLambda([this, SaveSettings](int32 Value, ETextCommit::Type CommitType)
		{
			GetLoudnessMeterSettings().LoudnessScaleRange = Value;
			SaveSettings();
		});

		LoudnessScaleParams.Offset.BindLambda([this]() { return GetLoudnessMeterSettings().LoudnessScaleOffset; });
		LoudnessScaleParams.OnOffsetValueChanged.BindLambda([this](int32 Value) { GetLoudnessMeterSettings().LoudnessScaleOffset = Value; });
		LoudnessScaleParams.OnOffsetValueCommitted.BindLambda([this, SaveSettings](int32 Value, ETextCommit::Type CommitType)
		{
			GetLoudnessMeterSettings().LoudnessScaleOffset = Value;
			SaveSettings();
		});

		LoudnessScaleParams.Target.BindLambda([this]() { return GetLoudnessMeterSettings().LoudnessScaleTarget; });
		LoudnessScaleParams.OnTargetValueChanged.BindLambda([this](int32 Value) { GetLoudnessMeterSettings().LoudnessScaleTarget = Value; });
		LoudnessScaleParams.OnTargetValueCommitted.BindLambda([this, SaveSettings](int32 Value, ETextCommit::Type CommitType)
		{
			GetLoudnessMeterSettings().LoudnessScaleTarget = Value;
			SaveSettings();
		});

		LoudnessScaleParams.TruePeakLimit.BindLambda([this]() { return GetLoudnessMeterSettings().TruePeakLimit; });
		LoudnessScaleParams.OnTruePeakLimitValueChanged.BindLambda([this](float Value) { GetLoudnessMeterSettings().TruePeakLimit = Value; });
		LoudnessScaleParams.OnTruePeakLimitValueCommitted.BindLambda([this, SaveSettings](float Value, ETextCommit::Type CommitType)
		{
			GetLoudnessMeterSettings().TruePeakLimit = Value;
			SaveSettings();
		});

		LoudnessScaleParams.TargetColor.BindLambda([this]() { return GetLoudnessMeterSettings().TargetColor; });
		LoudnessScaleParams.OnTargetColorChanged.BindLambda([this](FLinearColor NewColor) { GetLoudnessMeterSettings().TargetColor = NewColor; });
		LoudnessScaleParams.OnTargetColorPickerWindowClosed.BindLambda([SaveSettings](const TSharedRef<SWindow>& Window) { SaveSettings(); });
		LoudnessScaleParams.OnResetTargetColorRequested.BindLambda([this, SaveSettings]()
		{
			GetLoudnessMeterSettings().TargetColor = FLoudnessMeterWidgetView::GetDefaultTargetColor();
			SaveSettings();
		});

		LoudnessScaleParams.bRelativeScale.BindLambda([this]() { return GetLoudnessMeterSettings().bUseRelativeLoudnessScale; });
		LoudnessScaleParams.OnRelativeScaleToggleRequested.BindLambda([this, SaveSettings]()
		{
			FLoudnessMeterSettings& LoudnessMeterSettings = GetLoudnessMeterSettings();
			LoudnessMeterSettings.bUseRelativeLoudnessScale = !LoudnessMeterSettings.bUseRelativeLoudnessScale;
			SaveSettings();
		});

		// ValuesDisplayOrderParams
		FLoudnessMeterWidgetView::FDisplayOrderParams ValuesDisplayOrderParams;

		ValuesDisplayOrderParams.PermutationIndex.BindLambda([this]() { return GetLoudnessMeterSettings().ValuesOrderingPermutation; });
		ValuesDisplayOrderParams.OnPermutationChanged.BindLambda([this, SaveSettings](int32 NewPermutationIndex)
		{
			GetLoudnessMeterSettings().ValuesOrderingPermutation = NewPermutationIndex;
			SaveSettings();
		});

		// MetersDisplayOrderParams
		FLoudnessMeterWidgetView::FDisplayOrderParams MetersDisplayOrderParams;

		MetersDisplayOrderParams.PermutationIndex.BindLambda([this]() { return GetLoudnessMeterSettings().MetersOrderingPermutation; });
		MetersDisplayOrderParams.OnPermutationChanged.BindLambda([this, SaveSettings](int32 NewPermutationIndex)
		{
			GetLoudnessMeterSettings().MetersOrderingPermutation = NewPermutationIndex;
			SaveSettings();
		});

		// LoudnessMetrics
		FLoudnessMeterWidgetView::FLoudnessMetric LoudnessMetrics[] =
		{
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, LongTermLoudness),
				.DisplayName = LOCTEXT("IntegratedLoudnessDisplayName", "Integrated"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterResults.LongTermLoudness; }),
				.bShowValue = true,
				.bShowMeter = false,
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, ShortTermLoudness),
				.DisplayName = LOCTEXT("ShortTermLoudnessDisplayName", "Short Term"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterResults.ShortTermLoudness; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterMaxResults.ShortTermLoudness; }),
				.bShowValue = false,
				.bShowMeter = true,
				.bHoldMaxForValue = false,
				.bHoldMaxForMeter = false,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.ShortTermLoudness = MIN_VOLUME_DECIBELS; }),
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, MomentaryLoudness),
				.DisplayName = LOCTEXT("MomentaryLoudnessDisplayName", "Momentary"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterResults.MomentaryLoudness; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterMaxResults.MomentaryLoudness; }),
				.bShowValue = false,
				.bShowMeter = true,
				.bHoldMaxForValue = false,
				.bHoldMaxForMeter = false,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.MomentaryLoudness = MIN_VOLUME_DECIBELS; }),
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, LoudnessRange),
				.DisplayName = LOCTEXT("LoudnessRangeDisplayName", "Range"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Range = TAttribute<TOptional<FFloatInterval>>::CreateSPLambda(this, [this]() { return FFloatInterval{ LoudnessMeterResults.LoudnessRangeLowerBound, LoudnessMeterResults.LoudnessRangeUpperBound }; }),
				.bShowValue = true,
				.bShowMeter = false,
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, TruePeak),
				.DisplayName = LOCTEXT("TruePeakDisplayName", "True Peak"),
				.MeterMetric = EAudioMeterMetric::Decibels,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterResults.TruePeakDb; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterMaxResults.TruePeakDb; }),
				.bShowValue = true,
				.bShowMeter = false,
				.bHoldMaxForValue = true,
				.bHoldMaxForMeter = true,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.TruePeakDb = MIN_VOLUME_DECIBELS; })
			}
		};

		for (AudioWidgetsCore::FLoudnessMeterWidgetView::FLoudnessMetric& LoudnessMetric : LoudnessMetrics)
		{
			if (const FProperty* DisplayOptionsProperty = FLoudnessMeterSettings::StaticStruct()->FindPropertyByName(LoudnessMetric.Name))
			{
				// Show value
				LoudnessMetric.bShowValue.BindLambda([this, DisplayOptionsProperty]()
				{
					const FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					return DisplayOptions->bShowValue;
				});

				LoudnessMetric.OnShowValueToggleRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bShowValue = !DisplayOptions->bShowValue;

					SaveSettings();

					LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
				});

				LoudnessMetric.OnShowValueToggleFromDragDropRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bShowValue = !DisplayOptions->bShowValue;

					if (DisplayOptions->bShowValue)
					{
						DisplayOptions->bHoldMaxForValue = DisplayOptions->bHoldMaxForMeter;
					}

					SaveSettings();

					LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
				});

				// Show Meter
				LoudnessMetric.bShowMeter.BindLambda([this, DisplayOptionsProperty]()
				{
					const FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					return DisplayOptions->bShowMeter;
				});

				LoudnessMetric.OnShowMeterToggleRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bShowMeter = !DisplayOptions->bShowMeter;

					SaveSettings();

					LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
				});

				LoudnessMetric.OnShowMeterToggleFromDragDropRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bShowMeter = !DisplayOptions->bShowMeter;

					if (DisplayOptions->bShowMeter)
					{
						DisplayOptions->bHoldMaxForMeter = DisplayOptions->bHoldMaxForValue;
					}

					SaveSettings();

					LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
				});

				// Hold Max for Value
				LoudnessMetric.bHoldMaxForValue.BindLambda([this, DisplayOptionsProperty]()
				{
					const FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					return DisplayOptions->bHoldMaxForValue;
				});

				LoudnessMetric.OnHoldMaxForValueToggleRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bHoldMaxForValue = !DisplayOptions->bHoldMaxForValue;

					SaveSettings();
				});

				// Hold Max for Meter
				LoudnessMetric.bHoldMaxForMeter.BindLambda([this, DisplayOptionsProperty]()
				{
					const FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					return DisplayOptions->bHoldMaxForMeter;
				});

				LoudnessMetric.OnHoldMaxForMeterToggleRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->bHoldMaxForMeter = !DisplayOptions->bHoldMaxForMeter;

					SaveSettings();
				});

				// Color
				LoudnessMetric.Color.BindLambda([this, DisplayOptionsProperty]()
				{
					const FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					return DisplayOptions->Color;
				});

				LoudnessMetric.OnColorChanged.BindLambda([this, DisplayOptionsProperty](FLinearColor NewColor)
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->Color = NewColor;
				});

				LoudnessMetric.OnColorPickerWindowClosed.BindLambda([SaveSettings](const TSharedRef<SWindow>& Window)
				{
					SaveSettings();
				});

				LoudnessMetric.OnResetColorRequested.BindLambda([this, DisplayOptionsProperty, SaveSettings, MetricName = LoudnessMetric.Name]()
				{
					FLoudnessMeterDisplayOptions* DisplayOptions = DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMeterDisplayOptions>(&GetLoudnessMeterSettings());
					DisplayOptions->Color = FLoudnessMeterWidgetView::GetDefaultMeterColor(MetricName);

					SaveSettings();
				});
			}
		}

		LoudnessMeterWidgetView.InitTimerPanel(TimerPanelParams);
		LoudnessMeterWidgetView.InitLoudnessScale(LoudnessScaleParams);
		LoudnessMeterWidgetView.InitValuesDisplayOrder(ValuesDisplayOrderParams);
		LoudnessMeterWidgetView.InitMetersDisplayOrder(MetersDisplayOrderParams);

		for (const FLoudnessMeterWidgetView::FLoudnessMetric& LoudnessMetric : LoudnessMetrics)
		{
			LoudnessMeterWidgetView.AddLoudnessMetric(LoudnessMetric);
		}
	}

	TSharedRef<SWidget> FOutputMeterDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!bIsLoudnessMeterInitialized)
		{
			InitLoudnessMeterWidgetView();
			bIsLoudnessMeterInitialized = true;
		}

		TSharedRef<SWidget> Widget = SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SColorBlock)
				.Color(FSlateStyle::Get().GetColor("AudioInsights.Analyzers.BackgroundColor"))
			]
			+ SOverlay::Slot()
			[
				SAssignNew(ContentSplitter, SSplitter)
				.Orientation(Orient_Vertical)
				.ResizeMode(ESplitterResizeMode::Fill)
				.PhysicalSplitterHandleSize(4.0f)
				.HitDetectionSplitterHandleSize(6.0f)
				.OnSplitterFinishedResizing(this, &FOutputMeterDashboardViewFactory::OnSplitterFinishedResizing)
				+ SSplitter::Slot()
				.Value(GetOutputMeterDashboardSettings().LoudnessMetersSlotSize)
				[
					SNew(SExpandableArea)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OutputMeter_LoudnessMetersHeader", "Loudness Meters"))
					]
					.BodyContent()
					[
						LoudnessMeterWidgetView.MakeWidget()
					]
					.InitiallyCollapsed(GetOutputMeterDashboardSettings().bLoudnessMetersCollapsed)
					.OnAreaExpansionChanged(this, &FOutputMeterDashboardViewFactory::OnLoudnessMeterExpansionChanged)
				]
				+ SSplitter::Slot()
				.Value(GetOutputMeterDashboardSettings().RMSMeterSlotSize)
				[
					SNew(SExpandableArea)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OutputMeter_RMSMeterHeader", "RMS Meter"))
					]
					.BodyContent()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNew(SAudioMeterWidget)
							.Orientation(EOrientation::Orient_Vertical)
							.BackgroundColor(FLinearColor::Transparent)
							.MeterBackgroundColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterBackgroundColor)
							.MeterValueColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterValueColor)
							.MeterPeakColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterPeakColor)
							.MeterClippingColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterClippingColor)
							.MeterScaleColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterScaleColor)
							.MeterScaleLabelColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterScaleLabelColor)
							.MeterChannelInfo(TAttribute<TArray<FAudioMeterChannelInfo>>::CreateSP(this, &FOutputMeterDashboardViewFactory::GetAudioMeterChannelInfo))
						]
					]
					.InitiallyCollapsed(GetOutputMeterDashboardSettings().bRMSMeterCollapsed)
					.OnAreaExpansionChanged(this, &FOutputMeterDashboardViewFactory::OnChannelMeterExpansionChanged)
				]
			];

		// Restore collapsed sizing rules from settings
		const FOutputMeterDashboardSettings& OutputMeterDashboardSettings = GetOutputMeterDashboardSettings();

		if (OutputMeterDashboardSettings.bLoudnessMetersCollapsed)
		{
			ContentSplitter->SlotAt(0).SetSizingRule(SSplitter::SizeToContent);
		}

		if (OutputMeterDashboardSettings.bRMSMeterCollapsed)
		{
			ContentSplitter->SlotAt(1).SetSizingRule(SSplitter::SizeToContent);
		}

		return Widget;
	}

	void FOutputMeterDashboardViewFactory::UpdateDataViewEntries()
	{
		const TSharedPtr<const FOutputMeterTraceProvider> Provider = FindProvider<const FOutputMeterTraceProvider>();
		if (Provider.IsValid())
		{
			if (const FOutputMeterTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData())
			{
				DataViewEntries.Reset();

				auto TransformEntry = [](const typename FOutputMeterTraceProvider::FEntryPair& Pair)
				{
					return StaticCastSharedPtr<IDashboardDataViewEntry>(Pair.Value);
				};

				Algo::Transform(*DeviceData, DataViewEntries, TransformEntry);
			}
			else
			{
				if (!DataViewEntries.IsEmpty())
				{
					DataViewEntries.Empty();
				}
			}
		}
	}

	void FOutputMeterDashboardViewFactory::UpdateAudioMeterInfo()
	{
		if (SubmixProvider.IsValid())
		{
			const uint64 LastUpdateId = SubmixProvider->GetLastUpdateId();

			if (ProcessedUpdateId != LastUpdateId)
			{
				if (const FSubmixTraceProvider::FDeviceData* DeviceData = SubmixProvider->FindFilteredDeviceData())
				{
					// Try and find the FSubmixDashboardEntry for the main submix with updated audio meter values
					const auto IsMainSubmixEntryPair = [](const FSubmixTraceProvider::FEntryPair& EntryPair) { return EntryPair.Value->IsMainSubmix(); };

					const FSubmixTraceProvider::FEntryPair* MainSubmixEntryPair = Algo::FindByPredicate(*DeviceData, IsMainSubmixEntryPair);

					TSharedPtr<FSubmixDashboardEntry> MainSubmixDashboardEntry = MainSubmixEntryPair ? MainSubmixEntryPair->Value : nullptr;

					if (MainSubmixDashboardEntry.IsValid() && AudioMeterInfo.IsValid())
					{
						*AudioMeterInfo = MainSubmixDashboardEntry->AudioMeterInfo.Get();
					}
				}

				ProcessedUpdateId = LastUpdateId;
			}
		}
	}

	void FOutputMeterDashboardViewFactory::UpdateLoudnessValues()
	{
		if (!DataViewEntries.IsEmpty())
		{
			const FOutputMeterDashboardEntry& OutputMeterEntry = FOutputMeterDashboardViewFactoryPrivate::CastEntry(*DataViewEntries[0].Get());

#if WITH_EDITOR
			// Detect audio device change (e.g. PIE start/stop) and reset
			if (LastSeenDeviceId != INDEX_NONE && OutputMeterEntry.DeviceId != LastSeenDeviceId)
			{
				AnalysisStartTimestamp = 0.0;
				LatestTimestamp = 0.0;

				LoudnessMeterResults    = FLoudnessMeterResults();
				LoudnessMeterMaxResults = FLoudnessMeterMaxResults();
			}

			LastSeenDeviceId = OutputMeterEntry.DeviceId;
#endif // WITH_EDITOR

			if (AnalysisStartTimestamp == 0.0 && OutputMeterEntry.Timestamp > 0.0)
			{
				AnalysisStartTimestamp = OutputMeterEntry.Timestamp;
			}

			LatestTimestamp = OutputMeterEntry.Timestamp;

			LoudnessMeterResults.LongTermLoudness  = OutputMeterEntry.LongTermLoudness;
			LoudnessMeterResults.ShortTermLoudness = OutputMeterEntry.ShortTermLoudness;
			LoudnessMeterResults.MomentaryLoudness = OutputMeterEntry.MomentaryLoudness;
			LoudnessMeterResults.LoudnessRangeLowerBound = OutputMeterEntry.LoudnessRangeLowerBound;
			LoudnessMeterResults.LoudnessRangeUpperBound = OutputMeterEntry.LoudnessRangeUpperBound;
			LoudnessMeterResults.TruePeakDb = OutputMeterEntry.TruePeakMaxValueDb;

			LoudnessMeterMaxResults.ShortTermLoudness = FMath::Max<float>(LoudnessMeterMaxResults.ShortTermLoudness, LoudnessMeterResults.ShortTermLoudness);
			LoudnessMeterMaxResults.MomentaryLoudness = FMath::Max<float>(LoudnessMeterMaxResults.MomentaryLoudness, LoudnessMeterResults.MomentaryLoudness);
			LoudnessMeterMaxResults.TruePeakDb        = FMath::Max<float>(LoudnessMeterMaxResults.TruePeakDb, LoudnessMeterResults.TruePeakDb);

			bWasReceivingData = true;
		}
		else if (bWasReceivingData)
		{
			// Data stopped flowing (e.g. PIE stopped, audio device destroyed) - reset timer and max values
			AnalysisStartTimestamp = 0.0;
			LatestTimestamp = 0.0;
#if WITH_EDITOR
			LastSeenDeviceId = INDEX_NONE;
#endif // WITH_EDITOR

			LoudnessMeterResults    = FLoudnessMeterResults();
			LoudnessMeterMaxResults = FLoudnessMeterMaxResults();

			bWasReceivingData = false;
		}
	}

	void FOutputMeterDashboardViewFactory::ProcessEntries(FTraceDashboardViewFactoryBase::EProcessReason Reason)
	{
		UpdateDataViewEntries();

		UpdateAudioMeterInfo();
		UpdateLoudnessValues();

		LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
	}

	TArray<FAudioMeterChannelInfo> FOutputMeterDashboardViewFactory::GetAudioMeterChannelInfo() const
	{
		TArray<FAudioMeterChannelInfo> MeterChannelInfo;

		if (AudioMeterInfo.IsValid())
		{
			if (!AudioMeterInfo->EnvelopeValues.IsEmpty())
			{
				MeterChannelInfo.Reserve(AudioMeterInfo->EnvelopeValues.Num());
				for (const float EnvelopeValue : AudioMeterInfo->EnvelopeValues)
				{
					MeterChannelInfo.Emplace(::Audio::ConvertToDecibels(EnvelopeValue), MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS);
				}
			}
			else
			{
				MeterChannelInfo.Reserve(AudioMeterInfo->NumChannels);
				while (MeterChannelInfo.Num() < AudioMeterInfo->NumChannels)
				{
					MeterChannelInfo.Emplace(MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS);
				}
			}
		}

		return MeterChannelInfo;
	}

	FTimespan FOutputMeterDashboardViewFactory::GetAnalysisTime() const
	{
		if (AnalysisStartTimestamp > 0.0 && LatestTimestamp > AnalysisStartTimestamp)
		{
			return FTimespan::FromSeconds(LatestTimestamp - AnalysisStartTimestamp);
		}

		return FTimespan::Zero();
	}

	FReply FOutputMeterDashboardViewFactory::HandleResetButtonClicked()
	{
		LoudnessMeterResults    = FLoudnessMeterResults();
		LoudnessMeterMaxResults = FLoudnessMeterMaxResults();

		AnalysisStartTimestamp = 0.0;
		LatestTimestamp = 0.0;

		const ::Audio::FDeviceId DeviceId = FAudioInsightsModule::GetChecked().GetDeviceId();
		const FString ResetMainSubmixAnalyzersStr = FString::Printf(TEXT("au.ResetMainSubmixAnalyzers_AD%d"), DeviceId);

		IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();
		AudioInsightsTraceModule.ExecuteConsoleCommand(ResetMainSubmixAnalyzersStr);

		return FReply::Handled();
	}

	void FOutputMeterDashboardViewFactory::OnLoudnessMeterExpansionChanged(const bool bIsExpanded)
	{
		if (!ContentSplitter.IsValid())
		{
			return;
		}

		FOutputMeterDashboardSettings& OutputMeterDashboardSettings = GetOutputMeterDashboardSettings();

		OutputMeterDashboardSettings.bLoudnessMetersCollapsed = !bIsExpanded;

		if (bIsExpanded)
		{
			ContentSplitter->SlotAt(0).SetSizingRule(SSplitter::FractionOfParent);
			ContentSplitter->SlotAt(0).SetSizeValue(OutputMeterDashboardSettings.LoudnessMetersSlotSize);
		}
		else
		{
			OutputMeterDashboardSettings.LoudnessMetersSlotSize = ContentSplitter->SlotAt(0).GetSizeValue();
			ContentSplitter->SlotAt(0).SetSizingRule(SSplitter::SizeToContent);
		}

		SaveOutputMeterDashboardSettings();
	}

	void FOutputMeterDashboardViewFactory::OnChannelMeterExpansionChanged(const bool bIsExpanded)
	{
		if (!ContentSplitter.IsValid())
		{
			return;
		}

		FOutputMeterDashboardSettings& OutputMeterDashboardSettings = GetOutputMeterDashboardSettings();

		OutputMeterDashboardSettings.bRMSMeterCollapsed = !bIsExpanded;

		if (bIsExpanded)
		{
			ContentSplitter->SlotAt(1).SetSizingRule(SSplitter::FractionOfParent);
			ContentSplitter->SlotAt(1).SetSizeValue(OutputMeterDashboardSettings.RMSMeterSlotSize);
		}
		else
		{
			OutputMeterDashboardSettings.RMSMeterSlotSize = ContentSplitter->SlotAt(1).GetSizeValue();
			ContentSplitter->SlotAt(1).SetSizingRule(SSplitter::SizeToContent);
		}

		SaveOutputMeterDashboardSettings();
	}

	void FOutputMeterDashboardViewFactory::OnSplitterFinishedResizing()
	{
		if (!ContentSplitter.IsValid())
		{
			return;
		}

		FOutputMeterDashboardSettings& OutputMeterDashboardSettings = GetOutputMeterDashboardSettings();

		if (!OutputMeterDashboardSettings.bLoudnessMetersCollapsed)
		{
			OutputMeterDashboardSettings.LoudnessMetersSlotSize = ContentSplitter->SlotAt(0).GetSizeValue();
		}

		if (!OutputMeterDashboardSettings.bRMSMeterCollapsed)
		{
			OutputMeterDashboardSettings.RMSMeterSlotSize = ContentSplitter->SlotAt(1).GetSizeValue();
		}

		SaveOutputMeterDashboardSettings();
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
