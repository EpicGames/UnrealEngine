// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SAudioCurveView.h"

#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "AudioInsightsLog.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SAudioCurveView"

void SAudioCurveView::Construct( const SAudioCurveView::FArguments& InArgs )
{
	// A lot of this should go into a style 
	ViewRange = InArgs._ViewRange;
	AutoRangeYAxis = InArgs._AutoRangeYAxis;
	GridLineColor = InArgs._GridLineColor;
	AxesLabelColor = InArgs._AxesLabelColor;
	YMargin = FMath::Clamp(InArgs._YMargin.Get(), 0.0f, 0.5f);
	HorizontalAxisIncrement = InArgs._HorizontalAxisIncrement;
	DesiredSize = InArgs._DesiredSize;
	AdditionalToolTipText = InArgs._AdditionalToolTipText;
	YDataRange = FFloatInterval(0.0f, 1.0f);
	NumHorizontalGridLines = 10;
	OnScrubPositionChanged = InArgs._OnScrubPositionChanged;
	OnKeyDownHandler = InArgs._OnKeyDown;

	// set clipping on by default, since the OnPaint function is drawing outside the bounds
	Clipping = EWidgetClipping::ClipToBounds;
	XValueFormattingOptions.MaximumFractionalDigits = 3;
	LineDrawEffects = ESlateDrawEffect::NoPixelSnapping;
	LabelFont = FCoreStyle::GetDefaultFontStyle("Bold", 7);

	SetToolTip(CreateCurveTooltip());
}

int32 SAudioCurveView::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = PaintCurves( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}

FReply SAudioCurveView::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsMouseDragging = true;
	}

	BroadcastScrubPosition();

	return bIsMouseDragging ? FReply::Handled() : FReply::Unhandled();
}

FReply SAudioCurveView::OnMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	UpdateCurveToolTip(InMyGeometry, InMouseEvent);
	BroadcastScrubPosition();

	return FReply::Handled();
}

FReply SAudioCurveView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	BroadcastScrubPosition();
	
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsMouseDragging = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAudioCurveView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	CrosshairPosition.Reset();
	bIsMouseDragging = false;
}

FReply SAudioCurveView::OnPreviewKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
{
	if (OnKeyDownHandler.IsBound())
	{
		return OnKeyDownHandler.Execute(Geometry, KeyEvent);
	}

	return SCompoundWidget::OnPreviewKeyDown(Geometry, KeyEvent);
}

TSharedRef<SToolTip> SAudioCurveView::CreateCurveTooltip()
{
	return SNew(SToolTip)
		.Visibility_Lambda([this]()
		{ 
			return MetadataPerCurve == nullptr ? EVisibility::Collapsed : EVisibility::Visible; 
		})
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SAudioCurveView::GetCurveToolTipDisplayNameText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SAudioCurveView::GetCurveToolTipXValueText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SAudioCurveView::GetCurveToolTipYValueText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(AdditionalToolTipText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
				.Visibility_Lambda([this]() { return AdditionalToolTipText.IsSet() && !AdditionalToolTipText.Get().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		];
}

void SAudioCurveView::BroadcastScrubPosition()
{
	if (bIsMouseDragging && OnScrubPositionChanged.IsBound() && CrosshairPosition.IsSet())
	{
		const FCurvePoint CrosshairPoint = CrosshairPosition.GetValue();
		OnScrubPositionChanged.Execute(CrosshairPoint.Key, CrosshairPoint.Value);
	}
}

void SAudioCurveView::UpdateCurveToolTip(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()) || !MetadataPerCurve.IsValid())
	{
		CrosshairPosition.Reset();
		return;
	}

	if (!PointDataPerCurve.IsValid())
	{
		return;
	}

	// Mouse position in widget space
	const FVector2f HitPosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	// Range helper struct
	const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(ViewRange.Get(), InMyGeometry.GetLocalSize());

	// Mouse position from widget space to curve input space
	const double TargetX = RangeToScreen.LocalXToInput(HitPosition.X);

	// Keep track of closest curve index, closest point, value 
	uint64 ClosestCurveId = INDEX_NONE;
	FCurvePoint ClosestPoint;
	float ClosestDistance = TNumericLimits<float>::Max();
	float ThickestPlot = 0.0f;

	for (const auto& [CurveId, CurveMetadata] : *MetadataPerCurve)
	{
		if (CurveMetadata.CurveColor.A == 0.0f || CurveMetadata.PlotThickness < ThickestPlot)
		{
			continue;
		}

		const TArray<FCurvePoint>* CurvePoints = PointDataPerCurve->Find(CurveId);
		if (CurvePoints == nullptr)
		{
			continue;
		}

		const int32 NumPoints = CurvePoints->Num();
		if (NumPoints > 0)
		{
			for (int32 i = 1; i < NumPoints; ++i)
			{
				const FCurvePoint& Point1 = (*CurvePoints)[i - 1];
				const FCurvePoint& Point2 = (*CurvePoints)[i];

				// Find points that contain mouse hit-point x
				if (Point1.Key <= TargetX && Point2.Key >= TargetX)
				{
					// Choose point with the smallest x delta
					const float Delta1 = abs(TargetX - Point1.Key);
					const float Delta2 = abs(TargetX - Point2.Key);
					const FCurvePoint& TargetPoint = Delta1 < Delta2 ? Point1 : Point2;

					// Convert target point Y to widget space 
					const FVector2f LocalSize = InMyGeometry.GetLocalSize();
					const float WidgetSpaceY = ValueToLocalY(LocalSize, TargetPoint.Value);
					
					// Compare distance in widget space between HitPosition and closest point by x value on this curve
					const float Distance = FVector2f::Distance(HitPosition, FVector2f(RangeToScreen.InputToLocalX(TargetPoint.Key), WidgetSpaceY));

					if (Distance < ClosestDistance || CurveMetadata.PlotThickness > ThickestPlot)
					{
						ClosestDistance = Distance;
						ClosestCurveId = CurveId;
						ClosestPoint = TargetPoint;
						ThickestPlot = CurveMetadata.PlotThickness;
					}
					break;
				}
			}
		}
	}

	// Set tooltip text values  
	if (ClosestCurveId != INDEX_NONE)
	{
		CurveToolTipXValueText = FText::Format(LOCTEXT("CurveToolTipValueXFormat", "X: {0}"), FText::AsNumber(ClosestPoint.Key, &XValueFormattingOptions));
		CurveToolTipYValueText = FText::Format(LOCTEXT("CurveToolTipValueYFormat", "Y: {0}"), FText::AsNumber(ClosestPoint.Value, &YValueFormattingOptions));
		CurveToolTipDisplayNameText = MetadataPerCurve->Find(ClosestCurveId)->DisplayName;

		CrosshairPosition = ClosestPoint;
	}
	else
	{
		CrosshairPosition.Reset();
	}
}

FVector2D SAudioCurveView::ComputeDesiredSize( float ) const
{
	return DesiredSize.Get();
}

void SAudioCurveView::UpdateYDataRange()
{
	const static auto FCurvePointCompare = [](const FCurvePoint& A, const FCurvePoint& B) 
	{
		return A.Value < B.Value;
	};

	float MinValue = TNumericLimits<float>::Max();
	float MaxValue = TNumericLimits<float>::Lowest();
	for (auto Iter = PointDataPerCurve->CreateConstIterator(); Iter; ++Iter)
	{
		const TArray<FCurvePoint>& CurvePoints = Iter->Value;
		if (CurvePoints.IsEmpty())
		{
			continue;
		}
		const FCurvePoint* MinPoint = Algo::MinElement(CurvePoints, FCurvePointCompare);
		const FCurvePoint* MaxPoint = Algo::MaxElement(CurvePoints, FCurvePointCompare);
		MinValue = MinPoint ? FMath::Min(MinValue, MinPoint->Value) : 0.0f;
		MaxValue = MaxPoint ? FMath::Max(MaxValue, MaxPoint->Value) : 1.0f;
	}

	if (MinValue < TNumericLimits<float>::Max())
	{
		YDataRange.Min = MinValue;
	}
	
	if (MaxValue > TNumericLimits<float>::Lowest())
	{
		YDataRange.Max = MaxValue;
	}
}

#if !WITH_EDITOR
void SAudioCurveView::UpdateYDataRangeFromTimestampRange(const double InLowerBoundTimestamp, const double InUpperBoundTimestamp)
{
	const static auto FCurvePointCompare = [](const FCurvePoint& A, const FCurvePoint& B)
		{
			return A.Value < B.Value;
		};

	float MinValue = TNumericLimits<float>::Max();
	float MaxValue = TNumericLimits<float>::Lowest();

	if (PointDataPerCurve.IsValid())
	{
		for (const auto& [CurveId, CurvePoints] : *PointDataPerCurve)
		{
			if (!CurvePoints.IsEmpty())
			{
				const int32 FoundLowerIndex = CurvePoints.IndexOfByPredicate([&InLowerBoundTimestamp](const FCurvePoint& InDataPoint)
				{
					return InDataPoint.Key >= InLowerBoundTimestamp;
				});

				const int32 FoundUpperIndex = CurvePoints.IndexOfByPredicate([&InUpperBoundTimestamp](const FCurvePoint& InDataPoint)
				{
					return InDataPoint.Key >= InUpperBoundTimestamp;
				});

				if (FoundLowerIndex != INDEX_NONE && FoundUpperIndex != INDEX_NONE && FoundLowerIndex != FoundUpperIndex)
				{
					const TArrayView<const FCurvePoint> CurvePointsArrayView = MakeArrayView(CurvePoints.GetData() + FoundLowerIndex, FoundUpperIndex - FoundLowerIndex);

					const FCurvePoint* MinPoint = Algo::MinElement(CurvePointsArrayView, FCurvePointCompare);
					const FCurvePoint* MaxPoint = Algo::MaxElement(CurvePointsArrayView, FCurvePointCompare);
					MinValue = MinPoint ? FMath::Min(MinValue, MinPoint->Value) : 0.0f;
					MaxValue = MaxPoint ? FMath::Max(MaxValue, MaxPoint->Value) : 1.0f;
				}
			}
		}
	}

	// Adjust Y range in case MinValue and MaxValue are not updated or if values are too close together
	constexpr float Epsilon = 0.001f;

	if (MinValue == TNumericLimits<float>::Max())
	{
		MinValue = 0.0f;
	}

	if (MaxValue == TNumericLimits<float>::Lowest() || FMath::Abs(MaxValue - MinValue) < Epsilon)
	{
		MaxValue = MinValue + Epsilon;
	}

	YDataRange.Min = MinValue;
	YDataRange.Max = MaxValue;
}
#endif // !WITH_EDITOR

void SAudioCurveView::SetCurvesPointData(TSharedPtr<TMap<uint64, TArray<FCurvePoint>>> InPointDataPerCurve)
{
	PointDataPerCurve = InPointDataPerCurve;

	if (PointDataPerCurve.IsValid() && AutoRangeYAxis.Get())
	{
		UpdateYDataRange();
	}
}

void SAudioCurveView::SetCurvesMetadata(TSharedPtr<TMap<uint64, FCurveMetadata>> InMetadataPerCurve)
{
	MetadataPerCurve = InMetadataPerCurve;
}

void SAudioCurveView::SetYValueFormattingOptions(const FNumberFormattingOptions InYValueFormattingOptions)
{
	YValueFormattingOptions = InYValueFormattingOptions;
}

void SAudioCurveView::SetYAxisRange(const FFloatInterval& InRange)
{
	YDataRange = InRange;
}

float SAudioCurveView::ValueToLocalY(const FVector2f AllottedLocalSize, const float Value) const
{
	// Slate Y values increase going down the screen, so base < top but base is above top on the screen
	const float MarginBase = YMargin.Get() * AllottedLocalSize.Y;
	const float MarginTop = AllottedLocalSize.Y - MarginBase;
	
	// Special case to add padding based on YMargin if YDataRange min/max are the same 
	if (FMath::IsNearlyEqual(YDataRange.Min, YDataRange.Max))
	{
		const FVector2f PaddedYDataRange = FVector2f(YDataRange.Max * (1.0f - YMargin.Get()), YDataRange.Min * (1.0f + YMargin.Get()));
		return FMath::GetMappedRangeValueUnclamped(PaddedYDataRange, FVector2f(MarginTop, MarginBase), Value);
	}
	return FMath::GetMappedRangeValueUnclamped(FVector2f(YDataRange.Min, YDataRange.Max), FVector2f(MarginTop, MarginBase), Value);
}

float SAudioCurveView::LocalYToValue(const FVector2f AllottedLocalSize, const float LocalY) const
{
	// Slate Y values increase going down the screen, so base < top but base is above top on the screen
	const float MarginBase = YMargin.Get() * AllottedLocalSize.Y;
	const float MarginTop = AllottedLocalSize.Y - MarginBase;

	// Special case to add padding based on YMargin if YDataRange min/max are the same 
	if (FMath::IsNearlyEqual(YDataRange.Min, YDataRange.Max))
	{
		const FVector2f PaddedYDataRange = FVector2f(YDataRange.Max * (1.0f - YMargin.Get()), YDataRange.Min * (1.0f + YMargin.Get()));
		return FMath::GetMappedRangeValueUnclamped(FVector2f(MarginTop, MarginBase), PaddedYDataRange, LocalY);
	}
	return FMath::GetMappedRangeValueUnclamped(FVector2f(MarginTop, MarginBase), FVector2f(YDataRange.Min, YDataRange.Max), LocalY);
}

int32 SAudioCurveView::PaintGridLines(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled, const SSimpleTimeSlider::FScrubRangeToScreen& RangeToScreen) const
{
	const FVector2f Size = AllottedGeometry.GetLocalSize();
	const int32 GridLineLayer = LayerId++;
	TArray<FVector2f> GridPoints;
	GridPoints.AddDefaulted(2);

	// Draw vertical grid lines on multiples of HorizontalAxisIncrement
	if (HorizontalAxisIncrement.Get() > 0)
	{
		const double Factor = 1.0 / HorizontalAxisIncrement.Get();
		// Start at rounded nearest HorizontalAxisIncrement from the lower bound and increment by HorizontalAxisIncrement (ex. 1.5, 2.0, 2.5... if increment is 0.5)
		double VerticalLineValue = FMath::RoundToDouble(RangeToScreen.ViewInput.GetLowerBoundValue() * Factor) / Factor;
		while (VerticalLineValue < RangeToScreen.ViewInput.GetUpperBoundValue())
		{
			const float WidgetX = RangeToScreen.InputToLocalX(VerticalLineValue);
			GridPoints[0].X = WidgetX;
			GridPoints[0].Y = 0;
			GridPoints[1].X = WidgetX;
			GridPoints[1].Y = Size.Y;

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				GridLineLayer,
				AllottedGeometry.ToPaintGeometry(),
				GridPoints,
				LineDrawEffects,
				GridLineColor.Get(),
				true
			);
			VerticalLineValue += HorizontalAxisIncrement.Get();
		}
	}

	// Draw horizontal grid lines 
	const float MarginBase = YMargin.Get() * Size.Y;
	const float MarginTop = (1.0f - YMargin.Get()) * Size.Y;
	const float GridLineYIncrement = (MarginTop - MarginBase) / (NumHorizontalGridLines - 1);

	for (uint32 HorizontalLineIndex = 0; HorizontalLineIndex < NumHorizontalGridLines; ++HorizontalLineIndex)
	{
		const float WidgetY = GridLineYIncrement * HorizontalLineIndex + MarginBase;
		GridPoints[0].X = 0;
		GridPoints[0].Y = WidgetY;
		GridPoints[1].X = Size.X;
		GridPoints[1].Y = WidgetY;

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			GridLineLayer,
			AllottedGeometry.ToPaintGeometry(),
			GridPoints,
			LineDrawEffects,
			GridLineColor.Get(),
			false
		);
	}
	return LayerId;
}

int32 SAudioCurveView::PaintCrosshair(const FGeometry& MyGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const SSimpleTimeSlider::FScrubRangeToScreen& RangeToScreen) const
{
	if (!CrosshairPosition.IsSet())
	{
		return LayerId;
	}

	const FVector2f Size = MyGeometry.GetLocalSize();
	const int32 CrosshairLayerID = LayerId++;

	const float PointX = RangeToScreen.InputToLocalX(CrosshairPosition.GetValue().Key);
	const float PointY = ValueToLocalY(Size, CrosshairPosition.GetValue().Value);

	constexpr FLinearColor CrosshairColor(1.0f, 1.0f, 1.0f, 0.75f);

	TArray<FVector2f> HorizontalCrosshairLinePoints
	{
		{ 0.0f, PointY },
		{ Size.X, PointY }
	};

	const TArray<FVector2f> VerticalCrosshairLinePoints
	{
		{ PointX, 0.0f },
		{ PointX, Size.Y }
	};

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		CrosshairLayerID,
		MyGeometry.ToPaintGeometry(),
		HorizontalCrosshairLinePoints,
		LineDrawEffects,
		CrosshairColor,
		true
	);

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		CrosshairLayerID,
		MyGeometry.ToPaintGeometry(),
		VerticalCrosshairLinePoints,
		LineDrawEffects,
		CrosshairColor,
		true
	);

	return LayerId;
}

int32 SAudioCurveView::PaintYAxisLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const int32 BackgroundRectangleLayerId = LayerId++;
	float MaxTextWidth = 0.0f;

	// Draw Y axis labels
	const FVector2f Size = AllottedGeometry.GetLocalSize();

	const float MarginBase = YMargin.Get() * Size.Y;
	const float MarginTop  = (1.0f - YMargin.Get()) * Size.Y;

	const float GridLineYIncrement = (MarginTop - MarginBase) / (NumHorizontalGridLines - 1);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	for (uint32 HorizontalLineIndex = 0; HorizontalLineIndex < NumHorizontalGridLines; ++HorizontalLineIndex)
	{
		const float WidgetY = GridLineYIncrement * HorizontalLineIndex + MarginBase;

		// Draw y axis text label every other grid line
		if (HorizontalLineIndex % 2 == 1)
		{
			const float LabelValue  = LocalYToValue(Size, WidgetY);
			const FText LabelString = FText::AsNumber(LabelValue, &YValueFormattingOptions);

			// Position text slightly above the corresponding horizontal line 
			const FVector2f TextSize = FontMeasureService->Measure(LabelString, LabelFont);
			const FVector2f TextOffset(5.0f, WidgetY - TextSize.Y * 0.85f);

			MaxTextWidth = FMath::Max(MaxTextWidth, TextSize.X);

			FSlateDrawElement::MakeText(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				LabelString,
				LabelFont,
				LineDrawEffects,
				AxesLabelColor.Get()
			);
		}
	}

	// Draw Background rectangle (with gradient)
	const float RectanglePadding = MaxTextWidth * 0.6f;
	const FVector2D RectangleSize(MaxTextWidth + RectanglePadding, AllottedGeometry.GetLocalSize().Y);
	const FVector2D RectanglePosition(0.0f, 0.0f);

	const TArray<FSlateGradientStop> GradientStops
	{
		{FVector2D::ZeroVector,                    FLinearColor(0.0f, 0.0f, 0.0f, 0.8f)},
		{FVector2D(RectangleSize.X * 0.50f, 0.0f), FLinearColor(0.0f, 0.0f, 0.0f, 0.65f)},
		{FVector2D(RectangleSize.X * 0.75f, 0.0f), FLinearColor(0.0f, 0.0f, 0.0f, 0.5f)},
		{FVector2D(RectangleSize.X, 0.0f),         FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)}
	};

	FSlateDrawElement::MakeGradient(
		OutDrawElements,
		BackgroundRectangleLayerId,
		AllottedGeometry.ToPaintGeometry(RectangleSize, FSlateLayoutTransform(RectanglePosition)),
		GradientStops,
		Orient_Vertical,
		ESlateDrawEffect::None
	);

	return LayerId;
}

int32 SAudioCurveView::PaintCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const 
{
	// Skip drawing if curve data is not initialized yet
	if (!PointDataPerCurve.IsValid() || !MetadataPerCurve.IsValid())
	{
		return LayerId;
	}

	if (PointDataPerCurve->Num() != MetadataPerCurve->Num())
	{
		UE_LOGF(LogAudioInsights, Warning, "Invalid audio curve view data. Metadata and point curve data nums do not match.")
		return LayerId;
	}

	const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(ViewRange.Get(), AllottedGeometry.GetLocalSize());

	if (RangeToScreen.ViewInput.IsEmpty())
	{
		return LayerId;
	}

	LayerId = PaintGridLines(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled, RangeToScreen);
	LayerId = PaintCrosshair(AllottedGeometry, OutDrawElements, LayerId, RangeToScreen);
	
	static constexpr float LargeFrameTime = 0.5f; // ideally, we could check the recorded frame data for actual frame length
	const FVector2f Size = AllottedGeometry.GetLocalSize();

	// Create and draw points per curve
	for (const auto& [CurveId, CurveMetadata] : *MetadataPerCurve)
	{
		const TArray<FCurvePoint>* CurvePointsPtr = PointDataPerCurve->Find(CurveId);

		if (!CurvePointsPtr || CurvePointsPtr->IsEmpty())
		{
			continue;
		}

		const TArray<FCurvePoint>& CurvePoints = (*CurvePointsPtr);

		TArray<FVector2f> Points;
		Points.Reserve(CurvePoints.Num());

		float PrevX = CurvePoints[0].Key;

		// Keep track of the last 3 recorded Y values
		// These are used later to detect constant lines in the plot and remove all middle points that would be redundant
		float PrevY = CurvePoints[0].Value;
		float LineStartY = CurvePoints[0].Value;

		for (const FCurvePoint& Point : CurvePoints)
		{
			if (Point.Key - PrevX > LargeFrameTime && Points.Num() > 1)
			{
				// break the line list - data has stopped and started again
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					++LayerId,
					AllottedGeometry.ToPaintGeometry(),
					Points,
					LineDrawEffects,
					CurveMetadata.CurveColor,
					true,
					CurveMetadata.PlotThickness
				);

				Points.Reset();
			}

			const float X = RangeToScreen.InputToLocalX(Point.Key);

			if (Points.Num() >= 2 && Point.Value == PrevY && Point.Value == LineStartY)
			{
				// Filter out unnecessary points in the plot line
				// - if 3 or more points in a row have the same Y value, move the line end X value forward in the points array (Points.Last(0).X = X;)
				// - this can heavily reduce the number of points sent to FSlateDrawElement::MakeLines to draw
				Points.Last(0).X = X;
				PrevX = Point.Key;
			}
			else
			{
				const float Y = ValueToLocalY(Size, Point.Value);
				Points.Emplace(X, Y);

				PrevX = Point.Key;

				// Keep track of the previous values of Y to detect constant lines
				LineStartY = PrevY;
				PrevY = Point.Value;
			}
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			Points,
			LineDrawEffects,
			CurveMetadata.CurveColor,
			true,
			CurveMetadata.PlotThickness
		);
	}

	// Draw Y axis labels
	LayerId = PaintYAxisLabels(AllottedGeometry, OutDrawElements, LayerId);

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
