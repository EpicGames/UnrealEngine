// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SAudioMeterView.h"

#include "AudioDefines.h"
#include "AudioInsightsDataSource.h"
#include "DSP/Dsp.h"
#include "Providers/AudioMeterProvider.h"
#include "SAudioMeterWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace SAudioMeterViewPrivate
	{
		const TAttribute<TArray<FAudioMeterChannelInfo>> CreateMeterChannelInfoAttribute(const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo)
		{
			return TAttribute<TArray<FAudioMeterChannelInfo>>::Create([InAudioMeterInfo]()
				{
					TArray<FAudioMeterChannelInfo> MeterChannelInfo;

					const TArray<float>& EnvelopeValues = InAudioMeterInfo->EnvelopeValues;

					if (!EnvelopeValues.IsEmpty())
					{
						MeterChannelInfo.Reserve(EnvelopeValues.Num());

						for (const float EnvelopeValue : EnvelopeValues)
						{
							MeterChannelInfo.Emplace(::Audio::ConvertToDecibels(EnvelopeValue), MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS);
						}
					}
					else
					{
						const int32 NumChannels = InAudioMeterInfo->NumChannels;
							
						for (int32 Index = 0; Index < NumChannels; ++Index)
						{
							MeterChannelInfo.Emplace(MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS, MIN_VOLUME_DECIBELS);
						}
					}

					return MeterChannelInfo;
				});
		}
	}

	void SAudioMeterView::Construct(const SAudioMeterView::FArguments& InArgs, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo)
	{
		using namespace SAudioMeterViewPrivate;

		NameText = InArgs._NameText;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SVerticalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				// Height padding
				+ SVerticalBox::Slot()
				.FillHeight(0.05)
				[
					SNew(SBox)
				]
				// Audio Meter container
				+ SVerticalBox::Slot()
				.FillHeight(0.8)
				[
					// Audio Meter
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(AudioMeterWidget, SAudioMeterWidget)
						.Orientation(EOrientation::Orient_Vertical)
						.BackgroundColor(FLinearColor::Transparent)
						.MeterBackgroundColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterBackgroundColor)
						.MeterValueColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterValueColor)
						.MeterPeakColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterPeakColor)
						.MeterClippingColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterClippingColor)
						.MeterScaleColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterScaleColor)
						.MeterScaleLabelColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterScaleLabelColor)
						.MeterChannelInfo(CreateMeterChannelInfoAttribute(InAudioMeterInfo))
					]
				]
				// Height padding
				+ SVerticalBox::Slot()
				.FillHeight(0.025)
				[
					SNew(SBox)
				]
				// Audio meter name label
				+ SVerticalBox::Slot()
				.FillHeight(0.1)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return NameText;
					})
					.Justification(ETextJustify::Center)
					.ColorAndOpacity(InArgs._NameTextColor)
				]
				// Height padding
				+ SVerticalBox::Slot()
				.FillHeight(0.025)
				[
					SNew(SBox)
				]
			]
		];
	}

	void SAudioMeterView::SetAudioMeterChannelInfo(const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo)
	{
		using namespace SAudioMeterViewPrivate;

		AudioMeterWidget->SetMeterChannelInfo(CreateMeterChannelInfoAttribute(InAudioMeterInfo));
	}

	void SAudioMeterView::SetName(const FText& InName)
	{
		NameText = InName;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
