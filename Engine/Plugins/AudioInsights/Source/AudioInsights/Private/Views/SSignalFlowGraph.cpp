// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SSignalFlowGraph.h"

#include "Messages/SignalFlowTraceMessages.h"
#include "Settings/SignalFlowSettings.h"
#include "Views/SignalFlowNodes.h"
#include "Views/SSignalFlowGraphStyle.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"

#if WITH_EDITOR
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#endif // WITH_EDITOR

#if MUTE_SOLO_ENABLED
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#endif // MUTE_SOLO_ENABLED

namespace UE::Audio::Insights
{
	void SSignalFlowGraph::Construct(const SSignalFlowGraph::FArguments& InArgs)
	{
		ListSource = InArgs._ListSource;
		NodeDetailFilterSettings = InArgs._NodeDetailFilterSettings;
		SelectedItem = InArgs._SelectedItem;
		HighlightedItem = InArgs._HighlightedItem;
		ActiveAudioDeviceEntry = InArgs._ActiveAudioDeviceEntry;
		FocusedItem = InArgs._FocusedItem;
		HorizontalScrollBox = InArgs._HorizontalScrollBox;
		VerticalScrollBox = InArgs._VerticalScrollBox;
		GraphJustification = InArgs._GraphJustification;
		ShowNodeDetails = InArgs._ShowNodeDetails;
		DisplayAmpPeakInDb = InArgs._DisplayAmpPeakInDb;
		AnimateWires = InArgs._AnimateWires;
		HighlightPathActive = InArgs._HighlightPathActive;
		LargeNodePadding = InArgs._LargeNodePadding;
		SmallNodePadding = InArgs._SmallNodePadding;
		OnNodeSelected = InArgs._OnNodeSelected;
		IsEntryFilteredOutByText = InArgs._IsEntryFilteredOutByText;
		IsInHighlightedPath = InArgs._IsInHighlightedPath;

		Clipping = EWidgetClipping::ClipToBounds;

		Canvas = SNew(SCanvas);

		RefreshGraph();

		ChildSlot
		[
			SNew(SScaleBox)
			.Stretch(EStretch::UserSpecified)
			.StretchDirection(EStretchDirection::Both)
			.UserSpecifiedScale_Lambda([this]()
			{
				return ZoomScaleFactor;
			})
			.PixelSnappingMethod(EWidgetPixelSnapping::Disabled)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Content()
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					Canvas.ToSharedRef()
				]
			]
		];
	}

	void SSignalFlowGraph::RefreshGraph()
	{
		using namespace SSignalFlowGraphStyle;

		if (ListSource == nullptr)
		{
			return;
		}

		// Keep track of any node widgets that were not detected in the new graph
		// We will clean these up at the end
		TSet<FSignalFlowEntryKey> RemainingNodeKeys;
		NodeWidgets.GetKeys(RemainingNodeKeys);

		DummyToRealConnectionKeys.Reset();

		const FVector2D OriginalSize = GetDesiredSize();
		const FGraphLimits NewGraphWidthLimits = GetGraphWidthLimits();
		TArray<TreeDepthPair> NewTreeDepthOrder;

		CreateGraphNodes(RemainingNodeKeys, NewGraphWidthLimits, NewTreeDepthOrder);
		CreateSendLabels();
		CleanUpOldWidgets(RemainingNodeKeys);

		UpdateContentPadding(NewGraphWidthLimits, NewTreeDepthOrder);

		MaintainOldViewportPosition(OriginalSize, NewGraphWidthLimits, NewTreeDepthOrder);

		CachedGraphWidthLimits = NewGraphWidthLimits;
		CachedTreeDepthPairs = NewTreeDepthOrder;
	}

	void SSignalFlowGraph::ResetGraph()
	{
		if (Canvas.IsValid())
		{
			Canvas->ClearChildren();
		}

		NodeWidgets.Reset();
		DummyToRealConnectionKeys.Reset();
		CachedGraphWidthLimits = FGraphLimits();
		CachedTreeDepthPairs.Reset();
		ContentPadding = FVector2D::ZeroVector;
	}

	void SSignalFlowGraph::ToggleOrientation()
	{
		Orientation = (Orientation == EOrientation::Orient_Vertical) ? EOrientation::Orient_Horizontal : EOrientation::Orient_Vertical;

		RefreshGraph();
	}

	void SSignalFlowGraph::SetWireAnimationSettings(const float AmpPowerFactor, const float WireThicknessScalarMax)
	{
		WireSplineAmplitudePowerFactor = AmpPowerFactor;
		WireSplineThicknessScalarMax = WireThicknessScalarMax;
	}

	void SSignalFlowGraph::ResetZoom()
	{
		bAnimateZoom = true;
	}

	TSharedPtr<SSignalFlowGraphNode> SSignalFlowGraph::FindNodeWidget(const FSignalFlowEntryKey& NodeEntryKey) const
	{
		const TSharedPtr<SSignalFlowGraphNode>* NodeWidget = NodeWidgets.Find(NodeEntryKey);
		if (NodeWidget && NodeWidget->IsValid())
		{
			return *NodeWidget;
		}

		return nullptr;
	}

	void SSignalFlowGraph::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		using namespace SSignalFlowGraphStyle;

		DetectAndHandleViewportResize();

		if (!bAnimateZoom)
		{
			return;
		}

		ZoomScaleFactor = FMath::FInterpTo(ZoomScaleFactor, DefaultZoomScale, InDeltaTime, AnimationSpeed);

		TSharedPtr<SSignalFlowGraphNode>* FocusedNodeWidget = nullptr;
		if (SelectedItem.Get().IsValid())
		{
			FocusedNodeWidget = NodeWidgets.Find(SelectedItem.Get()->GetSignalFlowEntryKey());
		}
		else if (ActiveAudioDeviceEntry.Get().IsValid())
		{
			FocusedNodeWidget = NodeWidgets.Find(ActiveAudioDeviceEntry.Get()->GetSignalFlowEntryKey());
		}

		if (FocusedNodeWidget == nullptr || !FocusedNodeWidget->IsValid())
		{
			bAnimateZoom = false;
			return;
		}

		const FVector2D SelectedNodeCenterPos = (*FocusedNodeWidget)->GetCachedPositionInGraph() + ((*FocusedNodeWidget)->GetDesiredSize() * 0.5f);

		EAnimationResult XScrollAnimProgress = EAnimationResult::InProgress;
		EAnimationResult YScrollAnimProgress = EAnimationResult::InProgress;

		const TSharedPtr<SScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
		if (HorizontalScrollBoxPinned.IsValid())
		{
			XScrollAnimProgress = AnimateScrollTo(HorizontalScrollBoxPinned.ToSharedRef(), SelectedNodeCenterPos, InDeltaTime);
		}

		const TSharedPtr<SScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();
		if (VerticalScrollBoxPinned.IsValid())
		{
			YScrollAnimProgress = AnimateScrollTo(VerticalScrollBoxPinned.ToSharedRef(), SelectedNodeCenterPos, InDeltaTime);
		}

		if (XScrollAnimProgress == EAnimationResult::Complete 
			&& YScrollAnimProgress == EAnimationResult::Complete 
			&& FMath::IsNearlyEqual(ZoomScaleFactor, DefaultZoomScale))
		{
			bAnimateZoom = false;
		}
	}

	int32 SSignalFlowGraph::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		LayerId = OnPaintConnections(AllottedGeometry, OutDrawElements, LayerId);

		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
	}

	FReply SSignalFlowGraph::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		using namespace SSignalFlowGraphStyle;

		const TSharedPtr<SSignalFlowScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
		const TSharedPtr<SSignalFlowScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();

		bAnimateZoom = false;

		const float OldZoomScaleFactor = ZoomScaleFactor;
		ZoomScaleFactor = FMath::Clamp(ZoomScaleFactor + (MouseEvent.GetWheelDelta() * ZoomStepFactor), MinZoomScale, MaxZoomScale);

		if (HorizontalScrollBoxPinned.IsValid() && VerticalScrollBoxPinned.IsValid())
		{
			const FVector2D MousePosInGraph = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

			const FVector2D ViewportOffsetFraction(HorizontalScrollBoxPinned->GetViewOffsetFraction(), VerticalScrollBoxPinned->GetViewOffsetFraction());
			const FVector2D ViewportPosInGraph = MyGeometry.GetLocalSize() * ViewportOffsetFraction;
			const FVector2D MousePosInViewport = MousePosInGraph - ViewportPosInGraph;

			const float ZoomScaleRatio = ZoomScaleFactor / OldZoomScaleFactor;
			const FVector2D CurrentScrollOffset(HorizontalScrollBoxPinned->GetScrollOffset(), VerticalScrollBoxPinned->GetScrollOffset());
			const FVector2D NewScrollOffset = ((MousePosInViewport + CurrentScrollOffset) * ZoomScaleRatio) - MousePosInViewport;

			HorizontalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewScrollOffset.X, 0.0f, HorizontalScrollBoxPinned->GetScrollOffsetOfEnd() * ZoomScaleRatio));
			VerticalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewScrollOffset.Y, 0.0f, VerticalScrollBoxPinned->GetScrollOffsetOfEnd() * ZoomScaleRatio));

			// Update padding after scroll correction so the desired size stays valid
			// for the next layout pass at the new zoom level. Compensate scroll offset
			// for the node position shift caused by the padding change.
			if (!FMath::IsNearlyEqual(ZoomScaleFactor, OldZoomScaleFactor))
			{
				ApplyContentPaddingAndCompensateScroll(HorizontalScrollBoxPinned, VerticalScrollBoxPinned);
			}
		}

		return FReply::Handled();
	}

	FReply SSignalFlowGraph::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::RightMouseButton);
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && SelectedItem.Get().IsValid())
		{
			OnNodeSelected.ExecuteIfBound(nullptr);

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FReply SSignalFlowGraph::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			bIsDraggingToMove = true;

			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		return FReply::Unhandled();
	}

	FReply SSignalFlowGraph::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (bIsDraggingToMove)
		{
			if (!MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
			{
				bIsDraggingToMove = false;

				return FReply::Handled().ReleaseMouseCapture();
			}

			const FVector2D MoveDelta = MouseEvent.GetCursorDelta();

			const TSharedPtr<SScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
			if (HorizontalScrollBoxPinned.IsValid())
			{
				const float NewOffsetX = HorizontalScrollBoxPinned->GetScrollOffset() - MoveDelta.X;
				HorizontalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewOffsetX, 0.0f, HorizontalScrollBoxPinned->GetScrollOffsetOfEnd()));
			}

			const TSharedPtr<SScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();
			if (VerticalScrollBoxPinned.IsValid())
			{
				const float NewOffsetY = VerticalScrollBoxPinned->GetScrollOffset() - MoveDelta.Y;
				VerticalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewOffsetY, 0.0f, VerticalScrollBoxPinned->GetScrollOffsetOfEnd()));
			}

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	bool SSignalFlowGraph::IsCursorNearSendLabel(const FVector2D& CursorPosInViewport, const FVector2D& SendLabelPos) const
	{
		using namespace SSignalFlowGraphStyle;

		// Offset from the wire connection point towards the center of the send label icon
		const float IconOffset = SendLabelVerticalWidth * ZoomScaleFactor * 0.5f;
		FVector2D IconCenter = SendLabelPos;
		if (Orientation == EOrientation::Orient_Horizontal)
		{
			IconCenter.X -= IconOffset;
		}
		else
		{
			IconCenter.Y -= IconOffset;
		}

		const float ScaledRadius = SendLabelHoverRadius * ZoomScaleFactor;
		const float DistanceSquared = FVector2D::DistSquared(CursorPosInViewport, IconCenter);

		return DistanceSquared <= (ScaledRadius * ScaledRadius);
	}

	FReply SSignalFlowGraph::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (bIsDraggingToMove && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			bIsDraggingToMove = false;

			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	FCursorReply SSignalFlowGraph::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		return bIsDraggingToMove ? FCursorReply::Cursor(EMouseCursor::GrabHandClosed) : FCursorReply::Unhandled();
	}

	int32 SSignalFlowGraph::OnPaintConnections(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		using namespace SSignalFlowGraphStyle;

		if (!ListSource || !HorizontalScrollBox.IsSet() || !VerticalScrollBox.IsSet())
		{
			return LayerId;
		}

		TSharedPtr<SScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
		TSharedPtr<SScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();
		if (!HorizontalScrollBoxPinned.IsValid() || !VerticalScrollBoxPinned.IsValid())
		{
			return LayerId;
		}

		const FGeometry& ViewportGeometry = VerticalScrollBoxPinned->GetPaintSpaceGeometry();

		const FVector2D ViewportOffsetFraction(HorizontalScrollBoxPinned->GetViewOffsetFraction(), VerticalScrollBoxPinned->GetViewOffsetFraction());
		const FVector2D ViewportPosInGraph = AllottedGeometry.GetLocalSize() * ViewportOffsetFraction;

		FBox2D LocalBoundsBox(ForceInit);
		LocalBoundsBox += FVector2D::Zero();
		LocalBoundsBox += ViewportGeometry.GetLocalSize();

		const bool bHighlightPathModeActive = HighlightPathActive.Get() && IsInHighlightedPath.IsBound();

		const FVector2D CursorPosInViewport = VerticalScrollBoxPinned->GetCachedGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());

		for (const TSharedPtr<ISignalFlowNode>& Node : *ListSource)
		{
			if (!Node->IsRealNode())
			{
				continue;
			}

			const TSharedPtr<SSignalFlowGraphNode>* InputNode = NodeWidgets.Find(Node->GetEntryKey());

			if (InputNode == nullptr || !InputNode->IsValid())
			{
				continue;
			}

			const TSharedPtr<FSignalFlowDashboardEntry> Entry = (*InputNode)->GetEntry();

			// Draw any source bus connections first (flowing backwards through the graph)
			for (int32 SourceBusIndex = 0; SourceBusIndex < Node->FilteredLinkedSoundSources.Num(); ++SourceBusIndex)
			{
				const FSignalFlowEntryKey& LinkedSourceKey = Node->FilteredLinkedSoundSources[SourceBusIndex];
				const TSharedPtr<SSignalFlowGraphNode>* LinkedSourceNode = NodeWidgets.Find(LinkedSourceKey);

				if (LinkedSourceNode && LinkedSourceNode->IsValid())
				{
					constexpr float SendLevel = 1.0f;
					const int32 SendLabelIndex = SourceBusIndex + Node->FilteredOutputs.Num();
					constexpr bool bIsSourceBusConnection = true;

					LayerId = PaintConnectionToNode(AllottedGeometry,
													OutDrawElements,
													LayerId,
													Entry,
													**InputNode,
													**LinkedSourceNode,
													LocalBoundsBox,
													ViewportPosInGraph,
													CursorPosInViewport,
													SendLevel,
													SendLabelIndex,
													bIsSourceBusConnection,
													bHighlightPathModeActive);
				}
			}

			// Draw any bus patch output connections
			for (int32 BusPatchOutputIndex = 0; BusPatchOutputIndex < Node->FilteredLinkedBusPatchOutputs.Num(); ++BusPatchOutputIndex)
			{
				const FSignalFlowEntryKey& BusPatchOutputKey = Node->FilteredLinkedBusPatchOutputs[BusPatchOutputIndex];
				const TSharedPtr<SSignalFlowGraphNode>* BusPatchOutputNode = NodeWidgets.Find(BusPatchOutputKey);

				if (BusPatchOutputNode && BusPatchOutputNode->IsValid())
				{
					constexpr float SendLevel = 1.0f;
					const int32 SendLabelIndex = BusPatchOutputIndex + Node->FilteredOutputs.Num() + Node->FilteredLinkedSoundSources.Num();
					const bool bIsBusConnection = Entry.IsValid() && Entry->EntryType == ESignalFlowEntryType::AudioBus;

					LayerId = PaintConnectionToNode(AllottedGeometry,
													OutDrawElements,
													LayerId,
													Entry,
													**InputNode,
													**BusPatchOutputNode,
													LocalBoundsBox,
													ViewportPosInGraph,
													CursorPosInViewport,
													SendLevel,
													SendLabelIndex,
													bIsBusConnection,
													bHighlightPathModeActive);
				}
			}

			// Now draw any output connections (flowing forwards through the graph)
			for (int32 OutputIndex = 0; OutputIndex < Node->FilteredOutputs.Num(); ++OutputIndex)
			{
				const FSignalFlowEntryKey& NodeOutputKey = Node->FilteredOutputs[OutputIndex];
				const TSharedPtr<SSignalFlowGraphNode> OutputNode = FindRealNode(NodeOutputKey, ENodeConnectionType::Output);

				if (OutputNode.IsValid())
				{
					FSignalFlowOutputData* OutputData = Entry.IsValid() ? Entry->Outputs.Find(OutputNode->GetNodeEntryKey()) : nullptr;
					const float SendLevel = OutputData && OutputData->SendLevel.IsSet() ? OutputData->SendLevel.GetValue() : 1.0f;

					constexpr bool bIsSourceBusConnection = false;

					LayerId = PaintConnectionToNode(AllottedGeometry,
													OutDrawElements,
													LayerId,
													Entry,
													**InputNode,
													*OutputNode,
													LocalBoundsBox,
													ViewportPosInGraph,
													CursorPosInViewport,
													SendLevel,
													OutputIndex,
													bIsSourceBusConnection,
													bHighlightPathModeActive);
				}
			}
		}

		return LayerId;
	}

	int32 SSignalFlowGraph::PaintConnectionToNode(const FGeometry& AllottedGeometry,
												  FSlateWindowElementList& OutDrawElements,
												  int32 LayerId,
												  const TSharedPtr<FSignalFlowDashboardEntry>& SenderEntry,
												  const SSignalFlowGraphNode& SenderNode,
												  const SSignalFlowGraphNode& ReceiverNode,
												  const FBox2D& LocalBoundsBox,
												  const FVector2D& ViewportPosInGraph,
												  const FVector2D& CursorPosInViewport,
												  const float SendLevel,
												  const int32 SendLabelIndex,
												  const bool bIsSourceBusConnection,
												  const bool bHighlightPathModeActive) const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D SendPos = SenderNode.GetLocalSendLabelPos(ReceiverNode.GetNodeEntryKey(), SendLabelIndex);
		const FVector2D SendPosInView = SendPos - ViewportPosInGraph;

		// Check if the node connection wire is inside the viewport
		const FVector2D ReceiverPos = ReceiverNode.GetLocalInputPos();
		const FVector2D ReceiverPosInView = ReceiverPos - ViewportPosInGraph;
		if (!FMath::LineBoxIntersection2D(LocalBoundsBox, SendPosInView, ReceiverPosInView))
		{
			return LayerId;
		}

		const bool bConnectionIsOnHighlightedPath = !bHighlightPathModeActive
											   || (IsInHighlightedPath.Execute(SenderNode.GetNodeEntryKey())
												&& IsInHighlightedPath.Execute(ReceiverNode.GetNodeEntryKey()));

		const bool bIsSendLabelHovered = IsCursorNearSendLabel(CursorPosInViewport, SendPosInView);

		if ((bConnectionIsOnHighlightedPath || bIsSendLabelHovered) && AnimateWires.Get() && SenderEntry.IsValid() && SenderEntry->Amplitude.IsSet() && !SenderEntry->IsEntryAmplitudeWindowSilent())
		{
			LayerId = AnimateWire(AllottedGeometry, OutDrawElements, LayerId, SenderEntry, ReceiverNode.GetNodeEntryKey().EntryType, SendPos, ReceiverPos, SendLevel, bIsSourceBusConnection, bIsSendLabelHovered);
		}
		else
		{
			const float ThicknessMultiplier = bIsSendLabelHovered ? HoveredWireThicknessMultiplier : 1.0f;
			const float ScaledSplineThickness = FMath::Max(SplineThicknessDefault * ZoomScaleFactor * ThicknessMultiplier, SplineThicknessMin);
			FLinearColor ConnectionColor = GetConnectionColor(ReceiverNode.GetNodeEntryKey().EntryType, 0.0f, WireSplineAmplitudePowerFactor, bIsSourceBusConnection, bIsSendLabelHovered);

			if (!bConnectionIsOnHighlightedPath && !bIsSendLabelHovered)
			{
				ConnectionColor.A *= NonHighlightedPathAlphaFactor;
			}

			FSlateDrawElement::MakeSpline(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(),
				SendPos,
				GetSplineStartDirection(Orientation, bIsSourceBusConnection, ZoomScaleFactor),
				ReceiverPos,
				GetSplineEndDirection(Orientation, bIsSourceBusConnection, ZoomScaleFactor),
				ScaledSplineThickness,
				ESlateDrawEffect::NoPixelSnapping,
				ConnectionColor
			);
		}

		return LayerId;
	}

	int32 SSignalFlowGraph::AnimateWire(const FGeometry& AllottedGeometry,
										FSlateWindowElementList& OutDrawElements,
										int32 LayerId,
										const TSharedPtr<FSignalFlowDashboardEntry>& Entry,
										const ESignalFlowEntryType ReceiverEntryType,
										const FVector2D& ConnectionStartPos,
										const FVector2D& ConnectionEndPos,
										const float SendLevel,
										const bool bIsSourceBusConnection,
										const bool bIsSendLabelHovered) const
	{
		using namespace SSignalFlowGraphStyle;
		const float ThicknessMultiplier = bIsSendLabelHovered ? HoveredWireThicknessMultiplier : 1.0f;
		const float HeightMultiplier = bIsSendLabelHovered ? HoveredWireHeightMultiplier : 1.0f;

		if (!Entry.IsValid())
		{
			return LayerId;
		}

		// Use the straight line distance between the two points as an approximation of how long the spline length will be
		const float DistanceBetweenConnections = FVector2D::Distance(ConnectionStartPos, ConnectionEndPos);
		const float PointsPerUnit = bIsSourceBusConnection ? BackwardSplineAnimPointsPerUnit : SplineAnimPointsPerUnit;
		const int32 NumPoints = (DistanceBetweenConnections * PointsPerUnit);
		if (NumPoints <= 0)
		{
			return LayerId;
		}

		const uint32 NumAmpReadings = Entry->AmplitudeDataRange.Capacity();
		const float AmpIndexDelta = static_cast<float>(NumAmpReadings) / static_cast<float>(NumPoints);

		// Reuse member scratch buffers across wires/frames to avoid heap allocation churn.
		// Reset preserves capacity, so steady-state paint is allocation-free once buffers grow.
		AnimateWireTopPoints.Reset(NumPoints * 2);
		AnimateWireBottomPoints.Reset(NumPoints * 2);

		const int32 AmpWriteIndex = Entry->AmplitudeWriteIndex;

		int32 ReadIndex = Entry->AmplitudeDataRange.GetPreviousIndex(AmpWriteIndex);
		uint32 AmpReadingCount = 0u;

		const FVector2D StartDirection = GetSplineStartDirection(Orientation, bIsSourceBusConnection, ZoomScaleFactor);
		const FVector2D EndDirection = GetSplineEndDirection(Orientation, bIsSourceBusConnection, ZoomScaleFactor);
		FVector2D LastPointPos = ConnectionStartPos;

		float InterpolationAmount = 0.0f;
		for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			if (AmpReadingCount >= NumAmpReadings)
			{
				break;
			}

			const float CurrentAmp = Entry->AmplitudeDataRange[ReadIndex] * SendLevel;
			const float Prop = static_cast<float>(PointIndex) / static_cast<float>(NumPoints);

			CollectAmplitudeConnectionPoints(CurrentAmp,
											 ConnectionStartPos,
											 StartDirection,
											 ConnectionEndPos,
											 EndDirection,
											 Prop,
											 AnimateWireTopPoints,
											 AnimateWireBottomPoints,
											 LastPointPos,
											 HeightMultiplier);

			InterpolationAmount += AmpIndexDelta;
			while (InterpolationAmount >= 1.0f)
			{
				InterpolationAmount -= 1.0f;
				ReadIndex = Entry->AmplitudeDataRange.GetPreviousIndex(ReadIndex);
				++AmpReadingCount;
			}
		}

		const FLinearColor ConnectionColor(GetConnectionColor(ReceiverEntryType, Entry->Amplitude.GetValue(), WireSplineAmplitudePowerFactor, bIsSourceBusConnection, bIsSendLabelHovered));
		const float AnimatedThickness = SplineAnimatedWireThickness * ZoomScaleFactor * ThicknessMultiplier;

		FSlateDrawElement::MakeLines(OutDrawElements,
									 ++LayerId,
									 AllottedGeometry.ToPaintGeometry(),
									 AnimateWireTopPoints,
									 ESlateDrawEffect::None,
									 ConnectionColor,
									 true, /* bAntialias */
									 AnimatedThickness);

		FSlateDrawElement::MakeLines(OutDrawElements,
									 ++LayerId,
									 AllottedGeometry.ToPaintGeometry(),
									 AnimateWireBottomPoints,
									 ESlateDrawEffect::None,
									 ConnectionColor,
									 true, /* bAntialias */
									 AnimatedThickness);

		return LayerId;
	}

	void SSignalFlowGraph::CollectAmplitudeConnectionPoints(const float Amplitude,
															const FVector2D& ConnectionStartPos,
															const FVector2D& StartDirection,
															const FVector2D& ConnectionEndPos,
															const FVector2D& EndDirection,
															const float Prop,
															TArray<FVector2D>& OutTopPoints,
															TArray<FVector2D>& OutBottomPoints,
															FVector2D& OutLastPoint,
															const float HeightMultiplier) const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D SplinePoint = FMath::CubicInterp(ConnectionStartPos,
														 StartDirection,
														 ConnectionEndPos,
														 EndDirection,
														 Prop);

		const float SplineThickness = FMath::Lerp(SplineThicknessDefault * SplineThicknessScalarMin, SplineThicknessDefault * WireSplineThicknessScalarMax, FMath::Pow(Amplitude, WireSplineAmplitudePowerFactor)) * ZoomScaleFactor;
		const float PointHeight = SplineThickness * 0.25f * FMath::Min(ZoomScaleFactor, 1.0f) * HeightMultiplier; // Collapse wires closer to single points when far away

		const FVector2D LastTangent(SplinePoint.X - OutLastPoint.X, SplinePoint.Y - OutLastPoint.Y);
		const FVector2D Normal = FVector2D(LastTangent.Y, -1.0f * LastTangent.X).GetSafeNormal();

		const FVector2D Top = { SplinePoint.X - (Normal.X * PointHeight), SplinePoint.Y - (Normal.Y * PointHeight) };
		const FVector2D Bottom = { SplinePoint.X + (Normal.X * PointHeight), SplinePoint.Y + (Normal.Y * PointHeight) };

		if (Top.Equals(Bottom))
		{
			OutTopPoints.Add(Top);
			OutBottomPoints.Add(Bottom);
		}
		else
		{
			OutTopPoints.Add(Top);
			OutTopPoints.Add(Bottom);
			OutBottomPoints.Add(Bottom);
			OutBottomPoints.Add(Top);
		}

		OutLastPoint = SplinePoint;
	}

	FVector2D SSignalFlowGraph::ComputeDesiredSize(float InLayoutScaleMultiplier) const
	{
		const FVector2D RawContentSize = GetRawContentSize(CachedGraphWidthLimits, CachedTreeDepthPairs);
		const FVector2D PaddedSize = RawContentSize + (ContentPadding * 2.0f);

		return PaddedSize * InLayoutScaleMultiplier * ZoomScaleFactor;
	}

	SSignalFlowGraph::FGraphLimits SSignalFlowGraph::GetGraphWidthLimits() const
	{
		FGraphLimits GraphWidthLimits;

		if (ListSource)
		{
			// Gather stats on the width of the graph using only real nodes.
			// Dummy nodes are invisible and should not inflate the graph canvas size.
			for (const TSharedPtr<ISignalFlowNode>& Node : *ListSource)
			{
				if (!Node.IsValid() || !Node->IsRealNode())
				{
					continue;
				}

				const float XPos = Node->XPos;

				GraphWidthLimits.MinPos = FMath::Min(GraphWidthLimits.MinPos, XPos);
				GraphWidthLimits.MaxPos = FMath::Max(GraphWidthLimits.MaxPos, XPos);
			}
		}

		return GraphWidthLimits;
	}

	float SSignalFlowGraph::GetNodeGraphSlotWidth() const
	{
		using namespace SSignalFlowGraphStyle;

		const float XPaddingBetweenNodes = Orientation == EOrientation::Orient_Horizontal
										 ? LargeNodePadding.Get(LargeBetweenNodesDefaultPadding)
										 : SmallNodePadding.Get(SmallBetweenNodesDefaultPadding);

		return NodeMaxWidth + XPaddingBetweenNodes;
	}

	float SSignalFlowGraph::GetNodeGraphSlotHeight() const
	{
		using namespace SSignalFlowGraphStyle;

		// The max height of a node in the graph is driven by either the number of visible node details, or the number of send label outputs
		const TSet<ESignalFlowNodeDetailParam> IgnoredParams{ ESignalFlowNodeDetailParam::SendOutputVolume };
		const int32 NumParamsVisible = NodeDetailFilterSettings ? NodeDetailFilterSettings->GetNumVisibleParams(IgnoredParams) : 0;

		const bool bSendLabelsAreVisible = ShowNodeDetails.Get() && NodeDetailFilterSettings && NodeDetailFilterSettings->GetParameterIsVisible(ESignalFlowNodeDetailParam::SendOutputVolume);
		const int32 MaxNumVisibleSendLabels = bSendLabelsAreVisible ? MaxNumNodeOutputsInGraph : 0;

		const int32 MaxNumParamRows = FMath::Max(NumParamsVisible, MaxNumVisibleSendLabels);

		const float YPaddingBetweenNodes = Orientation == EOrientation::Orient_Horizontal
										 ? SmallNodePadding.Get(SmallBetweenNodesDefaultPadding)
										 : LargeNodePadding.Get(LargeBetweenNodesDefaultPadding);

		return GetNodeMaxHeight(MaxNumParamRows, ShowNodeDetails.Get(), ZoomScaleFactor) + YPaddingBetweenNodes;
	}

	void SSignalFlowGraph::CreateGraphNodes(TSet<FSignalFlowEntryKey>& OutRemainingNodeKeys, const FGraphLimits& GraphWidthLimits, TArray<TreeDepthPair>& OutTreeDepthStructure)
	{
		TreeDepthPair CurrentTreeDepth{ ESignalFlowEntryType::InvalidLow, TNumericLimits<int32>::Lowest() };
		OutTreeDepthStructure.Reset();

		// Create the nodes first
		for (const TSharedPtr<ISignalFlowNode>& Node : *ListSource)
		{
			if (!Node.IsValid())
			{
				continue;
			}

			if (CurrentTreeDepth != Node->GetTreeDepth())
			{
				CurrentTreeDepth = Node->GetTreeDepth();

				// Each tree depth layer is used to assign a YPosition to each node
				// Each time we cross a layer, add one to the depth tracker
				// OutTreeDepthStructure tracks the unique tree depth layers inside the graph
				OutTreeDepthStructure.Add(CurrentTreeDepth);
			}

			CreateGraphNode(Node, GraphWidthLimits.MinPos, GraphWidthLimits.MaxPos, OutTreeDepthStructure.Num() - 1, OutRemainingNodeKeys);
		}
	}

	void SSignalFlowGraph::CreateSendLabels()
	{
		MaxNumNodeOutputsInGraph = 0;

		// Run through the list source again to create send label widgets between nodes
		for (const TSharedPtr<ISignalFlowNode>& Node : *ListSource)
		{
			if (!Node.IsValid() || !Node->IsRealNode())
			{
				continue;
			}

			const TSharedPtr<SSignalFlowGraphNode> NodeWidget = FindNodeWidget(Node->GetEntryKey());
			if (!NodeWidget.IsValid())
			{
				continue;
			}

			// Build the set of receiver keys in the same order as OnPaintConnections expects:
			// FilteredOutputs first, then LinkedSoundSources, then BusPatchOutputs
			TSet<FSignalFlowEntryKey> ReceiverKeys;

			for (const FSignalFlowEntryKey& OutputKey : Node->FilteredOutputs)
			{
				const TOptional<FSignalFlowEntryKey> RealNodeOutputKey = FindRealNodeKey(OutputKey, ENodeConnectionType::Output);
				if (RealNodeOutputKey.IsSet())
				{
					ReceiverKeys.Add(RealNodeOutputKey.GetValue());
				}
			}

			for (const FSignalFlowEntryKey& LinkedSoundSourceKey : Node->FilteredLinkedSoundSources)
			{
				ReceiverKeys.Add(LinkedSoundSourceKey);
			}

			for (const FSignalFlowEntryKey& BusPatchOutputKey : Node->FilteredLinkedBusPatchOutputs)
			{
				ReceiverKeys.Add(BusPatchOutputKey);
			}

			NodeWidget->SetSendLabelWidgets(ReceiverKeys);

			MaxNumNodeOutputsInGraph = FMath::Max(MaxNumNodeOutputsInGraph, Node->FilteredOutputs.Num());
		}
	}

	void SSignalFlowGraph::CleanUpOldWidgets(const TSet<FSignalFlowEntryKey>& NodeKeysToRemove)
	{
		// Clean up and remove any old nodes that we did not detect in the new graph
		for (const FSignalFlowEntryKey& Key : NodeKeysToRemove)
		{
			TSharedPtr<SSignalFlowGraphNode>* RemovedNode = NodeWidgets.Find(Key);
			if (!RemovedNode || !RemovedNode->IsValid())
			{
				NodeWidgets.Remove(Key);
				continue;
			}

			Canvas->RemoveSlot(RemovedNode->ToSharedRef());

			const TSharedPtr<FSignalFlowDashboardEntry> RemovedEntry = (*RemovedNode)->GetEntry();
			if (RemovedEntry.IsValid())
			{
				for (const FSignalFlowEntryKey& InputKey : RemovedEntry->Inputs)
				{
					const TOptional<FSignalFlowEntryKey> RealInputKey = FindRealNodeKey(InputKey, ENodeConnectionType::Input);
					if (!RealInputKey.IsSet())
					{
						continue;
					}

					TSharedPtr<SSignalFlowGraphNode>* InputNode = NodeWidgets.Find(RealInputKey.GetValue());
					if (InputNode == nullptr || !InputNode->IsValid())
					{
						continue;
					}

					(*InputNode)->RemoveSendLabelWidget(Key);
				}

				if (RemovedEntry->LinkedSourceBus.IsSet())
				{
					TSharedPtr<SSignalFlowGraphNode>* SourceBusNode = NodeWidgets.Find(RemovedEntry->LinkedSourceBus.GetValue());

					if (SourceBusNode && SourceBusNode->IsValid())
					{
						(*SourceBusNode)->RemoveSendLabelWidget(Key);
					}
				}

				// Clean up bus patch input connections
				for (const FSignalFlowEntryKey& BusPatchInputKey : RemovedEntry->LinkedBusPatchInputs)
				{
					TSharedPtr<SSignalFlowGraphNode>* BusNode = NodeWidgets.Find(BusPatchInputKey);
					if (BusNode && BusNode->IsValid())
					{
						(*BusNode)->RemoveSendLabelWidget(Key);
					}
				}

				// Clean up bus patch output connections
				for (const FSignalFlowEntryKey& BusPatchOutputKey : RemovedEntry->LinkedBusPatchOutputs)
				{
					TSharedPtr<SSignalFlowGraphNode>* SourceNode = NodeWidgets.Find(BusPatchOutputKey);
					if (SourceNode && SourceNode->IsValid())
					{
						(*SourceNode)->RemoveSendLabelWidget(Key);
					}
				}
			}

			NodeWidgets.Remove(Key);
		}
	}

	FVector2D SSignalFlowGraph::ComputeNodePositionInGraph(const SSignalFlowGraphNode& Node) const
	{
		const FVector2D NodeSize = Node.GetDesiredSize();
		const float NodeGraphSlotWidth = GetNodeGraphSlotWidth();
		const float NodeGraphSlotHeight = GetNodeGraphSlotHeight();
		const float XCenter = ((NodeGraphSlotWidth - NodeSize.X) * 0.5f) + ContentPadding.X;
		const float YCenter = ((NodeGraphSlotHeight - NodeSize.Y) * 0.5f) + ContentPadding.Y;

		if (Orientation == EOrientation::Orient_Vertical)
		{
			return FVector2D(
				(Node.GetXPos() * NodeGraphSlotWidth) + XCenter,
				(Node.GetYPos() * NodeGraphSlotHeight) + YCenter
			);
		}

		return FVector2D(
			(Node.GetYPos() * NodeGraphSlotWidth) + XCenter,
			(Node.GetXPos() * NodeGraphSlotHeight) + YCenter
		);
	}

	void SSignalFlowGraph::MaintainOldViewportPosition(const FVector2D& OriginalSize, const SSignalFlowGraph::FGraphLimits& NewGraphWidthLimits, const TArray<TreeDepthPair>& NewTreeDepthOrder)
	{
		const TSharedPtr<SSignalFlowScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
		const TSharedPtr<SSignalFlowScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();

		if (!HorizontalScrollBoxPinned.IsValid() || !VerticalScrollBoxPinned.IsValid())
		{
			return;
		}

		// Find the focused node to track. Its old cached position (from the previous layout pass)
		// and new position (from the just-updated XPos/YPos + ContentPadding) tell us exactly
		// how far the viewport needs to scroll to keep it in place.
		TSharedPtr<SSignalFlowGraphNode>* FocusedNodeWidget = nullptr;

		if (FocusedItem.Get().IsValid())
		{
			FocusedNodeWidget = NodeWidgets.Find(FocusedItem.Get()->GetSignalFlowEntryKey());
		}

		if (FocusedNodeWidget == nullptr || !FocusedNodeWidget->IsValid())
		{
			return;
		}

		// Old position: cached from the last layout pass (includes old ContentPadding)
		const FVector2D OldNodePos = (*FocusedNodeWidget)->GetCachedPositionInGraph();

		// New position: computed from the just-updated XPos/YPos and current ContentPadding
		const FVector2D NewNodePos = ComputeNodePositionInGraph(**FocusedNodeWidget);

		const FVector2D Offset = (NewNodePos - OldNodePos) * ZoomScaleFactor;

		const FVector2D OldScrollOffset(HorizontalScrollBoxPinned->GetScrollOffset(), VerticalScrollBoxPinned->GetScrollOffset());
		const FVector2D NewDesiredPos = OldScrollOffset + Offset;

		// Clamp to valid scroll range
		constexpr float LayoutScaleMultiplier = 1.0f;
		const FVector2D NewRawContentSize = GetRawContentSize(NewGraphWidthLimits, NewTreeDepthOrder);
		const FVector2D NewDesiredSize = (NewRawContentSize + (ContentPadding * 2.0f)) * LayoutScaleMultiplier * ZoomScaleFactor;
		const FVector2D CurrentScrollOffsetOfEnd(HorizontalScrollBoxPinned->GetScrollOffsetOfEnd(), VerticalScrollBoxPinned->GetScrollOffsetOfEnd());
		const FVector2D MaxScrollOffset = NewDesiredSize - (OriginalSize - CurrentScrollOffsetOfEnd);

		HorizontalScrollBoxPinned->ResetAnimatingScroll();
		VerticalScrollBoxPinned->ResetAnimatingScroll();

		HorizontalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewDesiredPos.X, 0.0f, FMath::Max(MaxScrollOffset.X, 0.0f)));
		VerticalScrollBoxPinned->SetScrollOffset(FMath::Clamp(NewDesiredPos.Y, 0.0f, FMath::Max(MaxScrollOffset.Y, 0.0f)));
	}

	FVector2D SSignalFlowGraph::GetRawContentSize(const FGraphLimits& GraphWidthLimits, const TArray<TreeDepthPair>& TreeDepthOrder) const
	{
		if (!GraphWidthLimits.IsValid())
		{
			return FVector2D::ZeroVector;
		}

		const float XExtent = GraphWidthLimits.MaxPos - GraphWidthLimits.MinPos + 1.0f;
		const float YExtent = static_cast<float>(TreeDepthOrder.Num());

		const float NodeGraphSlotWidth = GetNodeGraphSlotWidth();
		const float NodeGraphSlotHeight = GetNodeGraphSlotHeight();

		if (Orientation == Orient_Vertical)
		{
			return FVector2D(XExtent * NodeGraphSlotWidth, YExtent * NodeGraphSlotHeight);
		}

		return FVector2D(YExtent * NodeGraphSlotWidth, XExtent * NodeGraphSlotHeight);
	}

	void SSignalFlowGraph::UpdateContentPadding(const FGraphLimits& GraphWidthLimits, const TArray<TreeDepthPair>& TreeDepthOrder)
	{
		using namespace SSignalFlowGraphStyle;

		const TSharedPtr<SScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();
		if (!VerticalScrollBoxPinned.IsValid())
		{
			return;
		}

		const FVector2D ViewportSize = VerticalScrollBoxPinned->GetPaintSpaceGeometry().GetLocalSize();
		if (ViewportSize.IsNearlyZero())
		{
			return;
		}

		const FVector2D RawContentSize = GetRawContentSize(GraphWidthLimits, TreeDepthOrder);

		// Target: DesiredSize >= 2 * ViewportSize on each axis.
		// DesiredSize = (RawSize + 2 * Padding) * ZoomScaleFactor
		// Solving: Padding >= ViewportSize / ZoomScaleFactor - RawSize * 0.5
		if (ensure(!FMath::IsNearlyZero(ZoomScaleFactor)))
		{
			const float PaddingX = FMath::Max(GraphMinPadding, (ViewportSize.X / ZoomScaleFactor) - (RawContentSize.X * 0.5f));
			const float PaddingY = FMath::Max(GraphMinPadding, (ViewportSize.Y / ZoomScaleFactor) - (RawContentSize.Y * 0.5f));

			ContentPadding = FVector2D(PaddingX, PaddingY);
		}
	}

	void SSignalFlowGraph::ApplyContentPaddingAndCompensateScroll(const TSharedPtr<SSignalFlowScrollBox>& InHorizontalScrollBox, const TSharedPtr<SSignalFlowScrollBox>& InVerticalScrollBox)
	{
		check(InHorizontalScrollBox.IsValid() && InVerticalScrollBox.IsValid());

		const FVector2D OldPadding = ContentPadding;
		UpdateContentPadding(CachedGraphWidthLimits, CachedTreeDepthPairs);

		const FVector2D PaddingShift = (ContentPadding - OldPadding) * ZoomScaleFactor;

		InHorizontalScrollBox->SetScrollOffset(InHorizontalScrollBox->GetScrollOffset() + PaddingShift.X);
		InVerticalScrollBox->SetScrollOffset(InVerticalScrollBox->GetScrollOffset() + PaddingShift.Y);
	}

	void SSignalFlowGraph::DetectAndHandleViewportResize()
	{
		const TSharedPtr<SSignalFlowScrollBox> HorizontalScrollBoxPinned = HorizontalScrollBox.Get().Pin();
		const TSharedPtr<SSignalFlowScrollBox> VerticalScrollBoxPinned = VerticalScrollBox.Get().Pin();

		if (!HorizontalScrollBoxPinned.IsValid() || !VerticalScrollBoxPinned.IsValid())
		{
			return;
		}

		const FVector2D CurrentViewportSize = VerticalScrollBoxPinned->GetPaintSpaceGeometry().GetLocalSize();
		if (CurrentViewportSize.IsNearlyZero())
		{
			return;
		}

		if (CachedViewportSize.IsSet())
		{
			if (CurrentViewportSize.Equals(CachedViewportSize.GetValue()))
			{
				return;
			}

			ApplyContentPaddingAndCompensateScroll(HorizontalScrollBoxPinned, VerticalScrollBoxPinned);
		}

		CachedViewportSize = CurrentViewportSize;
	}

	void SSignalFlowGraph::CreateGraphNode(const TSharedPtr<ISignalFlowNode>& Node, const float MinXPos, const float MaxXPos, const int32 DepthTracker, TSet<FSignalFlowEntryKey>& OutRemainingNodeKeys)
	{
		using namespace SSignalFlowGraphStyle;

		if (!Node.IsValid())
		{
			return;
		}

		if (!Node->IsRealNode())
		{
			// If this is a fake node, keep a note of it's key and connection end point for fast lookup later
			const TSharedPtr<FDummyConnectionNode> FoundDummyCast = StaticCastSharedPtr<FDummyConnectionNode>(Node);
			if (FoundDummyCast.IsValid())
			{
				DummyToRealConnectionKeys.Add(FoundDummyCast->GetEntryKey(), { FoundDummyCast->ConnectionInputKey, FoundDummyCast->ConnectionOutputKey });
				return;
			}
		}
		
		const FSignalFlowEntryKey EntryKey = Node->GetEntryKey();

		// Avoid creating this node again if it already exists
		if (TSharedPtr<SSignalFlowGraphNode>* NodeWidget = NodeWidgets.Find(EntryKey))
		{
			if (ensure(NodeWidget->IsValid()))
			{
				// Update the X and Y positions of the node
				(*NodeWidget)->SetXPos(Node->XPos - MinXPos);
				(*NodeWidget)->SetYPos(static_cast<float>(DepthTracker));

				OutRemainingNodeKeys.Remove(EntryKey);

				TSet<FSignalFlowEntryKey> ActiveOutputKeys;
				ActiveOutputKeys.Reserve(Node->FilteredOutputs.Num()
										+ Node->FilteredLinkedSoundSources.Num()
										+ Node->FilteredLinkedBusPatchOutputs.Num());

				for (const FSignalFlowEntryKey& NodeOutputKey : Node->FilteredOutputs)
				{
					const TOptional<FSignalFlowEntryKey> RealNodeOutputKeyOpt = FindRealNodeKey(NodeOutputKey, ENodeConnectionType::Output);
					if (RealNodeOutputKeyOpt.IsSet())
					{
						ActiveOutputKeys.Add(RealNodeOutputKeyOpt.GetValue());
					}
				}

				for (const FSignalFlowEntryKey& LinkedSoundSourceKey : Node->FilteredLinkedSoundSources)
				{
					ActiveOutputKeys.Add(LinkedSoundSourceKey);
				}

				for (const FSignalFlowEntryKey& BusPatchOutputKey : Node->FilteredLinkedBusPatchOutputs)
				{
					ActiveOutputKeys.Add(BusPatchOutputKey);
				}

				(*NodeWidget)->SetSendLabelWidgets(ActiveOutputKeys);

				return;
			}
			else
			{
				NodeWidgets.Remove(EntryKey);
			}
		}

		const TSharedPtr<FSignalFlowEntryNode> EntryNode = StaticCastSharedPtr<FSignalFlowEntryNode>(Node);

		TSharedPtr<SSignalFlowGraphNode> NodeWidget = SNew(SSignalFlowGraphNode)
			.Entry(EntryNode->Entry)
			.NodeDetailFilterSettings(NodeDetailFilterSettings)
			.DisplayName(EntryNode->Entry.IsValid() ? EntryNode->Entry->GetDisplayName() : FText::GetEmpty())
			.PixelSnappingMethod(EWidgetPixelSnapping::SnapToPixel)
			.PositionInGraph_Lambda([this, EntryKey]() -> FVector2D
			{
				const TSharedPtr<SSignalFlowGraphNode>* Node = NodeWidgets.Find(EntryKey);
				if (Node && Node->IsValid())
				{
					return ComputeNodePositionInGraph(**Node);
				}

				return FVector2D::ZeroVector;
			})
			.XPos(Node->XPos - MinXPos)
			.YPos(static_cast<float>(DepthTracker))
			.IsSelected_Lambda([this, EntryKey]()
			{
				return (SelectedItem.IsBound() && SelectedItem.Get().IsValid() && SelectedItem.Get()->GetSignalFlowEntryKey() == EntryKey);
			})
			.IsHighlighted_Lambda([this, EntryKey]()
			{
				return (HighlightedItem.IsBound() && HighlightedItem.Get().IsValid() && HighlightedItem.Get()->GetSignalFlowEntryKey() == EntryKey);
			})
			.ShowNodeDetails_Lambda([this]()
			{
				return ShowNodeDetails.Get();
			})
			.IsFilteredOutByText_Lambda([this, EntryKey]()
			{
				const TSharedPtr<SSignalFlowGraphNode>* NodeWidgetPtr = NodeWidgets.Find(EntryKey);
				if (NodeWidgetPtr == nullptr || !NodeWidgetPtr->IsValid())
				{
					return false;
				}

				const TSharedPtr<FSignalFlowDashboardEntry> NodeEntry = (*NodeWidgetPtr)->GetEntry();
				return NodeEntry.IsValid() && IsEntryFilteredOutByText.IsBound() && IsEntryFilteredOutByText.Execute(*NodeEntry);
			})
			.DisplayAmpPeakInDb_Lambda([this]()
			{
				return DisplayAmpPeakInDb.Get();
			})
			.GraphOrientation_Lambda([this]()
			{
				return Orientation;
			})
			.OnNodeSelected_Lambda([this](TSharedPtr<FSignalFlowDashboardEntry> SelectedEntry)
			{
				OnNodeSelected.ExecuteIfBound(SelectedEntry);
			})
#if MUTE_SOLO_ENABLED
			.IsDirectlySoloed_Lambda([this, EntryKey]()
			{
				FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
				if (AudioDeviceManager == nullptr)
				{
					return false;
				}

				const TSharedPtr<SSignalFlowGraphNode>* NodeWidget = NodeWidgets.Find(EntryKey);
				if (NodeWidget == nullptr || !NodeWidget->IsValid())
				{
					return false;
				}

				return AudioDeviceManager->GetDebugger().IsSubmixExplicitlySoloed((*NodeWidget)->GetAssetFName());
			})
#endif // MUTE_SOLO_ENABLED
#if WITH_EDITOR
			.OnOpenContextMenu(this, &SSignalFlowGraph::OnRequestedContextMenu)
			.GraphIsDraggingToMove_Lambda([this]()
			{
				return bIsDraggingToMove;
			})
#endif // WITH_EDITOR
			.ZoomScaleFactor_Lambda([this]()
			{
				return ZoomScaleFactor;
			});

		NodeWidgets.Add(Node->GetEntryKey(), NodeWidget);

		Canvas->AddSlot()
			.Position_Lambda([this, EntryKey]() -> FVector2D
			{
				const TSharedPtr<SSignalFlowGraphNode>* Node = NodeWidgets.Find(EntryKey);
				if (Node && Node->IsValid())
				{
					(*Node)->UpdateNodeLayout();
					return (*Node)->GetCachedPositionInGraph();
				}

				return FVector2D::ZeroVector;
			})
			.Size_Lambda([this, EntryKey]() -> FVector2D
			{
				const TSharedPtr<SSignalFlowGraphNode>* Node = NodeWidgets.Find(EntryKey);
				if (Node && Node->IsValid())
				{
					return (*Node)->GetDesiredSize();
				}

				return FVector2D::ZeroVector;
			})
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				NodeWidget.ToSharedRef()
			];

		// SCanvas only invalidates Paint, not Prepass, so force a measure pass to avoid zero GetDesiredSize() on the first frame.
		NodeWidget->SlatePrepass(1.0f);
	}

	TSharedPtr<SSignalFlowGraphNode> SSignalFlowGraph::FindRealNode(const FSignalFlowEntryKey& EntryKey, const ENodeConnectionType ConnectionType) const
	{
		const TSharedPtr<SSignalFlowGraphNode>* FoundNode = NodeWidgets.Find(EntryKey);
		if (FoundNode && FoundNode->IsValid())
		{
			return *FoundNode;
		}
		else
		{
			// If this is a dummy node, try and find the real input/output node we are connected to instead
			const FConnectedNodeKeys* FoundRealConnectionKeys = DummyToRealConnectionKeys.Find(EntryKey);
			if (FoundRealConnectionKeys)
			{
				const FSignalFlowEntryKey& LookupKey = ConnectionType == ENodeConnectionType::Input ? FoundRealConnectionKeys->RealInputKey 
																										: FoundRealConnectionKeys->RealOutputKey;

				const TSharedPtr<SSignalFlowGraphNode>* FoundConnectedNode = NodeWidgets.Find(LookupKey);

				if (FoundConnectedNode && FoundConnectedNode->IsValid())
				{
					return *FoundConnectedNode;
				}
			}
		}

		return nullptr;
	}

	TOptional<FSignalFlowEntryKey> SSignalFlowGraph::FindRealNodeKey(const FSignalFlowEntryKey& EntryKey, const ENodeConnectionType ConnectionType) const
	{
		const TSharedPtr<SSignalFlowGraphNode> RealNode = FindRealNode(EntryKey, ConnectionType);
		if (RealNode.IsValid() && RealNode->GetEntry().IsValid())
		{
			return RealNode->GetNodeEntryKey();
		}

		return NullOpt;
	}

	SSignalFlowGraph::EAnimationResult SSignalFlowGraph::AnimateScrollTo(const TSharedRef<SScrollBox>& ScrollBox, const FVector2D& ScrollToPos, const float DeltaTime)
	{
		using namespace SSignalFlowGraphStyle;

		const FGeometry& ScrollBoxGeometry = ScrollBox->GetTickSpaceGeometry();
		const FVector2D CenterPos = ScrollBoxGeometry.GetLocalSize() * 0.5f;
		const FVector2D ScrollOffsetTarget = ScrollToPos - CenterPos;

		const EOrientation ScrollOrientation = ScrollBox->GetOrientation();
		const float TargetAlongOrientation = ScrollOrientation == EOrientation::Orient_Horizontal ? ScrollOffsetTarget.X : ScrollOffsetTarget.Y;

		const float ScrollOffset = FMath::FInterpTo(ScrollBox->GetScrollOffset(), TargetAlongOrientation, DeltaTime, AnimationSpeed);
		ScrollBox->SetScrollOffset(ScrollOffset);

		return FMath::IsNearlyEqual(TargetAlongOrientation, ScrollOffset, ScrollAnimInRangeThreshold) ? EAnimationResult::Complete : EAnimationResult::InProgress;
	}

#if WITH_EDITOR
	void SSignalFlowGraph::OnRequestedContextMenu(const FPointerEvent& MouseEvent, const TSharedPtr<FSignalFlowDashboardEntry> Entry)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		AssetContextMenuHelper.SetAssetEntry(Entry);

		const TSharedPtr<SWidget> MenuContent = AssetContextMenuHelper.ContructContextMenuOptions();
		if (MenuContent.IsValid())
		{
			const FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
			const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			GraphContextMenu = FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

			if (GraphContextMenu.IsValid())
			{
				GraphContextMenu->GetOnMenuDismissed().AddSPLambda(this, [this](TSharedRef<IMenu> InMenu)
				{
					GraphContextMenu.Reset();
				});
			}
			else
			{
				AssetContextMenuHelper.ResetAssetEntry();
			}
		}
		else
		{
			AssetContextMenuHelper.ResetAssetEntry();
		}
	}
#endif // WITH_EDITOR

} // namespace UE::Audio::Insights
