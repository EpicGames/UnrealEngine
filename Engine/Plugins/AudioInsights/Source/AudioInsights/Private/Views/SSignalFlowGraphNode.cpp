// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SSignalFlowGraphNode.h"

#include "AudioInsightsStyle.h"
#include "AudioInsightsUtils.h"
#include "Internationalization/Text.h"
#include "Messages/SignalFlowTraceMessages.h"
#include "Settings/SignalFlowSettings.h"
#include "Styling/StyleColors.h"
#include "Views/SSignalFlowGraphStyle.h"
#include "Views/SSignalFlowGraphSendLabel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#if MUTE_SOLO_ENABLED
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#endif // MUTE_SOLO_ENABLED

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	void SSignalFlowGraphNode::Construct(const SSignalFlowGraphNode::FArguments& InArgs)
	{
		using namespace SSignalFlowGraphStyle;

		Entry = InArgs._Entry;
		NodeDetailFilterSettings = InArgs._NodeDetailFilterSettings;
		PositionInGraph = InArgs._PositionInGraph;
		XPos = InArgs._XPos;
		YPos = InArgs._YPos;
		ZoomScaleFactor = InArgs._ZoomScaleFactor;
		IsSelected = InArgs._IsSelected;
		IsHighlighted = InArgs._IsHighlighted;
		ShowNodeDetails = InArgs._ShowNodeDetails;
		IsFilteredOutByText = InArgs._IsFilteredOutByText;
		DisplayAmpPeakInDb = InArgs._DisplayAmpPeakInDb;
		GraphOrientation = InArgs._GraphOrientation;
		OnNodeSelected = InArgs._OnNodeSelected;

#if WITH_EDITOR
		OnOpenContextMenu = InArgs._OnOpenContextMenu;
		GraphIsDraggingToMove = InArgs._GraphIsDraggingToMove;
#endif // WITH_EDITOR

		if (!Entry.IsValid())
		{
			return;
		}

		NodeEntryKey = Entry->GetSignalFlowEntryKey();

#if MUTE_SOLO_ENABLED
		IsDirectlySoloed = InArgs._IsDirectlySoloed;

		const FSoftObjectPath AssetPath(Entry->Name);
		if (AssetPath.IsValid())
		{
			AssetString = AssetPath.GetAssetName();
			AssetFName = AssetPath.GetAssetFName();
		}
		else
		{
			AssetString = Entry->Name;
			AssetFName = FName(Entry->Name);
		}
#endif // MUTE_SOLO_ENABLED

		ChildSlot
		[
			SAssignNew(SelectedNodeBorder, SBorder)
			.BorderImage(&GetNodeRoundedBoxBrush())
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.ColorAndOpacity_Lambda([this]()
			{
				return IsFilteredOutByText.Get(false) ? FLinearColor(NodeFilteredOutByTextDimming, NodeFilteredOutByTextDimming, NodeFilteredOutByTextDimming, 1.0f) : FLinearColor::White;
			})
			.BorderBackgroundColor_Lambda([this]()
			{
				if (IsSelected.IsBound() && IsSelected.Get())
				{
					return FStyleColors::AccentOrange.GetSpecifiedColor();
				}

				if (IsHighlighted.IsBound() && IsHighlighted.Get())
				{
					return FStyleColors::AccentPink.GetSpecifiedColor();
				}

				return FLinearColor::Transparent;
			})
			.Padding(SelectedNodeBorderThickness)
			[
				SAssignNew(NodeBorder, SBorder)
				.BorderImage(&GetNodeRoundedBoxBrush())
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.BorderBackgroundColor(GetNodeAccentColor(NodeEntryKey.EntryType))
				.Padding(NodeBorderThickness)
				.Content()
				[
					SNew(SBorder)
					.BorderImage(&GetNodeRoundedBoxBrush())
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.BorderBackgroundColor_Lambda([this]()
					{
						return (IsSelected.IsBound() && IsSelected.Get()) ? FStyleColors::Header : FStyleColors::Panel;
					})
					.Padding(0.0f, NodeBorderPadding, 0.0f, NodeBorderPadding)
					.Content()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.Padding(NodeBorderPadding, 0.0f, NodeBorderPadding, 0.0f)
						.AutoHeight()
						[
							CreateNodeName(InArgs)
						]

						+ SVerticalBox::Slot()
						.Padding(NodeBorderPadding + NodeDetailIndentationPadding, 0.0f, 0.0f, 0.0f)
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								CreateNodeDetails()
							]

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Top)
							.AutoWidth()
							[
								// In vertical mode, we want to ensure that we maintain padding on the RHS of the node when node details is on
								// We do this by drawing a box with no content inside.
								// In horizontal mode, this is replaced by SendLevelsVerticalBox, which is flush to the RHS
								SNew(SVerticalBox)
								.Visibility_Lambda([this]()
								{
									return GraphOrientation.Get() == EOrientation::Orient_Vertical && ShowNodeDetails.Get()
																									? EVisibility::Visible
																									: EVisibility::Collapsed;
								})

								+ SVerticalBox::Slot()
								[
									SNew(SBox)
									.WidthOverride(NodeBorderPadding)
								]
							]

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Top)
							.AutoWidth()
							[
								CreateSendOutputsVerticalContainer()
							]
						]

						+ SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Bottom)
						.AutoHeight()
						[
							CreateSendOutputsHorizontalContainer()
						]
					]
				]
			]
		];
	}

	FReply SSignalFlowGraphNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			if (Entry.IsValid())
			{
				const bool bShouldToggleOff = !MouseEvent.IsShiftDown() && IsSelected.IsBound() && IsSelected.Get();
				OnNodeSelected.ExecuteIfBound(bShouldToggleOff ? nullptr : Entry);
			}

			return FReply::Handled();
		}
		else
		{
			return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
		}
	}

#if WITH_EDITOR
	FReply SSignalFlowGraphNode::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (Entry.IsValid() && !GraphIsDraggingToMove.Get())
			{
				const TObjectPtr<UObject> ObjectPtr = Entry->GetObject();
				if (ObjectPtr && ObjectPtr->IsAsset())
				{
					OnOpenContextMenu.ExecuteIfBound(MouseEvent, Entry);
					return FReply::Handled();
				}
			}
		}

		return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
	}
#endif // WITH_EDITOR

	FVector2D SSignalFlowGraphNode::ComputeDesiredSize(float InLayoutScaleMultiplier) const
	{
		using namespace SSignalFlowGraphStyle;

		int32 NumParamsVisible = 0;
		if (NodeDetailFilterSettings)
		{
			if (bNodeDetailsAreExpanded)
			{
				const TSet<ESignalFlowNodeDetailParam> IgnoredParams { ESignalFlowNodeDetailParam::SendOutputVolume };
				NumParamsVisible = NodeDetailFilterSettings->GetNumVisibleParams(IgnoredParams);
			}
			// We always show amplitude if the node is collapsed
			else if (Entry.IsValid() && Entry->Amplitude.IsSet())
			{
				NumParamsVisible = 1;
			}
		}

		const int32 NumOutputPinsShowing = SendLabels.Num();
		const FVector2D Size = SCompoundWidget::ComputeDesiredSize(InLayoutScaleMultiplier);

		if (GraphOrientation.Get() == EOrientation::Orient_Horizontal)
		{
			const int32 MaxNumParamRows = FMath::Max(NumParamsVisible, NumOutputPinsShowing);

			const FVector2D ClampedSize = { FMath::Min(Size.X, NodeMaxWidth), FMath::Min(Size.Y, GetNodeMaxHeight(MaxNumParamRows, ShowNodeDetails.Get(), ZoomScaleFactor.Get())) };
			return ClampedSize;
		}
		else
		{
			const float PinSectionWidth = SendLevelsHorizontalBox.IsValid() ? SendLevelsHorizontalBox->GetDesiredSize().X : 0.0f;
			const float OutputPinSectionHeight = SendLevelsHorizontalBox.IsValid() ? SendLevelsHorizontalBox->GetDesiredSize().Y : 0.0f;

			// Make sure Max width/height calculations account for the bottom output pin row
			const float ScaledMaxWidth = FMath::Max(NodeMaxWidth, PinSectionWidth);
			const float TotalMaxHeight = GetNodeMaxHeight(NumParamsVisible, ShowNodeDetails.Get(), ZoomScaleFactor.Get()) + OutputPinSectionHeight;

			const FVector2D ClampedSize = { FMath::Min(Size.X, ScaledMaxWidth), FMath::Min(Size.Y, TotalMaxHeight) };
			return ClampedSize;
		}
	}

	FVector2D SSignalFlowGraphNode::GetLocalInputPos() const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D Size = GetDesiredSize();

		if (GraphOrientation.IsBound() && GraphOrientation.Get() == EOrientation::Orient_Horizontal)
		{
			// Input pos = left center of node
			return (FVector2D{ CachedPositionInGraph.X, CachedPositionInGraph.Y + (Size.Y * 0.5f) } * ZoomScaleFactor.Get())
				+ FVector2D{ SelectedNodeBorderThickness, 0.0f };
		}
		else
		{
			// Input pos = top center of node
			return (FVector2D{ CachedPositionInGraph.X + (Size.X * 0.5f), CachedPositionInGraph.Y } * ZoomScaleFactor.Get())
				+ FVector2D{ 0.0f, SelectedNodeBorderThickness };
		}
	}
	
	FVector2D SSignalFlowGraphNode::GetLocalCenteredOutputPos() const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D Size = GetDesiredSize();

		if (GraphOrientation.IsBound() && GraphOrientation.Get() == EOrientation::Orient_Horizontal)
		{
			// Out pos = right center of node
			return (FVector2D{ CachedPositionInGraph.X + Size.X, CachedPositionInGraph.Y + (Size.Y / 2.0f) } * ZoomScaleFactor.Get())
				- FVector2D{ SelectedNodeBorderThickness, 0.0f };
		}
		else
		{
			// Output pos = bottom center of node
			return (FVector2D{ CachedPositionInGraph.X + (Size.X / 2.0f), CachedPositionInGraph.Y + Size.Y } * ZoomScaleFactor.Get())
				- FVector2D{ 0.0f, SelectedNodeBorderThickness };
		}
	}

	FVector2D SSignalFlowGraphNode::GetLocalSendLabelPos(const FSignalFlowEntryKey& ReceiverEntryKey, const int32 LabelIndex) const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D OutputPos = GetLocalCenteredOutputPos();
		if (!NodeLabelHorizontalBox.IsValid() || !SendLabelsAreVisible())
		{
			return OutputPos;
		}

		const TSharedRef<SSignalFlowGraphSendLabel>* SendLabelWidget = SendLabels.Find(ReceiverEntryKey);
		if (SendLabelWidget == nullptr)
		{
			return OutputPos;
		}

		constexpr float BorderThicknessYOffset = (SelectedNodeBorderThickness * 0.5f) + (NodeBorderThickness * 0.5f);

		const FVector2D SendLabelWidgetSize = (*SendLabelWidget)->GetDesiredSize();

		float XPosInGraph;
		float YPosInGraph;

		if (GraphOrientation.Get() == EOrientation::Orient_Horizontal)
		{
			XPosInGraph = OutputPos.X;

			const float NodeLabelHeight = NodeLabelHorizontalBox->GetDesiredSize().Y + NodeBorderPadding;
			const float YOffsetToLabelAreaInGraph = CachedPositionInGraph.Y + NodeLabelHeight;
			const float YOffsetInLabelList = ((SendLabelWidgetSize.Y + NodeDetailVerticalPadding) * LabelIndex) + NodeDetailVerticalPadding;
			const float HalfLabelHeight = (SendLabelWidgetSize.Y * 0.5f);

			YPosInGraph = ((YOffsetToLabelAreaInGraph + YOffsetInLabelList + HalfLabelHeight) * ZoomScaleFactor.Get()) + BorderThicknessYOffset;
		}
		else
		{
			YPosInGraph = OutputPos.Y;

			const float SendLabelWidth = SendLabelWidgetSize.X + SendLabelPadding;
			const float XOffset = (SendLabelWidth * (LabelIndex + 1)) - ((SendLabelWidth * SendLabels.Num()) * 0.5f) - (SendLabelWidth * 0.5f);
			XPosInGraph = OutputPos.X + (XOffset * ZoomScaleFactor.Get());
		}

		const FVector2D SendLabelPosition(XPosInGraph, YPosInGraph);
		return SendLabelPosition;
	}

	int32 SSignalFlowGraphNode::GetNumSendLabels() const
	{
		return SendLabels.Num();
	}

	void SSignalFlowGraphNode::UpdateNodeLayout()
	{
		using namespace SSignalFlowGraphStyle;

		// Ensure border thickness does not get scaled to avoid rendering artifacts
		if (SelectedNodeBorder.IsValid())
		{
			SelectedNodeBorder->SetPadding(SelectedNodeBorderThickness / ZoomScaleFactor.Get());
		}

		if (NodeBorder.IsValid())
		{
			NodeBorder->SetPadding(NodeBorderThickness / ZoomScaleFactor.Get());
		}

		// Update the node's position in the graph
		CachedPositionInGraph = PositionInGraph.Get();
	}

	FVector2D SSignalFlowGraphNode::GetCachedPositionInGraph() const
	{
		return CachedPositionInGraph;
	}

	void SSignalFlowGraphNode::SetSendLabelWidgets(const TSet<FSignalFlowEntryKey>& ReceiverEntryKeys)
	{
		using namespace SSignalFlowGraphStyle;

		// Create any new send label widgets that don't already exist
		for (const FSignalFlowEntryKey& ReceiverKey : ReceiverEntryKeys)
		{
			CreateSendLabelWidget(ReceiverKey);
		}

		// Remove any send labels that are no longer needed
		for (auto Itr = SendLabels.CreateIterator(); Itr; ++Itr)
		{
			if (!ReceiverEntryKeys.Contains(Itr->Key))
			{
				Itr.RemoveCurrent();
			}
		}

		// Rebuild the layout box slots in the order provided by ReceiverEntryKeys
		// to ensure the visual order matches the wire painting order
		if (SendLevelsVerticalBox.IsValid())
		{
			SendLevelsVerticalBox->ClearChildren();
		}

		if (SendLevelsHorizontalBox.IsValid())
		{
			SendLevelsHorizontalBox->ClearChildren();
		}

		for (const FSignalFlowEntryKey& ReceiverKey : ReceiverEntryKeys)
		{
			const TSharedRef<SSignalFlowGraphSendLabel>* SendLabelWidget = SendLabels.Find(ReceiverKey);
			if (SendLabelWidget == nullptr)
			{
				continue;
			}

			if (SendLevelsVerticalBox.IsValid())
			{
				SendLevelsVerticalBox->AddSlot()
				.Padding(SendLabelPadding, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Right)
				[
					*SendLabelWidget
				];
			}

			if (SendLevelsHorizontalBox.IsValid())
			{
				SendLevelsHorizontalBox->AddSlot()
				.Padding(SendLabelPadding * 0.5f, NodeDetailVerticalPadding, SendLabelPadding * 0.5f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					*SendLabelWidget
				];
			}
		}
	}

	void SSignalFlowGraphNode::CreateSendLabelWidget(const FSignalFlowEntryKey& ReceiverEntryKey)
	{
		using namespace SSignalFlowGraphStyle;

		if (SendLabels.Contains(ReceiverEntryKey))
		{
			return;
		}

		TSharedPtr<SSignalFlowGraphSendLabel> SendLabelWidget = SNew(SSignalFlowGraphSendLabel)
			.PixelSnappingMethod(EWidgetPixelSnapping::SnapToPixel)
			.ToolTipText(LOCTEXT("SignalFlowGraph_NodeDetails_SendLabelTooltip", "Send Output Volume"))
			.ReceiverEntryType(ReceiverEntryKey.EntryType)
			.SendLevel_Lambda([this, ReceiverEntryKey]() -> TOptional<float>
			{
				if (Entry.IsValid())
				{
					const FSignalFlowOutputData* SendInfo = Entry->Outputs.Find(ReceiverEntryKey);
					if (SendInfo && SendInfo->SendLevel.IsSet())
					{
						return SendInfo->SendLevel;
					}
					else if (Entry->LinkedSoundSources.Contains(ReceiverEntryKey)
						  || Entry->LinkedBusPatchOutputs.Contains(ReceiverEntryKey)
						  || Entry->LinkedBusPatchInputs.Contains(ReceiverEntryKey))
					{
						// Source buses, bus patch outputs and bus patch inputs always have a send value of 1.0
						return 1.0f;
					}
				}

				return NullOpt;
			})
			.GraphOrientation(GraphOrientation)
			.ZoomScaleFactor(ZoomScaleFactor);

		SendLabels.Add(ReceiverEntryKey, SendLabelWidget.ToSharedRef());
	}

	void SSignalFlowGraphNode::RemoveSendLabelWidget(const FSignalFlowEntryKey& ReceiverEntryKey)
	{
		TSharedRef<SSignalFlowGraphSendLabel>* SendLabelWidget = SendLabels.Find(ReceiverEntryKey);
		if (SendLabelWidget == nullptr)
		{
			return;
		}

		if (SendLevelsVerticalBox.IsValid())
		{
			SendLevelsVerticalBox->RemoveSlot(*SendLabelWidget);
		}

		if (SendLevelsHorizontalBox.IsValid())
		{
			SendLevelsHorizontalBox->RemoveSlot(*SendLabelWidget);
		}

		SendLabels.Remove(ReceiverEntryKey);
	}

	bool SSignalFlowGraphNode::ContainsSendLabel(const FSignalFlowEntryKey& ReceiverEntryKey) const
	{
		return SendLabels.Contains(ReceiverEntryKey);
	}

	TSharedRef<SHorizontalBox> SSignalFlowGraphNode::CreateNodeName(const SSignalFlowGraphNode::FArguments& InArgs)
	{
		using namespace SSignalFlowGraphStyle;

		return SAssignNew(NodeLabelHorizontalBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FSlateStyle::Get().CreateIcon(InArgs._Entry->IconName).GetIcon())
				.ToolTipText(InArgs._DisplayName)
				.ColorAndOpacity(GetNodeAccentColor(NodeEntryKey.EntryType))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(InArgs._DisplayName)
				.ToolTipText(InArgs._DisplayName)
				.Justification(ETextJustify::Left)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.Clipping(EWidgetClipping::ClipToBounds)
			]

#if MUTE_SOLO_ENABLED
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SoundDashboard.MuteSoloButton"))
				.Visibility_Lambda([this]()
				{
					return IsMuteSoloAvailableForNode() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked(ECheckBoxState::Unchecked)
				.OnCheckStateChanged_Lambda([this](ECheckBoxState)
				{
					ToggleMute();
				})
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Mute").GetIcon())
					.ToolTipText(LOCTEXT("SignalFlowGraph_MuteNode", "Mute this node.\nMute will apply to all instances of an asset."))
					.ColorAndOpacity_Lambda([this]()
					{
						if (GetMuteSoloState().bIsMuted)
						{
							return FStyleColors::AccentBlue;
						}

						// A submix can be explicitly soloed and muted at the same time
						// It won't be muted in this scenario, but we still want to flag
						// that the explicit mute state is stored
						if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
						{
							const ::Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();
							if (Debugger.IsSubmixExplicitlyMuted(AssetFName))
							{
								return FSlateColor(FStyleColors::AccentBlue.GetSpecifiedColor().CopyWithNewOpacity(MuteSoloLowerAlpha));
							}
						}

						return FStyleColors::White25;
					})
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SoundDashboard.MuteSoloButton"))
				.Visibility_Lambda([this]()
				{
					return IsMuteSoloAvailableForNode() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked(ECheckBoxState::Unchecked)
				.OnCheckStateChanged_Lambda([this](ECheckBoxState)
				{
					ToggleSolo();
				})
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Solo").GetIcon())
					.ToolTipText(NodeEntryKey.EntryType == ESignalFlowEntryType::Submix ? LOCTEXT("SignalFlowGraph_SoloSubmixNode", "Solo this node.\nSolo will apply to all submixes connected to this node.")
																						: LOCTEXT("SignalFlowGraph_SoloNode", "Solo this node.\nSolo will apply to all instances of an asset."))
					.ColorAndOpacity_Lambda([this]()
					{
						const FMuteSoloStates MuteSoloState = GetMuteSoloState();
						if (!MuteSoloState.bIsSoloed)
						{
							return FSlateColor(FStyleColors::White25);
						}

						if (GetNodeEntryKey().EntryType == ESignalFlowEntryType::Submix)
						{
							return IsDirectlySoloed.Get(false) ? FSlateColor(FStyleColors::AccentYellow) : FSlateColor(FStyleColors::AccentYellow.GetSpecifiedColor().CopyWithNewOpacity(MuteSoloLowerAlpha));
						}
						else
						{
							return FSlateColor(FStyleColors::AccentYellow);
						}
					})
				]
			]
#endif // MUTE_SOLO_ENABLED
		
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MinSize(12.0f)
			.MaxSize(12.0f)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("AudioInsights.LeftExpandArrowToggle"))
				.Visibility_Lambda([this]()
				{ 
					return NodeDetailsExpanderArrowIsEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked_Lambda([this]()
				{
					return bNodeDetailsAreExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](const ECheckBoxState CheckboxState)
				{
					bNodeDetailsAreExpanded = CheckboxState == ECheckBoxState::Checked;
				})
			];
	}

	TSharedRef<SVerticalBox> SSignalFlowGraphNode::CreateNodeDetails() const
	{
		using namespace SSignalFlowGraphStyle;

		return SNew(SVerticalBox)
				.Visibility_Lambda([this]() { return ShowNodeDetails.Get() ? EVisibility::Visible : EVisibility::Collapsed; })

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Amplitude)
				]
			
				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Volume)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Pitch)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::LPFFreq)
				]
			
				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::HPFFreq)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Priority)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Distance)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::Attenuation)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::RelativeRenderCost)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, NodeDetailVerticalPadding, 0.0f, 0.0f)
				.AutoHeight()
				[
					CreateNodeDetailsRow(ESignalFlowNodeDetailParam::AudioComponentName)
				];
	}

	TSharedRef<SHorizontalBox> SSignalFlowGraphNode::CreateNodeDetailsRow(const ESignalFlowNodeDetailParam ParamType) const
	{
		using namespace SSignalFlowGraphStyle;

		return SNew(SHorizontalBox)
				.Visibility_Lambda([this, ParamType]()
				{
					const bool bParamIsFiltered = NodeDetailFilterSettings && NodeDetailFilterSettings->GetParameterIsVisible(ParamType);
					const bool bParamIsHiddenByExpander = !bNodeDetailsAreExpanded && ParamType != ESignalFlowNodeDetailParam::Amplitude;

					if (!bParamIsFiltered || bParamIsHiddenByExpander || !Entry.IsValid())
					{
						return EVisibility::Collapsed;
					}

					const bool bHasValue = GetNodeDetailParamValue(Entry, ParamType).IsSet()
										|| GetNodeDetailParamStringValue(Entry, ParamType).IsSet();

					return bHasValue ? EVisibility::Visible : EVisibility::Collapsed;
				})

				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SignalFlow.Pill").GetIcon())
					.ColorAndOpacity(GetNodeDetailParamColor(ParamType))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text_Lambda([this, ParamType]()
					{
						const TOptional<FString> StringValueOpt = GetNodeDetailParamStringValue(Entry, ParamType);
						if (StringValueOpt.IsSet())
						{
							return FText::Format(GetNodeDetailDisplayName(ParamType), FText::FromString(StringValueOpt.GetValue()));
						}

						const TOptional<float> ParamValueOpt = GetNodeDetailParamValue(Entry, ParamType);
						if (!ParamValueOpt.IsSet())
						{
							return FText::GetEmpty();
						}

						if (ParamType == ESignalFlowNodeDetailParam::Amplitude && DisplayAmpPeakInDb.Get() == true)
						{
							return FText::Format(GetNodeDetailDisplayName(ParamType), AudioInsightsUtils::ConvertToDecibelsText(ParamValueOpt.GetValue()));
						}

						if (ParamType == ESignalFlowNodeDetailParam::Priority && SoundMessageUtils::IsPriorityValueSetToMax(ParamValueOpt.GetValue()))
						{
							return FText::Format(GetNodeDetailDisplayName(ParamType), LOCTEXT("SignalFlowGraph_NodeDetails_Max", "MAX"));
						}

						return FText::Format(GetNodeDetailDisplayName(ParamType), FText::AsNumber(ParamValueOpt.GetValue(), GetNodeDetailNumberFormat(ParamType)));
					})
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Justification(ETextJustify::Left)
				];
	}

	TSharedRef<SVerticalBox> SSignalFlowGraphNode::CreateSendOutputsVerticalContainer()
	{
		SendLevelsVerticalBox = SNew(SVerticalBox)
								.Visibility_Lambda([this]()
								{
									return GraphOrientation.Get() == EOrientation::Orient_Horizontal && SendLabelsAreVisible() 
																	? EVisibility::Visible 
																	: EVisibility::Collapsed;
								});

		return SendLevelsVerticalBox.ToSharedRef();
	}

	TSharedRef<SHorizontalBox> SSignalFlowGraphNode::CreateSendOutputsHorizontalContainer()
	{
		SendLevelsHorizontalBox = SNew(SHorizontalBox)
									.Visibility_Lambda([this]()
									{
										return GraphOrientation.Get() == EOrientation::Orient_Vertical && SendLabelsAreVisible()
																	? EVisibility::Visible
																	: EVisibility::Collapsed;
									});

		return SendLevelsHorizontalBox.ToSharedRef();
	}

	bool SSignalFlowGraphNode::NodeDetailsExpanderArrowIsEnabled() const
	{
		using namespace SSignalFlowGraphStyle;

		// The expander arrow should only be visible if a parameter other than amplitude is visible on the node
		// This requires show node details to be switched on, the parameter filter to be enabled, and data to be available for that parameter
		if (!ShowNodeDetails.Get() || !Entry.IsValid() || NodeDetailFilterSettings == nullptr)
		{
			return false;
		}

		const int32 NumVisibleParams = NodeDetailFilterSettings->GetNumVisibleParams();
		if (NumVisibleParams <= 0)
		{
			return false;
		}

		if (NodeDetailFilterSettings->GetNumVisibleParams() == 1 && NodeDetailFilterSettings->GetParameterIsVisible(ESignalFlowNodeDetailParam::Amplitude))
		{
			return false;
		}
		
		const int32 NumFilters = static_cast<uint8>(ESignalFlowNodeDetailParam::MAX);
		for (uint8 FilterID = 0u; FilterID < NumFilters; ++FilterID)
		{
			const ESignalFlowNodeDetailParam Param = static_cast<ESignalFlowNodeDetailParam>(FilterID);
			if (Param == ESignalFlowNodeDetailParam::Amplitude)
			{
				continue;
			}

			if (Param == ESignalFlowNodeDetailParam::SendOutputVolume)
			{
				if (NodeDetailFilterSettings->GetParameterIsVisible(Param))
				{
					for (const auto& [OutputKey, SendInfo] : Entry->Outputs)
					{
						if (SendInfo.SendLevel.IsSet())
						{
							return true;
						}
					}
				}
			}
			else if (NodeDetailFilterSettings->GetParameterIsVisible(Param)
				&& (GetNodeDetailParamValue(Entry, Param).IsSet() || GetNodeDetailParamStringValue(Entry, Param).IsSet()))
			{
				return true;
			}
		}

		return false;
	}

	bool SSignalFlowGraphNode::SendLabelsAreVisible() const
	{
		if (!ShowNodeDetails.Get() || SendLabels.IsEmpty())
		{
			return false;
		}

		const bool bParamIsFiltered = NodeDetailFilterSettings && NodeDetailFilterSettings->GetParameterIsVisible(ESignalFlowNodeDetailParam::SendOutputVolume);
		if (!bNodeDetailsAreExpanded || !bParamIsFiltered)
		{
			return false;
		}

		for (const auto& [ReceiverKey, Label] : SendLabels)
		{
			if (Label->ValueIsSet())
			{
				return true;
			}
		}

		return false;
	}

#if MUTE_SOLO_ENABLED
	void SSignalFlowGraphNode::ToggleMute()
	{
		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (AudioDeviceManager == nullptr)
		{
			return;
		}

		::Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();

		if (NodeEntryKey.EntryType == ESignalFlowEntryType::SoundSource)
		{
			Debugger.ToggleMuteSoundWave(AssetFName);
		}
		else if (NodeEntryKey.EntryType == ESignalFlowEntryType::Submix)
		{
			Debugger.ToggleMuteSubmix(AssetFName);
		}
	}

	void SSignalFlowGraphNode::ToggleSolo()
	{
		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (AudioDeviceManager == nullptr)
		{
			return;
		}

		::Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();

		if (NodeEntryKey.EntryType == ESignalFlowEntryType::SoundSource)
		{
			Debugger.ToggleSoloSoundWave(AssetFName);
		}
		else if (NodeEntryKey.EntryType == ESignalFlowEntryType::Submix)
		{
			Debugger.ToggleSoloSubmixChain(AssetFName);
		}
	}

	SSignalFlowGraphNode::FMuteSoloStates SSignalFlowGraphNode::GetMuteSoloState() const
	{
		FMuteSoloStates MuteSoloStates;
		if (Entry.IsValid())
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				FString Reason;
				const ::Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();

				if (NodeEntryKey.EntryType == ESignalFlowEntryType::SoundSource)
				{
					Debugger.QuerySoloMuteSoundWave(AssetString, MuteSoloStates.bIsSoloed, MuteSoloStates.bIsMuted, Reason);
				}
				else if (NodeEntryKey.EntryType == ESignalFlowEntryType::Submix)
				{
					Debugger.QuerySoloMuteSubmix(AssetString, MuteSoloStates.bIsSoloed, MuteSoloStates.bIsMuted, Reason);
				}
			}
		}

		return MuteSoloStates;
	}

	bool SSignalFlowGraphNode::IsMuteSoloAvailableForNode() const
	{
		return NodeEntryKey.EntryType == ESignalFlowEntryType::SoundSource
			|| NodeEntryKey.EntryType == ESignalFlowEntryType::Submix;
	}
#endif // MUTE_SOLO_ENABLED
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
