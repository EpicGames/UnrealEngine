// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SSignalFlowGraphSendLabel.h"

#include "AudioInsightsStyle.h"
#include "Internationalization/Text.h"
#include "Styling/StyleColors.h"
#include "Views/SSignalFlowGraphStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Audio::Insights
{
	void SSignalFlowGraphSendLabel::Construct(const SSignalFlowGraphSendLabel::FArguments& InArgs)
	{
		using namespace SSignalFlowGraphStyle;

		SendLevel = InArgs._SendLevel;
		GraphOrientation = InArgs._GraphOrientation;
		ZoomScaleFactor = InArgs._ZoomScaleFactor;

		ChildSlot
		[
			SNew(SOverlay)

			// Horizontal flow label (text then icon)
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]()
				{
					return GraphOrientation.Get() == EOrientation::Orient_Horizontal && SendLevel.IsBound() && SendLevel.Get().IsSet() 
																												? EVisibility::Visible 
																												: EVisibility::Collapsed;
				})

				+ SHorizontalBox::Slot()
				.FillContentWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					CreateOutputLabelText(InArgs._ToolTipText.Get())
				]

				+ SHorizontalBox::Slot()
				.FillContentWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					CreatePinIconWidget(InArgs._ToolTipText.Get(), InArgs._ReceiverEntryType)
				]
			]

			// Vertical flow label (text above icon)
			+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
				.Visibility_Lambda([this]()
				{
					return GraphOrientation.Get() == EOrientation::Orient_Vertical && SendLevel.IsBound() && SendLevel.Get().IsSet() 
																												? EVisibility::Visible 
																												: EVisibility::Collapsed;
				})

				+ SVerticalBox::Slot()
				.FillContentHeight(1.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Bottom)
				[
					CreateOutputLabelText(InArgs._ToolTipText.Get())
				]

				+ SVerticalBox::Slot()
				.FillContentHeight(1.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Bottom)
				[
					CreatePinIconWidget(InArgs._ToolTipText.Get(), InArgs._ReceiverEntryType)
				]
			]
		];
	}

	FVector2D SSignalFlowGraphSendLabel::ComputeDesiredSize(float InLayoutScaleMultiplier) const
	{
		using namespace SSignalFlowGraphStyle;

		const FVector2D Size = SCompoundWidget::ComputeDesiredSize(InLayoutScaleMultiplier);
		if (GraphOrientation.Get() == EOrientation::Orient_Horizontal)
		{
			return Size;
		}
		else
		{
			return FVector2D(SendLabelVerticalWidth, Size.Y);
		}
	}

	bool SSignalFlowGraphSendLabel::ValueIsSet() const
	{
		return SendLevel.IsBound() && SendLevel.Get().IsSet();
	}

	TSharedRef<STextBlock> SSignalFlowGraphSendLabel::CreateOutputLabelText(const FText& ToolTipText) const
	{
		return SNew(STextBlock)
				.Text_Lambda([this]()
				{
					if (!SendLevel.IsBound())
					{
						return FText::GetEmpty();
					}

					const TOptional<float> SendLevelOpt = SendLevel.Get();
					if (!SendLevelOpt.IsSet())
					{
						return FText::GetEmpty();
					}

					return FText::AsNumber(SendLevelOpt.GetValue(), FSlateStyle::Get().GetSendLevelFormat());
				})
				.ToolTipText(ToolTipText)
				.Justification(ETextJustify::Center)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"));
	}

	TSharedRef<SOverlay> SSignalFlowGraphSendLabel::CreatePinIconWidget(const FText& ToolTipText, const ESignalFlowEntryType ReceiverEntryType)
	{
		using namespace SSignalFlowGraphStyle;

		return SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SAssignNew(OutputPinImage, SImage)
				.Image(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SignalFlow.Disconnected").GetIcon())
				.ColorAndOpacity(GetNodeAccentColor(ReceiverEntryType))
			]

			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SignalFlow.Connected").GetIcon())
				.ToolTipText(ToolTipText)
				.ColorAndOpacity_Lambda([this, ReceiverEntryType]()
				{
					const FSlateColor BaseColor = GetNodeAccentColor(ReceiverEntryType);

					if (!SendLevel.IsBound())
					{
						return BaseColor;
					}

					const TOptional<float> SendLevelOpt = SendLevel.Get();
					if (!SendLevelOpt.IsSet())
					{
						return BaseColor;
					}

					return FSlateColor(BaseColor.GetSpecifiedColor().CopyWithNewOpacity(SendLevelOpt.GetValue()));
				})
			];
	}
} // namespace UE::Audio::Insights
