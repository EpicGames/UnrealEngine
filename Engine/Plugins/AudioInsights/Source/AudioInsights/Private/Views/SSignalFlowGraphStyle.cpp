// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SSignalFlowGraphStyle.h"

#include "AudioInsightsStyle.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Internationalization/Text.h"
#include "Messages/SignalFlowTraceMessages.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace SSignalFlowGraphStyle
	{
		const FSlateRoundedBoxBrush& GetNodeRoundedBoxBrush()
		{
			static const FSlateRoundedBoxBrush RoundedBoxBrush(FLinearColor::White, NodeRoundedBrushRadius);
			return RoundedBoxBrush;
		}

		float GetNodeMaxHeight(const int32 NumParamsVisible, const bool bShowNodeDetails, const float ZoomScaleFactor)
		{
			constexpr float NodeMaxHeight = 50.0f;
			constexpr float NodeDetailsRowHeight = 24.0f;

			float BorderInflationDelta = 0.0f;
			if (ZoomScaleFactor < 1.0f)
			{
				constexpr float BorderInflationAtUnitZoom = 2.0f * (SelectedNodeBorderThickness + NodeBorderThickness);
				BorderInflationDelta = (BorderInflationAtUnitZoom / ZoomScaleFactor) - BorderInflationAtUnitZoom;
			}

			const float BaseHeight = bShowNodeDetails ? NodeMaxHeight + (NodeDetailsRowHeight * NumParamsVisible) : NodeMaxHeight;
			return BaseHeight + BorderInflationDelta;
		}

		float NormalizedToNodePadding(const float NormalizedValue)
		{
			return FMath::Lerp(MinNodePadding, MaxNodePadding, NormalizedValue);
		}

		float NodePaddingToNormalized(const float PixelPadding)
		{
			return FMath::GetRangePct(MinNodePadding, MaxNodePadding, PixelPadding);
		}

		FSlateColor GetNodeAccentColor(const ESignalFlowEntryType NodeType)
		{
			switch (NodeType)
			{
				case ESignalFlowEntryType::SoundSource:
					return FStyleColors::AccentBlue;
				case ESignalFlowEntryType::AudioBus:
					return FStyleColors::AccentYellow;
				case ESignalFlowEntryType::Submix:
					return FStyleColors::AccentGreen;
				case ESignalFlowEntryType::OwnerObject:
				case ESignalFlowEntryType::AudioDevice:
				default:
					return FStyleColors::Foreground;
			}
		}

		FLinearColor GetConnectionColor(const ESignalFlowEntryType ReceiverEntryType, const float AmplitudeValue, const float WireSplineAmplitudePowerFactor, const bool bIsSourceBusConnection, const bool bIsSendLabelHovered)
		{
			if (bIsSendLabelHovered)
			{
				return GetNodeAccentColor(ReceiverEntryType).GetSpecifiedColor();
			}

			const float ScaledAmplitude = FMath::Pow(AmplitudeValue, WireSplineAmplitudePowerFactor);

			if (bIsSourceBusConnection)
			{
				constexpr float MaxAlpha = 0.5f;
				return FLinearColor::LerpUsingHSV(FStyleColors::White25.GetSpecifiedColor(), GetNodeAccentColor(ReceiverEntryType).GetSpecifiedColor().CopyWithNewOpacity(MaxAlpha), ScaledAmplitude);
			}

			return FLinearColor::LerpUsingHSV(FStyleColors::Hover.GetSpecifiedColor(), FStyleColors::AccentWhite.GetSpecifiedColor(), ScaledAmplitude);
		}

		FVector2d GetSplineStartDirection(const EOrientation Orientation, const bool bIsSourceBusConnection, const float ZoomScaleFactor)
		{
			const float SplineRadius = (bIsSourceBusConnection ? SplineRadiusFlowingBackwards : SplineRadiusFlowingForwards) * ZoomScaleFactor;
			return Orientation == EOrientation::Orient_Horizontal ? FVector2d{ SplineRadius, 0.0f } : FVector2d{ 0.0f, SplineRadius };
		}

		FVector2d GetSplineEndDirection(const EOrientation Orientation, const bool bIsSourceBusConnection, const float ZoomScaleFactor)
		{
			const float SplineRadius = (bIsSourceBusConnection ? SplineRadiusFlowingBackwards : SplineRadiusFlowingForwards) * ZoomScaleFactor;
			return Orientation == EOrientation::Orient_Horizontal ? FVector2d{ SplineRadius, 0.0f } : FVector2d{ 0.0f, SplineRadius };
		}

		TOptional<float> GetNodeDetailParamValue(const TSharedPtr<FSignalFlowDashboardEntry>& Entry, const ESignalFlowNodeDetailParam ParamType)
		{
			if (!Entry.IsValid())
			{
				return NullOpt;
			}

			switch (ParamType)
			{
				case ESignalFlowNodeDetailParam::Amplitude:
					return Entry->Amplitude;

				case ESignalFlowNodeDetailParam::Volume:
					return Entry->Volume;

				case ESignalFlowNodeDetailParam::Pitch:
					return Entry->Pitch;

				case ESignalFlowNodeDetailParam::LPFFreq:
					return Entry->LPFFreq;

				case ESignalFlowNodeDetailParam::HPFFreq:
					return Entry->HPFFreq;

				case ESignalFlowNodeDetailParam::Priority:
					return Entry->Priority;

				case ESignalFlowNodeDetailParam::Distance:
					return Entry->Distance;

				case ESignalFlowNodeDetailParam::Attenuation:
					return Entry->Attenuation;

				case ESignalFlowNodeDetailParam::RelativeRenderCost:
					return Entry->RelativeRenderCost;

				case ESignalFlowNodeDetailParam::AudioComponentName:
					return NullOpt;

				default:
					ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
					break;
			}

			return NullOpt;
		}

		TOptional<FString> GetNodeDetailParamStringValue(const TSharedPtr<FSignalFlowDashboardEntry>& Entry, const ESignalFlowNodeDetailParam ParamType)
		{
			if (!Entry.IsValid())
			{
				return NullOpt;
			}

			switch (ParamType)
			{
				case ESignalFlowNodeDetailParam::AudioComponentName:
					return Entry->AudioComponentName;

				default:
					break;
			}

			return NullOpt;
		}

		FSlateColor GetNodeDetailParamColor(const ESignalFlowNodeDetailParam ParamType)
		{
			switch (ParamType)
			{
				case ESignalFlowNodeDetailParam::Amplitude:
					return FStyleColors::AccentGreen;

				case ESignalFlowNodeDetailParam::Volume:
					return FStyleColors::AccentYellow;

				case ESignalFlowNodeDetailParam::Pitch:
					return FStyleColors::AccentOrange;

				case ESignalFlowNodeDetailParam::LPFFreq:
					return FStyleColors::AccentBrown;

				case ESignalFlowNodeDetailParam::HPFFreq:
					return FStyleColors::AccentRed;

				case ESignalFlowNodeDetailParam::Priority:
					return FStyleColors::AccentPink;

				case ESignalFlowNodeDetailParam::Distance:
					return FStyleColors::AccentPurple;

				case ESignalFlowNodeDetailParam::Attenuation:
					return FStyleColors::AccentBlue;

				case ESignalFlowNodeDetailParam::RelativeRenderCost:
					return FStyleColors::AccentGray;

				case ESignalFlowNodeDetailParam::AudioComponentName:
					return FStyleColors::AccentWhite;

				default:
					ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
					break;
			}

			return FStyleColors::White;
		}

		FText GetNodeDetailDisplayName(const ESignalFlowNodeDetailParam ParamType)
		{
			switch (ParamType)
			{
				case ESignalFlowNodeDetailParam::Amplitude:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Amplitude", "Amp (peak): {0}");

				case ESignalFlowNodeDetailParam::Volume:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Volume", "Volume: {0}");

				case ESignalFlowNodeDetailParam::Pitch:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Pitch", "Pitch: {0}");

				case ESignalFlowNodeDetailParam::LPFFreq:
					return LOCTEXT("SignalFlowGraph_NodeDetails_LPFFreq", "LPF Freq (Hz): {0}");

				case ESignalFlowNodeDetailParam::HPFFreq:
					return LOCTEXT("SignalFlowGraph_NodeDetails_HPFFreq", "HPF Freq (Hz): {0}");

				case ESignalFlowNodeDetailParam::Priority:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Priority", "Priority: {0}");

				case ESignalFlowNodeDetailParam::Distance:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Distance", "Distance: {0}");

				case ESignalFlowNodeDetailParam::Attenuation:
					return LOCTEXT("SignalFlowGraph_NodeDetails_Attenuation", "Distance/Occlusion: {0}");

				case ESignalFlowNodeDetailParam::RelativeRenderCost:
					return LOCTEXT("SignalFlowGraph_NodeDetails_RelativeRenderCost", "Rel. Render Cost: {0}");

				case ESignalFlowNodeDetailParam::AudioComponentName:
					return LOCTEXT("SignalFlowGraph_NodeDetails_AudioComponentName", "Component: {0}");

				default:
					ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
					break;
			}

			return FText::GetEmpty();
		}

		const FNumberFormattingOptions* GetNodeDetailNumberFormat(const ESignalFlowNodeDetailParam ParamType)
		{
			switch (ParamType)
			{
				case ESignalFlowNodeDetailParam::Amplitude:
					return FSlateStyle::Get().GetLinearAmplitudeFloatFormat();

				case ESignalFlowNodeDetailParam::Volume:
					return FSlateStyle::Get().GetLinearVolumeFloatFormat();

				case ESignalFlowNodeDetailParam::Pitch:
					return FSlateStyle::Get().GetPitchFloatFormat();

				case ESignalFlowNodeDetailParam::LPFFreq:
					return FSlateStyle::Get().GetFreqFloatFormat();

				case ESignalFlowNodeDetailParam::HPFFreq:
					return FSlateStyle::Get().GetFreqFloatFormat();

				case ESignalFlowNodeDetailParam::Priority:
					return FSlateStyle::Get().GetLinearVolumeFloatFormat();

				case ESignalFlowNodeDetailParam::Distance:
					return FSlateStyle::Get().GetDistanceFloatFormat();

				case ESignalFlowNodeDetailParam::Attenuation:
					return FSlateStyle::Get().GetLinearVolumeFloatFormat();

				case ESignalFlowNodeDetailParam::RelativeRenderCost:
					return FSlateStyle::Get().GetRelativeRenderCostFormat();

				case ESignalFlowNodeDetailParam::AudioComponentName:
					return nullptr;

				default:
					ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
					break;
			}

			return FSlateStyle::Get().GetDefaultFloatFormat();
		}
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
