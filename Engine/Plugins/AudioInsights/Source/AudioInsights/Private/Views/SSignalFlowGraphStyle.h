// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/MathFwd.h"
#include "Messages/SignalFlowEntryKey.h"
#include "Misc/Optional.h"
#include "Types/SlateEnums.h"

struct FLinearColor;
struct FNumberFormattingOptions;
struct FSlateColor;
struct FSlateRoundedBoxBrush;

class FText;

namespace UE::Audio::Insights
{
	struct FSignalFlowDashboardEntry;

	namespace SSignalFlowGraphStyle
	{
		const FSlateRoundedBoxBrush& GetNodeRoundedBoxBrush();
		float GetNodeMaxHeight(const int32 NumParamsVisible, const bool bShowNodeDetails, const float ZoomScaleFactor);
		float NormalizedToNodePadding(const float NormalizedValue);
		float NodePaddingToNormalized(const float PixelPadding);
		FSlateColor GetNodeAccentColor(const ESignalFlowEntryType NodeType);
		FLinearColor GetConnectionColor(const ESignalFlowEntryType ReceiverEntryType, const float AmplitudeValue, const float WireSplineAmplitudePowerFactor, const bool bIsSourceBusConnection, const bool bIsSendLabelHovered = false);
		FVector2d GetSplineStartDirection(const EOrientation Orientation, const bool bIsSourceBusConnection, const float ZoomScaleFactor);
		FVector2d GetSplineEndDirection(const EOrientation Orientation, const bool bIsSourceBusConnection, const float ZoomScaleFactor);
		TOptional<float> GetNodeDetailParamValue(const TSharedPtr<FSignalFlowDashboardEntry>& Entry, const ESignalFlowNodeDetailParam ParamType);
		TOptional<FString> GetNodeDetailParamStringValue(const TSharedPtr<FSignalFlowDashboardEntry>& Entry, const ESignalFlowNodeDetailParam ParamType);
		FSlateColor GetNodeDetailParamColor(const ESignalFlowNodeDetailParam ParamType);
		FText GetNodeDetailDisplayName(const ESignalFlowNodeDetailParam ParamType);
		const FNumberFormattingOptions* GetNodeDetailNumberFormat(const ESignalFlowNodeDetailParam ParamType);

		constexpr float GraphMinPadding = 400.0f;
		constexpr float NodeMaxWidth = 256.0f;

		constexpr float NodeRoundedBrushRadius = 12.0f;

		constexpr float SplineRadiusFlowingForwards = 80.0f;
		constexpr float SplineRadiusFlowingBackwards = 600.0f;
		constexpr float SplineThicknessScalarMin = 0.5f;
		constexpr float SplineThicknessDefault = 4.0f;
		constexpr float SplineAnimatedWireThickness = 4.0f;
		constexpr float SplineThicknessMin = 1.0f;
		constexpr float SplineAnimPointsPerUnit = 0.175f;
		constexpr float BackwardSplineAnimPointsPerUnit = 0.325f;

		constexpr float SelectedNodeBorderThickness = 3.0f;
		constexpr float NodeBorderThickness = 2.0f;

		constexpr float NodeFilteredOutByTextDimming = 0.3f;
		constexpr float NonHighlightedPathAlphaFactor = 0.15f;
		constexpr float HoveredWireThicknessMultiplier = 2.0f;
		constexpr float HoveredWireHeightMultiplier = 1.25f;
		constexpr float SendLabelHoverRadius = 12.0f;

		constexpr float NodeBorderPadding = 12.0f;
		constexpr float NodeDetailVerticalPadding = 8.0f;
		constexpr float NodeDetailIndentationPadding = 20.0f;
		constexpr float LargeBetweenNodesDefaultPadding = 128.0f;
		constexpr float SmallBetweenNodesDefaultPadding = 8.0f;
		constexpr float MinNodePadding = 0.25f;
		constexpr float MaxNodePadding = 500.0f;

		constexpr float SendLabelVerticalWidth = 20.0f;
		constexpr float SendLabelPadding = 8.0f;

		constexpr float AnimationSpeed = 15.0f;
		constexpr float DefaultZoomScale = 1.0f;
		constexpr float ZoomStepFactor = 0.1f;
		constexpr float MinZoomScale = 0.125f;
		constexpr float MaxZoomScale = 2.0f;
		constexpr float ScrollAnimInRangeThreshold = (1.e-2f);

		constexpr float MuteSoloLowerAlpha = 0.6f;
	}
} // namespace UE::Audio::Insights