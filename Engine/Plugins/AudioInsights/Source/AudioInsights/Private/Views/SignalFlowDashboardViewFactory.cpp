// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SignalFlowDashboardViewFactory.h"

#include "Algo/BinarySearch.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTimingViewExtender.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAudioInsightsModule.h"
#include "Internationalization/Text.h"
#include "Providers/SignalFlowTraceProvider.h"
#include "SignalFlowEditorCommands.h"
#include "Styling/StyleColors.h"
#include "Views/SignalFlowNodes.h"
#include "Views/SSignalFlowGraphStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"

#if MUTE_SOLO_ENABLED
#include "Audio/AudioDebug.h"
#endif // MUTE_SOLO_ENABLED

#if WITH_EDITOR
#include "AudioInsightsDetailsSelectionManager.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#else
#include "AudioInsightsComponent.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FSignalFlowDashboardViewFactoryPrivate
	{
		const FName ButtonStyleName = "SimpleButton";
		const FName ButtonTextStyleName = "SmallButtonText";

		constexpr float MinGraphWidthRatio = 0.25f;
		constexpr float MaxGraphWidthRatio = 0.85f;

		constexpr float CollapsedActiveEntryMenuHeight = 22.0f;

		constexpr float ToolbarButtonHeight = 16.0f;
		constexpr float ToolbarButtonPaddingX = 4.0f;
		constexpr float ToolbarButtonPaddingY = 4.0f;

		constexpr float NodePaddingLabelWidth = 130.0f;
		constexpr float NodePaddingSpinBoxWidth = 150.0f;

		// Maximum allowed gap between consecutive real-node X positions across the graph.
		// Gaps exceeding this are compressed in the post-layout compaction step.
		constexpr float MaxRealNodeGap = 1.0f;

		FText GetParamFilterName(const ESignalFlowNodeDetailParam Param)
		{
			switch (Param)
			{
				case ESignalFlowNodeDetailParam::Amplitude:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Amplitude", "Amplitude");

				case ESignalFlowNodeDetailParam::Volume:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Volume", "Volume");

				case ESignalFlowNodeDetailParam::Pitch:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Pitch", "Pitch");

				case ESignalFlowNodeDetailParam::LPFFreq:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_LPFFreq", "LPF Freq");

				case ESignalFlowNodeDetailParam::HPFFreq:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_HPFFreq", "HPF Freq");

				case ESignalFlowNodeDetailParam::Priority:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Priority", "Priority");

				case ESignalFlowNodeDetailParam::Distance:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Distance", "Distance");

				case ESignalFlowNodeDetailParam::Attenuation:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_Attenuation", "Distance/Occlusion Attenuation");

				case ESignalFlowNodeDetailParam::RelativeRenderCost:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_RelativeRenderCost", "Rel. Render Cost");

				case ESignalFlowNodeDetailParam::AudioComponentName:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_AudioComponentName", "Audio Component");

				case ESignalFlowNodeDetailParam::SendOutputVolume:
					return LOCTEXT("AudioDashboard_SignalFlow_Param_SendOutputVolume", "Send Output Vol.");

				default:
					ensureMsgf(false, TEXT("Unrecognized ESignalFlowNodeDetailParam value"));
					break;
			}

			return FText::GetEmpty();
		}

		const FSlateRoundedBoxBrush& GetRoundedBoxBrush()
		{
			static const FSlateRoundedBoxBrush RoundedBoxBrush(FLinearColor::White, 8.0f);
			return RoundedBoxBrush;
		}
	}

	FReply SSignalFlowDashboard::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (OnKeyDownHandler.IsBound())
		{
			return OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
		}

		return FReply::Unhandled();
	}

	FSignalFlowDashboardViewFactory::FSignalFlowDashboardViewFactory()
	{
		IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();

		SignalFlowProvider = MakeShared<FSignalFlowTraceProvider>();

		SignalFlowProvider->BindDelegates();

		AudioInsightsTraceModule.AddTraceProvider(SignalFlowProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			SignalFlowProvider
		};

		FSignalFlowEditorCommands::Register();
		BindCommands();

		SignalFlowProvider->OnRequestGraphRefresh.AddRaw(this, &FSignalFlowDashboardViewFactory::OnRequestGraphRefresh);
		SignalFlowProvider->OnResetGraph.AddRaw(this, &FSignalFlowDashboardViewFactory::Reset);

#if WITH_EDITOR
		FSignalFlowSettings::OnReadSettings.AddRaw(this, &FSignalFlowDashboardViewFactory::OnReadEditorSettings);
		FSignalFlowSettings::OnWriteSettings.AddRaw(this, &FSignalFlowDashboardViewFactory::OnWriteEditorSettings);
#endif // WITH_EDITOR
	}

	FSignalFlowDashboardViewFactory::~FSignalFlowDashboardViewFactory()
	{
		const TSharedPtr<FSignalFlowTraceProvider> Provider = FindProvider<FSignalFlowTraceProvider>();
		if (Provider.IsValid())
		{
			Provider->OnRequestGraphRefresh.RemoveAll(this);
			Provider->OnResetGraph.RemoveAll(this);
		}

#if WITH_EDITOR
		FSignalFlowSettings::OnReadSettings.RemoveAll(this);
		FSignalFlowSettings::OnWriteSettings.RemoveAll(this);
#endif // WITH_EDITOR

		FSignalFlowEditorCommands::Unregister();
	}

	FName FSignalFlowDashboardViewFactory::GetName() const
	{
		return "SignalFlow";
	}

	FText FSignalFlowDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_SignalFlow_DisplayName", "Signal Flow");
	}
	
	FSlateIcon FSignalFlowDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Submix"); // TODO: we should request a unique icon for this dashboard
	}

	EDefaultDashboardTabStack FSignalFlowDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Analysis;
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		bSentSelectedNodeDestroyedNotification = false;

		if (!DashboardWidget.IsValid())
		{
			SAssignNew(DashboardWidget, SSignalFlowDashboard)

			+ SSignalFlowDashboard::Slot()
			.AutoHeight()
			[
				CreateToolbar()
			]

			+ SSignalFlowDashboard::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(MainSplitter, SSplitter)
				.Orientation(Orient_Horizontal)
				.ResizeMode(ESplitterResizeMode::Fill)
				.PhysicalSplitterHandleSize(4.0f)
				.HitDetectionSplitterHandleSize(6.0f)
				.OnSplitterFinishedResizing(this, &FSignalFlowDashboardViewFactory::OnMainSplitterFinishedResizing)

				+ SSplitter::Slot()
				.Value(1.0f - GraphWidthRatio)
				[
					CreateSelectionPanel()
				]

				+ SSplitter::Slot()
				.Value(GraphWidthRatio)
				[
					CreateGraphSection()
				]
			];

			DashboardWidget->SetOnKeyDownHandler(FOnKeyDown::CreateSP(this, &FSignalFlowDashboardViewFactory::OnKeyDown));

			// Update initial state of graph
			OnRequestGraphRefresh();
			RefreshFilteredEntriesListView();
		}

#if WITH_EDITOR
		// Read the editor settings after the widget has been created
		FSignalFlowSettings::OnRequestReadSettings.Broadcast();
#endif // WITH_EDITOR

		return DashboardWidget->AsShared();
	}

	FReply FSignalFlowDashboardViewFactory::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (GraphWidgetContents.IsValid())
		{
			if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
#if WITH_EDITOR
			else if (AssetContextMenuHelper.ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
#endif // WITH_EDITOR
		}

		return FReply::Unhandled();
	}

	void FSignalFlowDashboardViewFactory::Reset()
	{
		if (GraphWidgetContents.IsValid())
		{
			GraphWidgetContents->ResetGraph();
		}

		SelectedItem.Reset();
		XPosFocusNodeEntryKey.Reset();

		SetNewSelection(nullptr);

		bAutoFocusAfterGraphRefresh = true;
	}

	void FSignalFlowDashboardViewFactory::SRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FSignalFlowDashboardEntry> Entry)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		RowEntry = Entry;

		STableRow<TSharedPtr<FSignalFlowDashboardEntry>>::Construct
		(
			STableRow<TSharedPtr<FSignalFlowDashboardEntry>>::FArguments()
			.Style(InArgs._Style)
			.Padding(4.0f)
			.Content()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FSlateStyle::Get().CreateIcon(RowEntry->IconName).GetIcon())
					.ColorAndOpacity(SSignalFlowGraphStyle::GetNodeAccentColor(RowEntry->EntryType))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(RowEntry->GetDisplayName())
					.ToolTipText(RowEntry->GetDisplayName())
				]
			]
			, InOwnerTable
		);
	}

	const FTableRowStyle* FSignalFlowDashboardViewFactory::GetRowStyle() const
	{
		return &FSlateStyle::Get().GetWidgetStyle<FTableRowStyle>("TreeDashboard.TableViewRow");
	}

	void FSignalFlowDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FSignalFlowEditorCommands& Commands = FSignalFlowEditorCommands::Get();

		CommandList->MapAction(
			Commands.GetToggleHorizontalViewCommand(),
			FExecuteAction::CreateLambda([this]() { ToggleGraphOrientation(); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return GraphWidgetContents.IsValid() && GraphWidgetContents->GetGraphOrientation() == Orient_Horizontal; }));

		CommandList->MapAction(
			Commands.GetJustifyEdgeCommand(),
			FExecuteAction::CreateLambda([this]() { SetJustification(ESignalFlowJustification::Edge); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return GraphJustification == ESignalFlowJustification::Edge; }));

		CommandList->MapAction(
			Commands.GetJustifyCenterCommand(),
			FExecuteAction::CreateLambda([this]() { SetJustification(ESignalFlowJustification::Center); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return GraphJustification == ESignalFlowJustification::Center; }));

		CommandList->MapAction(
			Commands.GetShowNodeDetailsCommand(),
			FExecuteAction::CreateLambda([this]() { ToggleShowNodeDetails(); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return bShowNodeDetails; }));

		CommandList->MapAction(
			Commands.GetPauseOnSelectCommand(),
			FExecuteAction::CreateLambda([this]() { TogglePauseGraphEnabled(); }),
#if WITH_EDITOR
			FCanExecuteAction(),
#else
			FCanExecuteAction::CreateLambda([this]()
			{
				return !IsReadingTraceFile();
			}),
#endif // WITH_EDITOR
			
			FIsActionChecked::CreateLambda([this]() { return bPauseGraphOnSelect; }));

		CommandList->MapAction(
			Commands.GetToggleAnimateWiresCommand(),
			FExecuteAction::CreateLambda([this]() { ToggleAnimateWires(); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return bAnimateWires; }));

		CommandList->MapAction(
			Commands.GetCenterViewCommand(),
			FExecuteAction::CreateLambda([this]() { ScrollGraphViewToSelectedNode(true /*bResetZoom*/); }));

		CommandList->MapAction(
			Commands.GetResetTimestampPauseCommand(),
			FExecuteAction::CreateLambda([this]() { ResetPauseTimestamp(); }),
#if WITH_EDITOR
			FCanExecuteAction()
#else
			FCanExecuteAction::CreateLambda([this]()
			{
				return !IsReadingTraceFile();
			})
#endif // WITH_EDITOR
		);
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbar()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			CreateToolbarToggleControls()
		]
			
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, 0.0f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			CreateToolbarButtonActions()
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, 0.0f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

#if MUTE_SOLO_ENABLED
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			CreateToolbarClearMuteSoloButton()
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, 0.0f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]
#endif // MUTE_SOLO_ENABLED

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			CreateToolbarIndicators()
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, 0.0f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			CreateToolbarSettingsMenu()
		];
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbarSettingsMenu()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;
		
		return SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &FSignalFlowDashboardViewFactory::OnGetSettingsMenuContent)
			.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
			.HasDownArrow(false)
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_SettingsTooltip", "Open the settings menu."))
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Settings"))
				]
			];
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbarToggleControls()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		const FSlateStyle& Style = FSlateStyle::Get();

		TFunction<FSlateColor()> HorizontalFlowColorFunc = [this]()
		{ 
			return GraphWidgetContents.IsValid() && GraphWidgetContents->GetGraphOrientation() == Orient_Horizontal ? FSlateColor(FColor::Green) : FSlateColor::UseForeground();
		};

		TFunction<FSlateColor()> NodeDetailsColorFunc = [this]()
		{ 
			return bShowNodeDetails ? FSlateColor(FColor::Green) : FSlateColor::UseForeground();
		};

		TFunction<FSlateColor()> PauseOnSelectColorFunc = [this]()
		{ 
			return bPauseGraphOnSelect ? FSlateColor(FColor::Green) : FSlateColor::UseForeground();
		};

		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleName))
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_HorizontalFlowTooltip", "Toggles between horizontally/vertically orienting the Signal Flow graph. (Ctrl + h)"))
			.OnClicked_Lambda([this]()
			{
				ToggleGraphOrientation();

				return FReply::Handled();
			})
			[
				Style.CreateToggleButtonContent("AudioInsights.Icon.SignalFlow.HorizontalFlow",
												ButtonTextStyleName,
												HorizontalFlowColorFunc,
												[]() { return LOCTEXT("AudioDashboard_SignalFlow_HorizontalFlow", "Horizontal Flow"); },
												ToolbarButtonHeight)
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleName))
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_NodeDetailsTooltip", "Toggles visibility of node details. (Ctrl + d)"))
			.OnClicked_Lambda([this]()
			{
				ToggleShowNodeDetails();

				return FReply::Handled();
			})
			[
				Style.CreateToggleButtonContent("AudioInsights.Icon.SignalFlow.NodeDetails",
												ButtonTextStyleName,
												NodeDetailsColorFunc,
												[]() { return LOCTEXT("AudioDashboard_SignalFlow_NodeDetails", "Node Details"); },
												ToolbarButtonHeight)
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleName))
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_PauseTimestampTooltip", "Toggles pausing the graph at the current timestamp when selecting a node. (Ctrl + p)"))
#if !WITH_EDITOR
			.Visibility_Lambda([this]()
			{
				return IsReadingTraceFile() ? EVisibility::Collapsed : EVisibility::Visible;
			})
#endif // !WITH_EDITOR
			.OnClicked_Lambda([this]()
			{
				TogglePauseGraphEnabled();

				return FReply::Handled();
			})
			[
				Style.CreateToggleButtonContent("AudioInsights.Icon.Pause",
												ButtonTextStyleName,
												PauseOnSelectColorFunc,
												[]() { return LOCTEXT("AudioDashboard_SignalFlow_PauseTimestamp", "Pause on Select"); },
												ToolbarButtonHeight)
			]
		];
	}
	
	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbarButtonActions()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		const FSlateStyle& Style = FSlateStyle::Get();

		// Button actions
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleName))
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_CenterViewTooltip", "Focus the graph view on the selected node. (Ctrl + f)"))
			.OnClicked_Lambda([this]()
			{
				return ScrollGraphViewToSelectedNode(true /*bResetZoom*/);
			})
			[
				Style.CreateButtonContentWidget("AudioInsights.Icon.SignalFlow.CenterView", FText::GetEmpty(), ButtonTextStyleName)
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSegmentedControl<ESignalFlowJustification>)
				.OnValueChanged(this, &FSignalFlowDashboardViewFactory::SetJustification)
				.Value_Lambda([this]
				{
					return GraphJustification;
				})

				+ SSegmentedControl<ESignalFlowJustification>::Slot(ESignalFlowJustification::Edge)
				.ToolTip(LOCTEXT("AudioDashboard_SignalFlow_Justification_Top_Tooltip", "Position nodes towards one edge of the graph."))
				.Icon_Lambda([this]() -> const FSlateBrush*
				{
					if (!GraphWidgetContents.IsValid())
					{
						return nullptr;
					}

					return GraphWidgetContents->GetGraphOrientation() == EOrientation::Orient_Horizontal ? FSlateStyle::Get().GetBrush("AudioInsights.Icon.SignalFlow.AlignHorizontalTop")
																										 : FSlateStyle::Get().GetBrush("AudioInsights.Icon.SignalFlow.AlignVerticalLeft");
				})

				+ SSegmentedControl<ESignalFlowJustification>::Slot(ESignalFlowJustification::Center)
				.ToolTip(LOCTEXT("AudioDashboard_SignalFlow_Justification_Center_Tooltip", "Position nodes around the center of the graph."))
				.Icon_Lambda([this]() -> const FSlateBrush*
				{
					if (!GraphWidgetContents.IsValid())
					{
						return nullptr;
					}

					return GraphWidgetContents->GetGraphOrientation() == EOrientation::Orient_Horizontal ? FSlateStyle::Get().GetBrush("AudioInsights.Icon.SignalFlow.AlignHorizontalCenter")
																										 : FSlateStyle::Get().GetBrush("AudioInsights.Icon.SignalFlow.AlignVerticalCenter");
				})
			]
		];
	}

#if MUTE_SOLO_ENABLED
	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbarClearMuteSoloButton()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		const FSlateStyle& Style = FSlateStyle::Get();

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleName))
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_ClearMutesAndSolosTooltip", "Clears all assigned mute/solo states."))
			.OnClicked_Lambda([this]()
			{
				if (FAudioDeviceManager* const AudioDeviceManager = FAudioDeviceManager::Get())
				{
					AudioDeviceManager->GetDebugger().ClearMutesAndSolos();
				}
				return FReply::Handled();
			})
			[
				Style.CreateButtonContentWidget("AudioInsights.Icon.SoundDashboard.Reset", LOCTEXT("AudioDashboard_SignalFlow_ClearMutesAndSolosText", "Clear All Mutes/Solos"), ButtonTextStyleName)
			];
	}
#endif // MUTE_SOLO_ENABLED

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateToolbarIndicators()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		const FSlateStyle& Style = FSlateStyle::Get();

		TFunction<FSlateColor()> HighlightPathColorFunc = []()
		{
			return FSlateApplication::Get().GetModifierKeys().IsShiftDown() ? FStyleColors::AccentYellow : FSlateColor::UseForeground();
		};

		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolbarButtonPaddingX, ToolbarButtonPaddingY)
		[
			SNew(SBox)
			.ToolTipText(LOCTEXT("AudioDashboard_SignalFlow_HighlightPathTooltip", "Hold Shift and click a node to highlight its connection path."))
			[
				Style.CreateToggleButtonContent("AudioInsights.Icon.SignalFlow.HighlightPath",
												ButtonTextStyleName,
												HighlightPathColorFunc,
												[]() { return LOCTEXT("AudioDashboard_SignalFlow_HighlightPath", "Highlight Path"); },
												ToolbarButtonHeight)
			]
		];
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::OnGetSettingsMenuContent()
	{
		if (!CommandList.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		FMenuBuilder MenuBuilder(false /*bShouldCloseWindowAfterMenuSelection*/, CommandList);

		const FSignalFlowEditorCommands& Commands = FSignalFlowEditorCommands::Get();

		MenuBuilder.BeginSection("SignalFlowSettingActions", LOCTEXT("AudioDashboard_SignalFlow_Settings_HeaderText", "Settings"));
		{
			CreateAmpDisplayModeToggle(MenuBuilder);
			MenuBuilder.AddMenuEntry(Commands.GetToggleAnimateWiresCommand());

			MenuBuilder.AddSubMenu
			(
				LOCTEXT("AudioDashboard_SignalFlow_Settings_NodeDetailFilters", "Node Detail Filters"),
				LOCTEXT("AudioDashboard_SignalFlow_Settings_NodeDetailFiltersTooltip", "Choose which parameters are visible on the Signal Flow graph"),
				FNewMenuDelegate::CreateRaw(this, &FSignalFlowDashboardViewFactory::BuildNodeDetailFiltersMenuContent)
			);
		}

		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("GraphPadding", LOCTEXT("AudioDashboard_SignalFlow_Settings_GraphPadding_HeaderText", "Graph Padding"));
		{
			CreateNodePaddingControls(MenuBuilder);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void FSignalFlowDashboardViewFactory::BuildJustificationMenuContent(FMenuBuilder& OutMenuBuilder)
	{
		const FSignalFlowEditorCommands& Commands = FSignalFlowEditorCommands::Get();

		OutMenuBuilder.AddMenuEntry(Commands.GetJustifyEdgeCommand());
		OutMenuBuilder.AddMenuEntry(Commands.GetJustifyCenterCommand());
	}

	void FSignalFlowDashboardViewFactory::BuildNodeDetailFiltersMenuContent(FMenuBuilder& OutMenuBuilder)
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		const FSignalFlowEditorCommands& Commands = FSignalFlowEditorCommands::Get();

		OutMenuBuilder.AddMenuEntry
		(
			LOCTEXT("AudioDashboard_SignalFlow_Settings_ToggleAllNodeDetails", "Show all"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateSP(this, &FSignalFlowDashboardViewFactory::ToggleEnableAllNodeDetailFilters),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FGetActionCheckState::CreateLambda([this]() { return NodeDetailFilters.GetShowAllNodeDetailFiltersCheckboxState(); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		const uint8 NumFilters = static_cast<uint8>(ESignalFlowNodeDetailParam::MAX);

		for (uint8 FilterID = 0u; FilterID < NumFilters; ++FilterID)
		{
			const ESignalFlowNodeDetailParam ParamType = static_cast<ESignalFlowNodeDetailParam>(FilterID);

			OutMenuBuilder.AddMenuEntry
			(
				GetParamFilterName(ParamType),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateLambda([this, ParamType]()
					{
						const bool bParamIsVisible = NodeDetailFilters.GetParameterIsVisible(ParamType);
						NodeDetailFilters.SetParameterVisibility(ParamType, !bParamIsVisible);

#if WITH_EDITOR
						FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
					}),
					FCanExecuteAction::CreateLambda([]() { return true; }),
					FGetActionCheckState::CreateLambda([this, ParamType]()
					{
						return NodeDetailFilters.GetParameterIsVisible(ParamType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	void FSignalFlowDashboardViewFactory::CreateAmpDisplayModeToggle(FMenuBuilder& OutMenuBuilder)
	{
		OutMenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				bDisplayAmpPeakInDb = !bDisplayAmpPeakInDb;
#if WITH_EDITOR
				FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
			})),

			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Margin(FMargin(0.0f, 4.0f, 5.0f, 0.0f))
				.Text(LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpPeakDisplayMode_Text", "Amp (Peak) Display Mode"))
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SSegmentedControl<bool>)
				.OnValueChanged_Lambda([this](bool bInValue)
				{
					bDisplayAmpPeakInDb = bInValue;
#if WITH_EDITOR
					FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
				})
				.Value_Lambda([this]
				{
					return bDisplayAmpPeakInDb;
				})
					
				// Decibels mode
				+ SSegmentedControl<bool>::Slot(true)
				.Text(LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpDisplayMode_dB_Text", "dB"))
				.ToolTip(LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpDisplayMode_dB_ToolTipText", "Displays amplitude values in decibels."))
				
				// Linear mode
				+ SSegmentedControl<bool>::Slot(false)
				.Text(LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpDisplayMode_Linear_Text", "Lin"))
				.ToolTip(LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpDisplayMode_Linear_TooltipText", "Displays amplitude values in linear scale."))
			],
			NAME_None /*InExtensionHook*/,
			LOCTEXT("AudioDashboard_SignalFlow_Settings_AmpPeakDisplayMode_ToolTipText", "Switches the amplitude display mode between decibels or linear scale."),
			EUserInterfaceActionType::CollapsedButton
		);
	}

	void FSignalFlowDashboardViewFactory::CreateNodePaddingSpinBox(FMenuBuilder& OutMenuBuilder, const FText& Label, const FText& Tooltip, float& OutValue)
	{
		using namespace SSignalFlowGraphStyle;
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		OutMenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction()),
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(NodePaddingLabelWidth)
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0, 4.0, 5.0, 0.0))
					.Text(Label)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(NodePaddingSpinBoxWidth)
				[
					SNew(SSpinBox<float>)
					.MinValue(0.0f)
					.MaxValue(1.0f)
					.MinSliderValue(0.0f)
					.MaxSliderValue(1.0f)
					.SliderExponent(2.0f)
					.MaxFractionalDigits(3)
					.Value_Lambda([&OutValue]() { return NodePaddingToNormalized(OutValue); })
					.OnValueChanged_Lambda([this, &OutValue](const float NewValue)
					{
						OutValue = NormalizedToNodePadding(NewValue);
#if WITH_EDITOR
						FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
					})
				]
			],
			NAME_None /*InExtensionHook*/,
			Tooltip,
			EUserInterfaceActionType::CollapsedButton
		);
	}

	void FSignalFlowDashboardViewFactory::CreateNodePaddingControls(FMenuBuilder& OutMenuBuilder)
	{
		const bool bIsHorizontal = GraphWidgetContents.IsValid()
			&& GraphWidgetContents->GetGraphOrientation() == EOrientation::Orient_Horizontal;

		float& HorizontalValue = bIsHorizontal ? LargeNodePadding : SmallNodePadding;
		float& VerticalValue = bIsHorizontal ? SmallNodePadding : LargeNodePadding;

		CreateNodePaddingSpinBox(
			OutMenuBuilder,
			LOCTEXT("AudioDashboard_SignalFlow_Settings_HorizontalNodePadding_Text", "Horizontal Padding"),
			LOCTEXT("AudioDashboard_SignalFlow_Settings_HorizontalNodePadding_Tooltip", "Padding between nodes along the horizontal axis."),
			HorizontalValue);

		CreateNodePaddingSpinBox(
			OutMenuBuilder,
			LOCTEXT("AudioDashboard_SignalFlow_Settings_VerticalNodePadding_Text", "Vertical Padding"),
			LOCTEXT("AudioDashboard_SignalFlow_Settings_VerticalNodePadding_Tooltip", "Padding between nodes along the vertical axis."),
			VerticalValue);
	}

	FReply FSignalFlowDashboardViewFactory::ScrollGraphViewToSelectedNode(const bool bResetZoom)
	{
		if (GraphWidgetContents.IsValid())
		{
			const TSharedPtr<FSignalFlowDashboardEntry> FocusedEntry = SelectedItem.IsValid() ? SelectedItem : GetFilteredAudioDeviceEntry();
			if (FocusedEntry.IsValid())
			{
				const TSharedPtr<SSignalFlowGraphNode> FocusedNodeWidget = GraphWidgetContents->FindNodeWidget(FocusedEntry->GetSignalFlowEntryKey());
				if (FocusedNodeWidget.IsValid())
				{
					if (bResetZoom)
					{
						GraphWidgetContents->ResetZoom();
					}
					else
					{
						if (GraphHorizontalScrollBox.IsValid())
						{
							GraphHorizontalScrollBox->ScrollDescendantIntoView(FocusedNodeWidget, true, EDescendantScrollDestination::Center);
						}

						if (GraphVerticalScrollBox.IsValid())
						{
							GraphVerticalScrollBox->ScrollDescendantIntoView(FocusedNodeWidget, true, EDescendantScrollDestination::Center);
						}
					}

					return FReply::Handled();
				}
			}
		}

		return FReply::Unhandled();
	}

	void FSignalFlowDashboardViewFactory::ToggleGraphOrientation()
	{
		if (GraphWidgetContents.IsValid())
		{
			GraphWidgetContents->ToggleOrientation();
		}

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	

		OnRequestGraphRefresh();
	}

	void FSignalFlowDashboardViewFactory::ToggleShowNodeDetails()
	{
		bShowNodeDetails = !bShowNodeDetails;
		OnRequestGraphRefresh();

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
	}

	void FSignalFlowDashboardViewFactory::TogglePauseGraphEnabled()
	{
		bPauseGraphOnSelect = !bPauseGraphOnSelect;

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
	}

	void FSignalFlowDashboardViewFactory::ToggleEnableAllNodeDetailFilters()
	{
		NodeDetailFilters.ToggleEnableAllNodeDetailFilters();

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
	}

	void FSignalFlowDashboardViewFactory::ToggleAnimateWires()
	{
		bAnimateWires = !bAnimateWires;

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
	}

	bool FSignalFlowDashboardViewFactory::CanPauseTimestamp() const
	{
		if (!bPauseGraphOnSelect)
		{
			return false;
		}

		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();

#if !WITH_EDITOR
		if (!IAudioInsightsModule::IsLiveSession())
		{
			// Don't update the caching and processing status if the cache isn't limited (i.e. loading in a .urace file)
			return false;
		}
#endif // !WITH_EDITOR

		return AudioInsightsModule.GetTimingViewExtender().GetMessageCacheAndProcessingStatus() == ECacheAndProcess::Latest;
	}

	bool FSignalFlowDashboardViewFactory::IsTimestampPaused() const
	{
		FAudioInsightsTimingViewExtender& TimingViewExtender = FAudioInsightsModule::GetChecked().GetTimingViewExtender();
		return TimingViewExtender.GetMessageCacheAndProcessingStatus() != ECacheAndProcess::Latest;
	}

	void FSignalFlowDashboardViewFactory::ResetPauseTimestamp()
	{
		FAudioInsightsTimingViewExtender& TimingViewExtender = FAudioInsightsModule::GetChecked().GetTimingViewExtender();
		if (TimingViewExtender.GetMessageCacheAndProcessingStatus() == ECacheAndProcess::Latest)
		{
			return;
		}

		TimingViewExtender.ResumeTimeMarker();
	}

	void FSignalFlowDashboardViewFactory::SetJustification(const ESignalFlowJustification Justification)
	{
		GraphJustification = Justification;

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR

		OnRequestGraphRefresh();
		RefreshFilteredEntriesListView();
	}

	TSharedPtr<FSignalFlowDashboardEntry> FSignalFlowDashboardViewFactory::GetFilteredAudioDeviceEntry() const
	{
		const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();
		if (!Provider.IsValid())
		{
			return nullptr;
		}

		const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData();
		if (DeviceData == nullptr || DeviceData->IsEmpty())
		{
			return nullptr;
		}

		const ::Audio::FDeviceId DeviceID = Provider->GetFilteredAudioDeviceID();
		const TSharedPtr<FSignalFlowDashboardEntry>* DeviceEntry = DeviceData->Find({ DeviceID, ESignalFlowEntryType::AudioDevice, DeviceID });

		if (DeviceEntry == nullptr || !DeviceEntry->IsValid())
		{
			return nullptr;
		}

		return *DeviceEntry;
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateSelectionPanel()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		TSharedRef<SWidget> OwnerObjectsWidget = CreateSelectableList(LOCTEXT("AudioDashboard_SignalFlow_ActiveOwnerObjectsTitle", "Owner Objects"), ActiveOwnerObjects, ESignalFlowEntryType::OwnerObject, ActiveOwnerObjectsListView);
		TSharedRef<SWidget> SourcesWidget = CreateSelectableList(LOCTEXT("AudioDashboard_SignalFlow_ActiveSourcesTitle", "Active Sources"), ActiveSources, ESignalFlowEntryType::SoundSource, ActiveSourcesListView);
		TSharedRef<SWidget> BusesWidget = CreateSelectableList(LOCTEXT("AudioDashboard_SignalFlow_ActiveBusesTitle", "Active Buses"), ActiveBuses, ESignalFlowEntryType::AudioBus, ActiveBusesListView);
		TSharedRef<SWidget> SubmixesWidget = CreateSelectableList(LOCTEXT("AudioDashboard_SignalFlow_ActiveSubmixesTitle", "Active Submixes"), ActiveSubmixes, ESignalFlowEntryType::Submix, ActiveSubmixesListView);

		SAssignNew(CategorySplitter, SSplitter)
		.Orientation(Orient_Vertical)
		.ResizeMode(ESplitterResizeMode::Fill)
		.PhysicalSplitterHandleSize(4.0f)
		.HitDetectionSplitterHandleSize(6.0f)
		.OnSplitterFinishedResizing(this, &FSignalFlowDashboardViewFactory::OnCategorySplitterFinishedResizing)
		// Owner Objects
		+ SSplitter::Slot()
		.Value(ActiveEntryMenuExpansionSettings.OwnerObjectsSlotSize)
		.MinSize(CollapsedActiveEntryMenuHeight)
		[
			OwnerObjectsWidget
		]
		// Active Sources
		+ SSplitter::Slot()
		.Value(ActiveEntryMenuExpansionSettings.SourcesSlotSize)
		.MinSize(CollapsedActiveEntryMenuHeight)
		[
			SourcesWidget
		]
		// Active Buses
		+ SSplitter::Slot()
		.Value(ActiveEntryMenuExpansionSettings.BusesSlotSize)
		.MinSize(CollapsedActiveEntryMenuHeight)
		[
			BusesWidget
		]
		// Active Submixes
		+ SSplitter::Slot()
		.Value(ActiveEntryMenuExpansionSettings.SubmixesSlotSize)
		.MinSize(CollapsedActiveEntryMenuHeight)
		[
			SubmixesWidget
		];

		ApplyAllCategorySlotSizes();

		return SNew(SVerticalBox)
			// Search box
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(FSlateStyle::Get().GetSearchBoxMaxHeight())
			.Padding(0.0f, 4.0f, 0.0f, 6.0f)
			[
				SNew(SSearchBox)
				.SelectAllTextWhenFocused(true)
				.HintText(LOCTEXT("AudioDashboard_SignalFlow_SearchBoxHintText", "Search"))
				.MinDesiredWidth(200.0f)
				.OnTextChanged(this, &FSignalFlowDashboardViewFactory::SetSearchBoxFilterText)
			]
			// Category lists
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				CategorySplitter.ToSharedRef()
			];
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateGraphSection()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		TSharedRef<SOverlay> GraphWidget = SNew(SOverlay)

		+ SOverlay::Slot()
		[
			SAssignNew(GraphVerticalScrollBox, SSignalFlowScrollBox)
			.Orientation(EOrientation::Orient_Vertical)
			.ScrollBarVisibility(EVisibility::Collapsed)

			+ SScrollBox::Slot()
			.FillSize(1.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(GraphHorizontalScrollBox, SSignalFlowScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				.ScrollBarVisibility(EVisibility::Collapsed)

				+ SScrollBox::Slot()
				.FillSize(1.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SOverlay)

					// Graph background
					+ SOverlay::Slot()
					[
						SNew(SColorBlock)
						.Color(FStyleColors::Background.GetSpecifiedColor())
					]

					// Graph
					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(GraphWidgetContents, SSignalFlowGraph)
						.ListSource(&FilteredGraphNodes)
						.NodeDetailFilterSettings(&NodeDetailFilters)
						.LargeNodePadding_Lambda([this]() { return LargeNodePadding; })
						.SmallNodePadding_Lambda([this]() { return SmallNodePadding; })
						.SelectedItem_Lambda([this]()
						{
							return SelectedItem;
						})
						.HighlightedItem_Lambda([this]()
						{
							return HighlightedItem;
						})
						.ActiveAudioDeviceEntry_Lambda([this]()
						{
							return GetFilteredAudioDeviceEntry();
						})
						.FocusedItem_Lambda([this]() -> TSharedPtr<FSignalFlowDashboardEntry>
						{
							if (XPosFocusNodeEntryKey.IsSet())
							{
								const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();
								const FSignalFlowTraceProvider::FDeviceData* const DeviceData = Provider.IsValid() ? Provider->FindFilteredDeviceData() : nullptr;
								if (DeviceData)
								{
									const TSharedPtr<FSignalFlowDashboardEntry>* const Entry = DeviceData->Find(XPosFocusNodeEntryKey.GetValue());
									if (Entry && Entry->IsValid())
									{
										return *Entry;
									}
								}
							}

							return GetFilteredAudioDeviceEntry();
						})
						.OnNodeSelected_Lambda([this](TSharedPtr<FSignalFlowDashboardEntry> SelectedNodeEntry)
						{
							if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
							{
								if (SelectedNodeEntry.IsValid())
								{
									SetHighlightedItem(SelectedNodeEntry);
								}
								return;
							}

							if (SelectedNodeEntry.IsValid())
							{
								const TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> SelectedItemListView = GetSelectableList(SelectedNodeEntry->EntryType);
								if (SelectedItemListView.IsValid())
								{
									ActiveOwnerObjectsListView->SetSelection(SelectedNodeEntry);
								}
								else
								{
									if (ActiveOwnerObjectsListView.IsValid())
									{
										ActiveOwnerObjectsListView->ClearSelection();
									}

									if (ActiveSourcesListView.IsValid())
									{
										ActiveSourcesListView->ClearSelection();
									}

									if (ActiveBusesListView.IsValid())
									{
										ActiveBusesListView->ClearSelection();
									}

									if (ActiveSubmixesListView.IsValid())
									{
										ActiveSubmixesListView->ClearSelection();
									}
								}
							}

							OnSelectionChanged(SelectedNodeEntry, ESelectInfo::OnMouseClick);
						})
						.GraphJustification_Lambda([this]()
						{
							return GraphJustification;
						})
						.ShowNodeDetails_Lambda([this]()
						{
							return bShowNodeDetails;
						})
						.IsEntryFilteredOutByText_Lambda([this](const FSignalFlowDashboardEntry& Entry)
						{
							return IsEntryFilteredOutByText(Entry);
						})
						.DisplayAmpPeakInDb_Lambda([this]()
						{
							return bDisplayAmpPeakInDb;
						})
						.AnimateWires_Lambda([this]()
						{
							return bAnimateWires;
						})
						.HighlightPathActive_Lambda([this]()
						{
							return !HighlightedPathKeys.IsEmpty();
						})
						.IsInHighlightedPath_Lambda([this](const FSignalFlowEntryKey& Key) -> bool
						{
							return HighlightedPathKeys.Contains(Key);
						})
					]
				]
			]
		]
		
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(24.0f)
		[
			SNew(SBorder)
			.BorderImage(&GetRoundedBoxBrush())
			.BorderBackgroundColor(FStyleColors::AccentPurple.GetSpecifiedColor().CopyWithNewOpacity(0.5f))
			.Visibility_Lambda([this]()
			{
#if WITH_EDITOR
				return IsTimestampPaused() ? EVisibility::Visible : EVisibility::Collapsed;
#else
				if (IsTimestampPaused())
				{
					return IsReadingTraceFile() ? EVisibility::Collapsed : EVisibility::Visible;
				}

				return EVisibility::Collapsed;
#endif // WITH_EDITOR
			})
			.Padding(8.0f, 4.0f)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AudioDashboard_SignalFlow_GraphPausedWarning", "Processing is paused! Press Esc to resume."))
				.Font(FCoreStyle::Get().GetFontStyle("BoldFont"))
				.ColorAndOpacity(FStyleColors::White.GetSpecifiedColor())
			]
		];

		if (GraphWidgetContents.IsValid())
		{
			GraphWidgetContents->SetHorizontalScrollBox(GraphHorizontalScrollBox);
			GraphWidgetContents->SetVerticalScrollBox(GraphVerticalScrollBox);
		}

		return GraphWidget;
	}

	void FSignalFlowDashboardViewFactory::OnMainSplitterFinishedResizing()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		if (!MainSplitter.IsValid())
		{
			return;
		}

		const float RawRatio = MainSplitter->SlotAt(MainSplitterGraphSectionSlotIndex).GetSizeValue();
		const float Clamped = FMath::Clamp(RawRatio, MinGraphWidthRatio, MaxGraphWidthRatio);

		if (!FMath::IsNearlyEqual(RawRatio, Clamped))
		{
			MainSplitter->SlotAt(MainSplitterSelectionPanelSlotIndex).SetSizeValue(1.0f - Clamped);
			MainSplitter->SlotAt(MainSplitterGraphSectionSlotIndex).SetSizeValue(Clamped);
		}

		GraphWidthRatio = Clamped;

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
	}

	TSharedRef<SWidget> FSignalFlowDashboardViewFactory::CreateSelectableList(const FText& HeadingText, const TArray<TSharedPtr<FSignalFlowDashboardEntry>>& ListSource, const ESignalFlowEntryType EntryType, TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>>& OutListView)
	{
		TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow)
												 .CanSelectGeneratedColumn(false);

		static const FLazyName HeadingColumnID("Heading");
		const SHeaderRow::FColumn::FArguments NameColumnArgs = SHeaderRow::Column(HeadingColumnID)
			.DefaultLabel(HeadingText)
			.HAlignCell(HAlign_Left)
			.FillWidth(1.0f)
			.HeaderContentPadding(FMargin(4.0f, 4.0f))
			.HeaderContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.MinSize(12.0f)
				.MaxSize(12.0f)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SCheckBox)
					.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("AudioInsights.RightExpandArrowToggle"))
					.IsChecked_Lambda([this, EntryType]()
					{
						return GetSelectableListIsExpanded(EntryType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this, EntryType](const ECheckBoxState CheckboxState)
					{
						const bool bMenuIsExpanded = CheckboxState == ECheckBoxState::Checked;
						SetSelectableListIsExpanded(EntryType, bMenuIsExpanded);
					})
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text(HeadingText)
					.Font(FCoreStyle::Get().GetFontStyle("BoldFont"))
				]
			];

		HeaderRowWidget->AddColumn(NameColumnArgs);

		OutListView = SNew(SListView<TSharedPtr<FSignalFlowDashboardEntry>>)
			.ListItemsSource(&ListSource)
			.SelectionMode(ESelectionMode::SingleToggle)
			.Visibility_Lambda([this, EntryType]()
			{
				return GetSelectableListIsExpanded(EntryType) ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.OnSelectionChanged(this, &FSignalFlowDashboardViewFactory::OnListSelectionChanged)
#if WITH_EDITOR
			.OnContextMenuOpening(this, &FSignalFlowDashboardViewFactory::OnConstructContextMenu)
#endif // WITH_EDITOR
			.OnGenerateRow_Lambda([this, EntryType](TSharedPtr<FSignalFlowDashboardEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(SRowWidget, OwnerTable, Item)
					.Style(GetRowStyle());
			});

		return SNew(SVerticalBox)
			// Header
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				HeaderRowWidget
			]
			// List rows
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				OutListView.ToSharedRef()
			];
	}

	TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> FSignalFlowDashboardViewFactory::GetSelectableList(const ESignalFlowEntryType EntryType)
	{
		switch (EntryType)
		{
			case ESignalFlowEntryType::OwnerObject:
				return ActiveOwnerObjectsListView;

			case ESignalFlowEntryType::SoundSource:
				return ActiveSourcesListView;

			case ESignalFlowEntryType::AudioBus:
				return ActiveBusesListView;

			case ESignalFlowEntryType::Submix:
				return ActiveSubmixesListView;

			default:
				break;
		}

		return nullptr;
	}

	bool FSignalFlowDashboardViewFactory::GetSelectableListIsExpanded(const ESignalFlowEntryType EntryType) const
	{
		switch (EntryType)
		{
			case ESignalFlowEntryType::OwnerObject:
				return ActiveEntryMenuExpansionSettings.bShowActiveOwnerObjects;

			case ESignalFlowEntryType::SoundSource:
				return ActiveEntryMenuExpansionSettings.bShowActiveSources;

			case ESignalFlowEntryType::AudioBus:
				return ActiveEntryMenuExpansionSettings.bShowActiveBuses;

			case ESignalFlowEntryType::Submix:
				return ActiveEntryMenuExpansionSettings.bShowActiveSubmixes;

			default:
				ensureMsgf(false, TEXT("Unsupported ESignalFlowEntryType in FSignalFlowDashboardViewFactory::GetSelectableListIsExpanded"));
				break;
		}

		return false;
	}

	void FSignalFlowDashboardViewFactory::SetSelectableListIsExpanded(const ESignalFlowEntryType EntryType, const bool bIsExpanded)
	{
		switch (EntryType)
		{
			case ESignalFlowEntryType::OwnerObject:
				ActiveEntryMenuExpansionSettings.bShowActiveOwnerObjects = bIsExpanded;
				SetSelectableListSlotSize(ESignalFlowEntryType::OwnerObject, bIsExpanded, ActiveEntryMenuExpansionSettings.OwnerObjectsSlotSize);
				break;

			case ESignalFlowEntryType::SoundSource:
				ActiveEntryMenuExpansionSettings.bShowActiveSources = bIsExpanded;
				SetSelectableListSlotSize(ESignalFlowEntryType::SoundSource, bIsExpanded, ActiveEntryMenuExpansionSettings.SourcesSlotSize);
				break;

			case ESignalFlowEntryType::AudioBus:
				ActiveEntryMenuExpansionSettings.bShowActiveBuses = bIsExpanded;
				SetSelectableListSlotSize(ESignalFlowEntryType::AudioBus, bIsExpanded, ActiveEntryMenuExpansionSettings.BusesSlotSize);
				break;

			case ESignalFlowEntryType::Submix:
				ActiveEntryMenuExpansionSettings.bShowActiveSubmixes = bIsExpanded;
				SetSelectableListSlotSize(ESignalFlowEntryType::Submix, bIsExpanded, ActiveEntryMenuExpansionSettings.SubmixesSlotSize);
				break;

			default:
				ensureMsgf(false, TEXT("Unsupported ESignalFlowEntryType in FSignalFlowDashboardViewFactory::SetSelectableListIsExpanded"));
				break;
		}

#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
	}

	void FSignalFlowDashboardViewFactory::SetSelectableListSlotSize(const ESignalFlowEntryType EntryType, const bool bIsExpanded, float& OutSlotSize)
	{
		if (!CategorySplitter.IsValid())
		{
			return;
		}

		const int32 SlotIndex = static_cast<int32>(EntryType);

		if (bIsExpanded)
		{
			CategorySplitter->SlotAt(SlotIndex).SetSizingRule(SSplitter::FractionOfParent);
			CategorySplitter->SlotAt(SlotIndex).SetSizeValue(OutSlotSize);
		}
		else
		{
			OutSlotSize = CategorySplitter->SlotAt(SlotIndex).GetSizeValue();
			CategorySplitter->SlotAt(SlotIndex).SetSizingRule(SSplitter::SizeToContent);
		}
	}

	void FSignalFlowDashboardViewFactory::ApplyAllCategorySlotSizes()
	{
		SetSelectableListSlotSize(ESignalFlowEntryType::OwnerObject, ActiveEntryMenuExpansionSettings.bShowActiveOwnerObjects, ActiveEntryMenuExpansionSettings.OwnerObjectsSlotSize);
		SetSelectableListSlotSize(ESignalFlowEntryType::SoundSource, ActiveEntryMenuExpansionSettings.bShowActiveSources, ActiveEntryMenuExpansionSettings.SourcesSlotSize);
		SetSelectableListSlotSize(ESignalFlowEntryType::AudioBus, ActiveEntryMenuExpansionSettings.bShowActiveBuses, ActiveEntryMenuExpansionSettings.BusesSlotSize);
		SetSelectableListSlotSize(ESignalFlowEntryType::Submix, ActiveEntryMenuExpansionSettings.bShowActiveSubmixes, ActiveEntryMenuExpansionSettings.SubmixesSlotSize);
	}

	void FSignalFlowDashboardViewFactory::OnCategorySplitterFinishedResizing()
	{
#if WITH_EDITOR
		FSignalFlowSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
	}

	void FSignalFlowDashboardViewFactory::SetSearchBoxFilterText(const FText& NewText)
	{
		SearchBoxFilterText = NewText.ToString();
		UpdateFilterReason = EProcessReason::FilterUpdated;
	}

	void FSignalFlowDashboardViewFactory::OnListSelectionChanged(TSharedPtr<FSignalFlowDashboardEntry> InSelectedItem, ESelectInfo::Type SelectInfo)
	{
		OnSelectionChanged(InSelectedItem, SelectInfo, true /*bCenterViewAfterSelection*/);
	}

	void FSignalFlowDashboardViewFactory::OnSelectionChanged(TSharedPtr<FSignalFlowDashboardEntry> InSelectedItem, ESelectInfo::Type SelectInfo, const bool bCenterViewAfterSelection/*= false*/)
	{
		if (SelectInfo == ESelectInfo::Direct)
		{
			return;
		}

		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown() && InSelectedItem.IsValid())
		{
			SetHighlightedItem(InSelectedItem);

			// Undo the selection change on the shift-clicked list
			const TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> NewlySelectedItemListView = GetSelectableList(InSelectedItem->EntryType);
			if (NewlySelectedItemListView.IsValid())
			{
				NewlySelectedItemListView->SetItemSelection(InSelectedItem, false, ESelectInfo::Direct);
			}

			// Re-select the current SelectedItem on its list in case it was on the same list
			if (SelectedItem.IsValid() && SelectedItem->EntryType == InSelectedItem->EntryType)
			{
				const TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> PreviouslySelectedItemListView = GetSelectableList(SelectedItem->EntryType);
				if (PreviouslySelectedItemListView.IsValid())
				{
					PreviouslySelectedItemListView->SetSelection(SelectedItem, ESelectInfo::Direct);
				}
			}

			return;
		}

		SetNewSelection(InSelectedItem, bCenterViewAfterSelection);

#if WITH_EDITOR
		FAudioInsightsDetailsSelectionManager& SelectionManager = IAudioInsightsModule::GetChecked().GetDetailsSelectionManager();
		TObjectPtr<UObject> Object = InSelectedItem.IsValid() ? InSelectedItem->GetObject() : nullptr;

		// If we could not find a loaded asset for this object, try to load one using the object path
		if (Object == nullptr && InSelectedItem.IsValid())
		{
			const FString ObjectPath = InSelectedItem->GetObjectPath();
			if (!ObjectPath.IsEmpty())
			{
				Object = FSoftObjectPath(ObjectPath).TryLoad();
			}
		}

		if (Object && Object->IsAsset())
		{
			SelectionManager.SetSelectedAsset(Object);
		}
		else
		{
			SelectionManager.ClearSelection();
		}
#endif // WITH_EDITOR
	}

	void FSignalFlowDashboardViewFactory::SetNewSelection(const TSharedPtr<FSignalFlowDashboardEntry>& InSelectedItem, const bool bCenterViewAfterSelection /*= false*/)
	{
		ClearListHighlights();
		HighlightedItem.Reset();
		HighlightedPathKeys.Reset();

		if (!ActiveOwnerObjectsListView.IsValid() || !ActiveSourcesListView.IsValid() || !ActiveBusesListView.IsValid() || !ActiveSubmixesListView.IsValid())
		{
#if WITH_EDITOR
			AssetContextMenuHelper.ResetAssetEntry();
#endif // WITH_EDITOR
			SelectedItem.Reset();
			return;
		}

		FilteredEntries.Reset();

		if (InSelectedItem.IsValid())
		{
			switch (InSelectedItem->EntryType)
			{
				case ESignalFlowEntryType::OwnerObject:
				{
					ActiveSourcesListView->ClearSelection();
					ActiveBusesListView->ClearSelection();
					ActiveSubmixesListView->ClearSelection();
					break;
				}
				case ESignalFlowEntryType::SoundSource:
				{
					ActiveOwnerObjectsListView->ClearSelection();
					ActiveBusesListView->ClearSelection();
					ActiveSubmixesListView->ClearSelection();
					break;
				}
				case ESignalFlowEntryType::AudioBus:
				{
					ActiveOwnerObjectsListView->ClearSelection();
					ActiveSourcesListView->ClearSelection();
					ActiveSubmixesListView->ClearSelection();
					break;
				}
				case ESignalFlowEntryType::Submix:
				{
					ActiveOwnerObjectsListView->ClearSelection();
					ActiveSourcesListView->ClearSelection();
					ActiveBusesListView->ClearSelection();
					break;
				}
			}

			SelectedItem = InSelectedItem;
			if (SelectedItem.IsValid())
			{
#if WITH_EDITOR
				AssetContextMenuHelper.SetAssetEntry(SelectedItem);
#endif // WITH_EDITOR

				if (CanPauseTimestamp())
				{
					FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
					const double PauseTime = AudioInsightsModule.GetCacheManager().GetCacheEndTimeStamp();
					AudioInsightsModule.GetTimingViewExtender().PauseTimeMarker(PauseTime, ESystemControllingTimeMarker::SignalFlow);
				}
			}
		}
		else
		{
#if WITH_EDITOR
			AssetContextMenuHelper.ResetAssetEntry();
#endif // WITH_EDITOR

			if (SelectedItem.IsValid())
			{
				XPosFocusNodeEntryKey = SelectedItem->GetSignalFlowEntryKey();
			}

			SelectedItem.Reset();
			ActiveOwnerObjectsListView->ClearSelection();
			ActiveSourcesListView->ClearSelection();
			ActiveBusesListView->ClearSelection();
			ActiveSubmixesListView->ClearSelection();
		}

		OnRequestGraphRefresh();

		if (bCenterViewAfterSelection && SelectedItem.IsValid())
		{
			bAutoFocusAfterGraphRefresh = true;
		}

		RefreshFilteredEntriesListView();

		if (!SelectedItem.IsValid())
		{
			const TSharedPtr<FSignalFlowDashboardEntry> DeviceEntry = GetFilteredAudioDeviceEntry();
			if (DeviceEntry.IsValid())
			{
				XPosFocusNodeEntryKey = DeviceEntry->GetSignalFlowEntryKey();
			}
		}
	}

	void FSignalFlowDashboardViewFactory::RecomputeHighlightedPathKeys(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData)
	{
		HighlightedPathKeys.Reset();
		TSet<FSignalFlowEntryKey> DummyExistingEntries;
		AddFilteredInputsRecursive(DeviceData, HighlightedItem, nullptr, 0, DummyExistingEntries, &HighlightedPathKeys);
		AddFilteredOutputs(DeviceData, HighlightedItem, 1, DummyExistingEntries, &HighlightedPathKeys);
	}

	void FSignalFlowDashboardViewFactory::ClearListHighlights()
	{
		if (HighlightedItem.IsValid())
		{
			const TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> HighlightedItemListView = GetSelectableList(HighlightedItem->EntryType);
			if (HighlightedItemListView.IsValid())
			{
				HighlightedItemListView->SetItemHighlighted(HighlightedItem, false);
			}
		}
	}

	void FSignalFlowDashboardViewFactory::SetHighlightedItem(const TSharedPtr<FSignalFlowDashboardEntry>& InHighlightedItem)
	{
		ClearListHighlights();

		HighlightedItem = InHighlightedItem;

		if (HighlightedItem.IsValid())
		{
			const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();
			if (Provider.IsValid())
			{
				const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData();
				if (DeviceData != nullptr)
				{
					RecomputeHighlightedPathKeys(*DeviceData);
				}
			}

			const TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> HighlightedItemListView = GetSelectableList(HighlightedItem->EntryType);
			if (HighlightedItemListView.IsValid())
			{
				HighlightedItemListView->SetItemHighlighted(HighlightedItem, true);
			}
		}
		else
		{
			HighlightedPathKeys.Reset();
		}
	}

	void FSignalFlowDashboardViewFactory::ProcessEntries(EProcessReason Reason)
	{
		const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();
		if (!Provider.IsValid())
		{
			return;
		}

		if (const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData())
		{
			BuildSelectionMenus(*DeviceData, Reason);
		}
	}

	void FSignalFlowDashboardViewFactory::BuildSelectionMenus(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const EProcessReason Reason)
	{
		ActiveOwnerObjects.Reset();
		ActiveSources.Reset();
		ActiveBuses.Reset();
		ActiveSubmixes.Reset();

		// Build the left-hand menus of selectable nodes from this device's data
		for (const auto& [Key, Entry] : DeviceData)
		{
			if (!Entry.IsValid())
			{
				continue;
			}

			if (IsEntryFilteredOutByText(*Entry))
			{
				continue;
			}

			switch (Entry->EntryType)
			{
				case ESignalFlowEntryType::OwnerObject:
					ActiveOwnerObjects.Add(Entry);
					break;
				case ESignalFlowEntryType::SoundSource:
					ActiveSources.Add(Entry);
					break;
				case ESignalFlowEntryType::AudioBus:
					ActiveBuses.Add(Entry);
					break;
				case ESignalFlowEntryType::Submix:
					ActiveSubmixes.Add(Entry);
					break;
				default:
					break;
			}
		}
	}

	bool FSignalFlowDashboardViewFactory::IsEntryFilteredOutByText(const FSignalFlowDashboardEntry& Entry) const
	{
		return !SearchBoxFilterText.IsEmpty() && !Entry.GetDisplayName().ToString().Contains(SearchBoxFilterText);
	}

	void FSignalFlowDashboardViewFactory::RefreshFilteredEntriesListView()
	{
		if (ActiveOwnerObjectsListView.IsValid())
		{
			ActiveOwnerObjectsListView->RequestListRefresh();
		}

		if (ActiveSourcesListView.IsValid())
		{
			ActiveSourcesListView->RequestListRefresh();
		}

		if (ActiveBusesListView.IsValid())
		{
			ActiveBusesListView->RequestListRefresh();
		}

		if (ActiveSubmixesListView.IsValid())
		{
			ActiveSubmixesListView->RequestListRefresh();
		}

		if (GraphWidgetContents.IsValid() && bGraphRequiresRefresh)
		{
			GraphWidgetContents->RefreshGraph();
			bGraphRequiresRefresh = false;

			if (bAutoFocusAfterGraphRefresh)
			{
				ScrollGraphViewToSelectedNode(false /*bResetZoom*/);
				bAutoFocusAfterGraphRefresh = false;
			}
		}
	}

	void FSignalFlowDashboardViewFactory::OnRequestGraphRefresh()
	{
		const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();

		if (!Provider.IsValid())
		{
			return;
		}

		const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData();
		if (DeviceData == nullptr)
		{
			return;
		}

		if (SelectedItem.IsValid())
		{
			const FSignalFlowEntryKey SelectedEntryKey = SelectedItem->GetSignalFlowEntryKey();

			// Check that the selected item still exists in the device data - if not we need to clear the selection first
			if (DeviceData->Contains(SelectedEntryKey))
			{
				// Selection is valid, begin filtering the graph
				XPosFocusNodeEntryKey = SelectedEntryKey;
				CreateFilteredNodeGraph(*Provider, SelectedItem, false /*bAddAllNodes*/);
			}
			else
			{
				// Reset the selection - this will refresh the graph again
				SetNewSelection(nullptr);

				SendSelectedNodeDestroyedNotification();
			}
		}
		else
		{
			AddAllNodesToGraph();
		}

		if (HighlightedItem.IsValid())
		{
			const FSignalFlowEntryKey HighlightedEntryKey = HighlightedItem->GetSignalFlowEntryKey();
			if (DeviceData == nullptr || !DeviceData->Contains(HighlightedEntryKey))
			{
				HighlightedItem.Reset();
				HighlightedPathKeys.Reset();
			}
			else
			{
				RecomputeHighlightedPathKeys(*DeviceData);
			}
		}
	}

	void FSignalFlowDashboardViewFactory::SendSelectedNodeDestroyedNotification()
	{
		if (bSentSelectedNodeDestroyedNotification)
		{
			return;
		}

		FNotificationInfo Info(LOCTEXT("AudioDashboard_SignalFlow_SelectedObjectDestroyedText", "Signal flow: The selected node was destroyed!"));
		Info.SubText = LOCTEXT("AudioDashboard_SignalFlow_SelectedObjectDestroyedSubText", "The graph has returned to an unfiltered state because the selected node has been removed from the graph.");
		Info.bFireAndForget = true;
		Info.FadeOutDuration = 1.0f;
		Info.ExpireDuration = 10.0f;

		FSlateNotificationManager::Get().AddNotification(Info);

		bSentSelectedNodeDestroyedNotification = true;
	}

	void FSignalFlowDashboardViewFactory::CreateFilteredNodeGraph(const FSignalFlowTraceProvider& Provider, TSharedPtr<FSignalFlowDashboardEntry> FilterFromEntry, const bool bAddAllNodes)
	{
		if (!FilterFromEntry.IsValid())
		{
			return;
		}

		const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider.FindFilteredDeviceData();
		if (DeviceData == nullptr)
		{
			return;
		}

		bGraphRequiresRefresh = true;

		// Make a note of the existing entries in the graph before we start the creation process
		// Any remaining existing nodes by the end of the process will be removed
		TSet<FSignalFlowEntryKey> ExistingEntries;
		ExistingEntries.Reserve(FilteredEntries.Num());
		for (const auto& [EntryKey, Entry] : FilteredEntries)
		{
			ExistingEntries.Add(EntryKey);
		}

		// Add nodes recursively from the FilterFromEntry, connecting direct input and output paths
		AddFilteredInputsRecursive(*DeviceData, FilterFromEntry, nullptr, 0, ExistingEntries);
		AddFilteredOutputs(*DeviceData, FilterFromEntry, 1, ExistingEntries);

		if (bAddAllNodes)
		{
			// Only try to locate missing nodes when the graph is not filtered
			// When filtered, we only want to show the directly connected paths to the filtering node, making this step unnecessary
			AddMissingNodesToGraph(*DeviceData, ExistingEntries);
		}
		
		// Remove any remaining nodes in the graph that were not discovered in the creation process
		CleanUpFilteredEntries(ExistingEntries);

		// Gather stats (such as root entry and tree depth structure)
		TOptional<FSignalFlowEntryKey> RootEntryKey;
		TArray<TreeDepthPair> SortedTreeDepths;
		CalculateFilteredGraphDepthStructure(RootEntryKey, SortedTreeDepths);

		if (!RootEntryKey.IsSet())
		{
			return;
		}

		const TSharedPtr<ISignalFlowNode>* RootNode = FilteredEntries.Find(RootEntryKey.GetValue());

		if (RootNode && RootNode->IsValid())
		{
			// Grab a handle to the shared ptr directly - the structure of the FilteredEntries map may change inside CreateDummyInputs
			// so a raw ptr handle may point to a different entry (or garbage) afterwards
			TSharedPtr<ISignalFlowNode> RootNodeSharedPtr = *RootNode;

			// Create dummy nodes in places where a connection between two nodes spans multiple rows.
			// This helps when trying to maintain good X-positioning and ordering
			CreateDummyInputs(SortedTreeDepths);

			// Now start at the root node and iterate upwards via inputs to assign an order to each node
			// This helps keep related nodes together
			int32 NodeProcessingOrder = 0;
			AssignNodeOrderRecursive(RootNodeSharedPtr, 0, NodeProcessingOrder);

			// Run through again and find any nodes that did not get an order assigned
			// This can happen if a node is not the root node but has no outputs
			FixUnassignedNodeOrders();

			// Generate the final node array that will be passed on to the SSignalFlowGraph widget
			FilteredEntries.GenerateValueArray(FilteredGraphNodes);

			// Sort the new array to be in tree depth/node connection order
			SortNodeOrder();

			// Assign X positions based off the new order
			// X positions given in units of 1.0 that will be scaled up by SSignalFlowGraph later
			CalcXPositions(RootNodeSharedPtr);
		}
	}

	void FSignalFlowDashboardViewFactory::AddAllNodesToGraph()
	{
		const TSharedPtr<const FSignalFlowTraceProvider> Provider = FindProvider<const FSignalFlowTraceProvider>();
		if (!Provider.IsValid())
		{
			return;
		}

		const TSharedPtr<FSignalFlowDashboardEntry> DeviceEntry = GetFilteredAudioDeviceEntry();
		if (DeviceEntry.IsValid())
		{
			const FSignalFlowTraceProvider::FDeviceData* DeviceData = Provider->FindFilteredDeviceData();
			const bool bXPosFocusEntryExists = XPosFocusNodeEntryKey.IsSet() && DeviceData && DeviceData->Contains(XPosFocusNodeEntryKey.GetValue());

			if (!bXPosFocusEntryExists)
			{
				XPosFocusNodeEntryKey = DeviceEntry->GetSignalFlowEntryKey();
			}

			CreateFilteredNodeGraph(*Provider, DeviceEntry, true /*bAddAllNodes*/);
		}
	}

	void FSignalFlowDashboardViewFactory::AddFilteredInputsRecursive(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, TSharedPtr<FSignalFlowDashboardEntry> ChildEntry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		TSharedPtr<ISignalFlowNode> EntryNode = nullptr;
		const FSignalFlowEntryKey EntryKey = Entry->GetSignalFlowEntryKey();

		if (OutPathKeys)
		{
			bool bAlreadyInSet = false;
			OutPathKeys->Add(EntryKey, &bAlreadyInSet);
			if (bAlreadyInSet)
			{
				return;
			}
		}
		else
		{
			TSharedPtr<ISignalFlowNode>* FoundNode = FilteredEntries.Find(EntryKey);
			if (FoundNode && FoundNode->IsValid())
			{
				if (OutExistingEntryKeys.Contains(EntryKey))
				{
					// First time we have tried to recreate this node but it was here already
					// Remove from tracking container and make a note of the new tree depth
					OutExistingEntryKeys.Remove(EntryKey);
					(*FoundNode)->GetTreeDepth().TreeDepth = TreeDepth;
				}
				else
				{
					// We have created and assigned assigned this node a tree depth - move it higher in the tree
					const ESignalFlowEntryType EntryType = EntryKey.EntryType;
					const bool bCanBeLayered = EntryType == ESignalFlowEntryType::AudioBus || EntryType == ESignalFlowEntryType::Submix;

					TreeDepthPair& NodeTreeDepth = (*FoundNode)->GetTreeDepth();

					if (bCanBeLayered && NodeTreeDepth.TreeDepth >= TreeDepth)
					{
						NodeTreeDepth.TreeDepth = TreeDepth - 1;
					}
				}

				EntryNode = *FoundNode;
			}
			else
			{
				ensure(!OutExistingEntryKeys.Contains(EntryKey));

				EntryNode = MakeShared<FSignalFlowEntryNode>(Entry, TreeDepth, Entry->EntryType, Entry->Timestamp);

				FilteredEntries.Add(Entry->GetSignalFlowEntryKey(), EntryNode);
			}

			if (!ensure(EntryNode.IsValid()))
			{
				// We should never hit this, but return just in case to avoid a crash
				return;
			}

			if (ChildEntry.IsValid())
			{
				EntryNode->FilteredOutputs.AddUnique(ChildEntry->GetSignalFlowEntryKey());
			}
		}

		for (const FSignalFlowEntryKey& InputKey : Entry->Inputs)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* InputEntry = DeviceData.Find(InputKey);

			if (InputEntry && InputEntry->IsValid())
			{
				if (EntryNode.IsValid())
				{
					EntryNode->FilteredInputs.AddUnique((*InputEntry)->GetSignalFlowEntryKey());
				}

				bool bResetTreeDepth = (*InputEntry)->EntryType != Entry->EntryType;
				AddFilteredInputsRecursive(DeviceData, *InputEntry, Entry, bResetTreeDepth ? 0 : TreeDepth - 1, OutExistingEntryKeys, OutPathKeys);
			}
		}

		AddFilteredLinkedSourceBus(DeviceData, Entry, EntryNode.Get(), OutExistingEntryKeys, OutPathKeys);
		AddFilteredLinkedBusPatchInputs(DeviceData, Entry, EntryNode.Get(), OutExistingEntryKeys, OutPathKeys);
	}

	void FSignalFlowDashboardViewFactory::AddFilteredOutputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		for (const auto& [OutputKey, SendInfo] : Entry->Outputs)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = DeviceData.Find(OutputKey);
			if (OutputEntry && OutputEntry->IsValid())
			{
				AddFilteredOutputsRecursive(DeviceData, *OutputEntry, Entry, TreeDepth, OutExistingEntryKeys, OutPathKeys);

				if (OutPathKeys == nullptr)
				{
					TSharedPtr<ISignalFlowNode>* EntryNode = FilteredEntries.Find(Entry->GetSignalFlowEntryKey());
					TSharedPtr<ISignalFlowNode>* OutputNode = FilteredEntries.Find(OutputKey);
					if (EntryNode && OutputNode && EntryNode->IsValid() && OutputNode->IsValid())
					{
						(*EntryNode)->FilteredOutputs.AddUnique(OutputKey);
					}
				}
			}
		}

		TSharedPtr<ISignalFlowNode> EntryNode = nullptr;
		if (OutPathKeys == nullptr)
		{
			TSharedPtr<ISignalFlowNode>* FoundNode = FilteredEntries.Find(Entry->GetSignalFlowEntryKey());
			if (FoundNode && FoundNode->IsValid())
			{
				EntryNode = *FoundNode;
			}
		}

		AddFilteredLinkedSources(DeviceData, Entry, EntryNode.Get(), OutExistingEntryKeys, OutPathKeys);
		AddFilteredLinkedBusPatchOutputs(DeviceData, Entry, EntryNode.Get(), OutExistingEntryKeys, OutPathKeys);
	}

	void FSignalFlowDashboardViewFactory::AddFilteredOutputsRecursive(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, TSharedPtr<FSignalFlowDashboardEntry> ParentEntry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		const FSignalFlowEntryKey& EntryKey = Entry->GetSignalFlowEntryKey();

		TSharedPtr<ISignalFlowNode> Node = nullptr;

		if (OutPathKeys != nullptr)
		{
			bool bAlreadyInSet = false;
			OutPathKeys->Add(EntryKey, &bAlreadyInSet);
			if (bAlreadyInSet)
			{
				return;
			}
		}
		else
		{
			TSharedPtr<ISignalFlowNode>* FoundNode = FilteredEntries.Find(EntryKey);

			if (FoundNode && FoundNode->IsValid())
			{
				if (OutExistingEntryKeys.Contains(EntryKey))
				{
					// First time we have tried to recreate this node but it was here already
					// Remove from tracking container and make a note of the new tree depth
					OutExistingEntryKeys.Remove(EntryKey);
					(*FoundNode)->GetTreeDepth().TreeDepth = TreeDepth;
				}
				else
				{
					// We have created and assigned assigned this node a tree depth - move it lower in the tree
					TreeDepthPair& NodeTreeDepth = (*FoundNode)->GetTreeDepth();
					if (NodeTreeDepth.TreeDepth <= TreeDepth)
					{
						NodeTreeDepth.TreeDepth = TreeDepth + 1;
					}
				}

				Node = (*FoundNode);
			}
			else
			{
				ensure(!OutExistingEntryKeys.Contains(EntryKey));

				Node = MakeShared<FSignalFlowEntryNode>(Entry, TreeDepth, Entry->EntryType, Entry->Timestamp);

				FilteredEntries.Add(EntryKey, Node);
			}

			if (!ensure(Node.IsValid()))
			{
				// We should never hit this, but return just in case to avoid a crash
				return;
			}

			if (ParentEntry.IsValid())
			{
				Node->FilteredInputs.AddUnique(ParentEntry->GetSignalFlowEntryKey());
			}
		}

		for (const auto& [OutputKey, SendInfo] : Entry->Outputs)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = DeviceData.Find(OutputKey);
			if (OutputEntry && OutputEntry->IsValid())
			{
				if (Node.IsValid())
				{
					Node->FilteredOutputs.AddUnique(OutputKey);
				}

				bool bResetTreeDepth = (*OutputEntry)->EntryType != Entry->EntryType;
				AddFilteredOutputsRecursive(DeviceData, *OutputEntry, Entry, bResetTreeDepth ? 0 : TreeDepth + 1, OutExistingEntryKeys, OutPathKeys);
			}
		}

		AddFilteredLinkedSources(DeviceData, Entry, Node.Get(), OutExistingEntryKeys, OutPathKeys);
		AddFilteredLinkedBusPatchOutputs(DeviceData, Entry, Node.Get(), OutExistingEntryKeys, OutPathKeys);
	}

	void FSignalFlowDashboardViewFactory::AddFilteredLinkedSourceBus(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid() || !Entry->LinkedSourceBus.IsSet())
		{
			return;
		}

		const FSignalFlowEntryKey SourceBusEntryKey = Entry->LinkedSourceBus.GetValue();

		if (OutEntryNode != nullptr)
		{
			OutEntryNode->FilteredLinkedSourceBus = SourceBusEntryKey;
		}

		// Check if we need to create a new path inside the graph from the linked source bus entry
		const bool bAlreadyVisited = (OutPathKeys != nullptr) ? OutPathKeys->Contains(SourceBusEntryKey) 
															  : EntryIsInGraph(SourceBusEntryKey, OutExistingEntryKeys);

		if (!bAlreadyVisited)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* SourceBusEntry = DeviceData.Find(SourceBusEntryKey);

			if (SourceBusEntry && SourceBusEntry->IsValid())
			{
				// Detected a new source bus connected to this entry
				// Filter the graph from the found source bus to make sure it's full path is present in the graph
				AddFilteredInputsRecursive(DeviceData, *SourceBusEntry, nullptr, 0, OutExistingEntryKeys, OutPathKeys);

				for (const auto& [OutputKey, SendInfo] : (*SourceBusEntry)->Outputs)
				{
					const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = DeviceData.Find(OutputKey);
					if (OutputEntry && OutputEntry->IsValid())
					{
						AddFilteredOutputsRecursive(DeviceData, *OutputEntry, *SourceBusEntry, 0, OutExistingEntryKeys, OutPathKeys);
					}
				}
			}
		}

		// Make sure the source bus entry is connected to this entry
		if (OutPathKeys == nullptr)
		{
			const TSharedPtr<ISignalFlowNode>* LinkedSourceBusNode = FilteredEntries.Find(SourceBusEntryKey);
			if (LinkedSourceBusNode && LinkedSourceBusNode->IsValid())
			{
				(*LinkedSourceBusNode)->FilteredLinkedSoundSources.AddUnique(Entry->GetSignalFlowEntryKey());
			}
		}
	}

	void FSignalFlowDashboardViewFactory::AddFilteredLinkedSources(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		const FSignalFlowEntryKey EntryKey = Entry->GetSignalFlowEntryKey();

		AddFilteredLinkedEntries(DeviceData,
								 Entry,
								 Entry->LinkedSoundSources,
								 OutEntryNode != nullptr ? &OutEntryNode->FilteredLinkedSoundSources : nullptr,
								 [&EntryKey](ISignalFlowNode& Node) { Node.FilteredLinkedSourceBus = EntryKey; },
								 OutExistingEntryKeys,
								 OutPathKeys);
	}

	// Does not use AddFilteredLinkedEntries because the traversal direction is inverted:
	// we walk inputs-then-outputs (upstream-first) to discover the bus's full graph context,
	// whereas the shared helper walks outputs-then-inputs (downstream-first).
	void FSignalFlowDashboardViewFactory::AddFilteredLinkedBusPatchInputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid() || Entry->LinkedBusPatchInputs.IsEmpty())
		{
			return;
		}

		for (const FSignalFlowEntryKey& BusPatchInputKey : Entry->LinkedBusPatchInputs)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* BusEntry = DeviceData.Find(BusPatchInputKey);
			if (!BusEntry || !BusEntry->IsValid())
			{
				continue;
			}

			if (OutEntryNode != nullptr)
			{
				OutEntryNode->FilteredLinkedBusPatchInputs.AddUnique(BusPatchInputKey);
			}

			const bool bAlreadyVisited = (OutPathKeys != nullptr) ? OutPathKeys->Contains(BusPatchInputKey)
																  : EntryIsInGraph(BusPatchInputKey, OutExistingEntryKeys);

			if (!bAlreadyVisited)
			{
				AddFilteredInputsRecursive(DeviceData, *BusEntry, nullptr, 0, OutExistingEntryKeys, OutPathKeys);

				for (const auto& [OutputKey, SendInfo] : (*BusEntry)->Outputs)
				{
					const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = DeviceData.Find(OutputKey);
					if (OutputEntry && OutputEntry->IsValid())
					{
						AddFilteredOutputsRecursive(DeviceData, *OutputEntry, *BusEntry, 0, OutExistingEntryKeys, OutPathKeys);
					}
				}
			}

			if (OutPathKeys == nullptr)
			{
				const TSharedPtr<ISignalFlowNode>* LinkedBusNode = FilteredEntries.Find(BusPatchInputKey);
				if (LinkedBusNode && LinkedBusNode->IsValid())
				{
					(*LinkedBusNode)->FilteredLinkedBusPatchOutputs.AddUnique(Entry->GetSignalFlowEntryKey());
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::AddFilteredLinkedBusPatchOutputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		const FSignalFlowEntryKey EntryKey = Entry->GetSignalFlowEntryKey();

		AddFilteredLinkedEntries(DeviceData,
								 Entry,
								 Entry->LinkedBusPatchOutputs,
								 OutEntryNode != nullptr ? &OutEntryNode->FilteredLinkedBusPatchOutputs : nullptr,
								 [&EntryKey](ISignalFlowNode& Node) { Node.FilteredLinkedBusPatchInputs.AddUnique(EntryKey); },
								 OutExistingEntryKeys,
								 OutPathKeys);
	}

	void FSignalFlowDashboardViewFactory::AddFilteredLinkedEntries(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, const TSet<FSignalFlowEntryKey>& LinkedEntryKeys, TArray<FSignalFlowEntryKey>* OutFilteredLinkedEntries, TFunctionRef<void(ISignalFlowNode&)> LinkBackFunction, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys)
	{
		if (!Entry.IsValid() || LinkedEntryKeys.IsEmpty())
		{
			return;
		}

		for (const FSignalFlowEntryKey& LinkedSourceEntryKey : LinkedEntryKeys)
		{
			const TSharedPtr<FSignalFlowDashboardEntry>* LinkedSourceEntry = DeviceData.Find(LinkedSourceEntryKey);
			if (LinkedSourceEntry && LinkedSourceEntry->IsValid())
			{
				if (OutFilteredLinkedEntries != nullptr)
				{
					OutFilteredLinkedEntries->AddUnique(LinkedSourceEntryKey);
				}

				const bool bAlreadyVisited = (OutPathKeys != nullptr) ? OutPathKeys->Contains(LinkedSourceEntryKey)
																	  : EntryIsInGraph(LinkedSourceEntryKey, OutExistingEntryKeys);

				if (!bAlreadyVisited)
				{
					// Detected a new source connected to this entry
					// Filter the graph from the found source to make sure it's full path is present in the graph
					AddFilteredOutputsRecursive(DeviceData, *LinkedSourceEntry, nullptr, 0, OutExistingEntryKeys, OutPathKeys);

					for (const FSignalFlowEntryKey& InputKey : (*LinkedSourceEntry)->Inputs)
					{
						const TSharedPtr<FSignalFlowDashboardEntry>* InputEntry = DeviceData.Find(InputKey);
						if (InputEntry && InputEntry->IsValid())
						{
							AddFilteredInputsRecursive(DeviceData, *InputEntry, *LinkedSourceEntry, 0, OutExistingEntryKeys, OutPathKeys);
						}
					}
				}

				// Make sure the linked source is connected to this entry
				if (OutPathKeys == nullptr)
				{
					const TSharedPtr<ISignalFlowNode>* LinkedSourceNode = FilteredEntries.Find(LinkedSourceEntryKey);
					if (LinkedSourceNode && LinkedSourceNode->IsValid())
					{
						LinkBackFunction(**LinkedSourceNode);
					}
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::CleanUpFilteredEntries(const TSet<FSignalFlowEntryKey>& EntriesToRemove)
	{
		if (EntriesToRemove.IsEmpty())
		{
			return;
		}

		for (const auto& [EntryKey, Entry] : FilteredEntries)
		{
			if (EntriesToRemove.Contains(EntryKey))
			{
				continue;
			}

			Entry->FilteredInputs.RemoveAll([&EntriesToRemove](const FSignalFlowEntryKey& Input) { return EntriesToRemove.Contains(Input); });
			Entry->FilteredOutputs.RemoveAll([&EntriesToRemove](const FSignalFlowEntryKey& Output) { return EntriesToRemove.Contains(Output); });

			if (Entry->FilteredLinkedSourceBus.IsSet() && EntriesToRemove.Contains(Entry->FilteredLinkedSourceBus.GetValue()))
			{
				Entry->FilteredLinkedSourceBus.Reset();
			}

			Entry->FilteredLinkedSoundSources.RemoveAll([&EntriesToRemove](const FSignalFlowEntryKey& LinkedSource) { return EntriesToRemove.Contains(LinkedSource); });
			Entry->FilteredLinkedBusPatchInputs.RemoveAll([&EntriesToRemove](const FSignalFlowEntryKey& Key) { return EntriesToRemove.Contains(Key); });
			Entry->FilteredLinkedBusPatchOutputs.RemoveAll([&EntriesToRemove](const FSignalFlowEntryKey& Key) { return EntriesToRemove.Contains(Key); });
		}

		for (const FSignalFlowEntryKey& EntryKey : EntriesToRemove)
		{
			FilteredEntries.Remove(EntryKey);
		}
	}

	void FSignalFlowDashboardViewFactory::AddMissingNodesToGraph(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete)
	{
		// If a node has no inputs/outputs, it can get missed by the recursive node graph creation
		// Run through all device data entries and add any missing entries to the graph
		// Note: If a node has both no inputs and no outputs (like an unconnected audio bus), we ignore it entirely
		for (const auto& [EntryKey, Entry] : DeviceData)
		{
			if (!Entry.IsValid())
			{
				continue;
			}

			// Ignore entries that are not connected to anything
			if (Entry->Inputs.IsEmpty() && Entry->Outputs.IsEmpty())
			{
				continue;
			}

			// Ignore entries that have already been entered into the graph
			if (EntryIsInGraph(EntryKey, OutEntriesMarkedToDelete))
			{
				continue;
			}

			if (Entry->Inputs.IsEmpty())
			{
				AddMissingNodeByOutputs(*Entry, DeviceData, OutEntriesMarkedToDelete);
			}
			else if (Entry->Outputs.IsEmpty())
			{
				AddMissingNodeByInputs(*Entry, DeviceData, OutEntriesMarkedToDelete);
			}
		}
	}

	void FSignalFlowDashboardViewFactory::AddMissingNodeByOutputs(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete)
	{
		// Gather the closest output nodes to this entry that are already in the graph
		// This may require running down the branch recursively until we find the next output in the graph
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> NextOutputs;
		GetNextOutputsInGraphRecursive(Entry, DeviceData, OutEntriesMarkedToDelete, NextOutputs);

		// Run through all closest added outputs and add their inputs recursively, creating a connection path to the missing node
		for (const TSharedPtr<FSignalFlowDashboardEntry>& NextOutputEntryInGraph : NextOutputs)
		{
			if (!NextOutputEntryInGraph.IsValid())
			{
				continue;
			}

			const FSignalFlowEntryKey OutputKey = NextOutputEntryInGraph->GetSignalFlowEntryKey();
			const TSharedPtr<ISignalFlowNode>* NextOutputEntryInGraphNode = FilteredEntries.Find(OutputKey);
			if (NextOutputEntryInGraphNode == nullptr || !NextOutputEntryInGraphNode->IsValid())
			{
				continue;
			}

			const int32 NextOutputEntryInGraphTreeDepth = (*NextOutputEntryInGraphNode)->GetTreeDepth().TreeDepth;
			for (const FSignalFlowEntryKey& InputKey : NextOutputEntryInGraph->Inputs)
			{
				const TSharedPtr<FSignalFlowDashboardEntry>* InputEntry = DeviceData.Find(InputKey);
				if (InputEntry == nullptr || !InputEntry->IsValid())
				{
					continue;
				}

				(*NextOutputEntryInGraphNode)->FilteredInputs.AddUnique(InputKey);

				const TSharedPtr<ISignalFlowNode>* InputNode = FilteredEntries.Find(InputKey);
				if (InputNode == nullptr || OutEntriesMarkedToDelete.Contains(InputKey))
				{
					bool bResetTreeDepth = (*InputEntry)->EntryType != NextOutputEntryInGraph->EntryType;
					AddFilteredInputsRecursive(DeviceData, *InputEntry, NextOutputEntryInGraph, bResetTreeDepth ? 0 : NextOutputEntryInGraphTreeDepth - 1, OutEntriesMarkedToDelete);
				}
				else if (InputNode && InputNode->IsValid())
				{
					(*InputNode)->FilteredOutputs.Add(OutputKey);
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::AddMissingNodeByInputs(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete)
	{
		// Gather the closest input nodes to this entry that are already in the graph
		// This may require running up the branch recursively until we find the next input in the graph
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> NextInputs;
		GetNextInputsInGraphRecursive(Entry, DeviceData, OutEntriesMarkedToDelete, NextInputs);

		// Run through all closest added inputs and add their outputs recursively, creating a connection path to the missing node
		for (const TSharedPtr<FSignalFlowDashboardEntry>& NextInputEntryInGraph : NextInputs)
		{
			if (!NextInputEntryInGraph.IsValid())
			{
				continue;
			}

			const FSignalFlowEntryKey InputKey = NextInputEntryInGraph->GetSignalFlowEntryKey();
			const TSharedPtr<ISignalFlowNode>* NextInputEntryInGraphNode = FilteredEntries.Find(InputKey);
			if (NextInputEntryInGraphNode == nullptr || !NextInputEntryInGraphNode->IsValid())
			{
				continue;
			}

			const int32 NextInputEntryInGraphTreeDepth = (*NextInputEntryInGraphNode)->GetTreeDepth().TreeDepth;
			for (const auto& [OutputKey, OutputInfo] : NextInputEntryInGraph->Outputs)
			{
				const TSharedPtr<FSignalFlowDashboardEntry>* OutputEntry = DeviceData.Find(OutputKey);
				if (OutputEntry && OutputEntry->IsValid())
				{
					(*NextInputEntryInGraphNode)->FilteredOutputs.AddUnique(OutputKey);

					const TSharedPtr<ISignalFlowNode>* OutputNode = FilteredEntries.Find(OutputKey);
					if ((OutputNode == nullptr || OutEntriesMarkedToDelete.Contains(OutputKey)))
					{
						bool bResetTreeDepth = (*OutputEntry)->EntryType != NextInputEntryInGraph->EntryType;
						AddFilteredOutputsRecursive(DeviceData, *OutputEntry, NextInputEntryInGraph, bResetTreeDepth ? 0 : NextInputEntryInGraphTreeDepth + 1, OutEntriesMarkedToDelete);
					}
					else if (OutputNode && OutputNode->IsValid())
					{
						(*OutputNode)->FilteredInputs.AddUnique(InputKey);
					}
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::GetNextInputsInGraphRecursive(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete, TArray<TSharedPtr<FSignalFlowDashboardEntry>>& OutNextInputs) const
	{
		// Iterate over this node's inputs - follow the branches until we reach the nearest nodes that are present in the current graph
		// Gather all of these together and add to OutNextInputs
		if (!Entry.IsValid())
		{
			return;
		}

		for (const FSignalFlowEntryKey& InputKey : Entry.Inputs)
		{
			if (const TSharedPtr<FSignalFlowDashboardEntry>* DeviceDataEntry = DeviceData.Find(InputKey))
			{
				if (!DeviceDataEntry->IsValid())
				{
					continue;
				}

				if (EntryIsInGraph(InputKey, EntriesMarkedToDelete))
				{
					OutNextInputs.Add(*DeviceDataEntry);
				}
				else
				{
					// This node is also not currently in the graph - we need to check it's inputs
					GetNextInputsInGraphRecursive(**DeviceDataEntry, DeviceData, EntriesMarkedToDelete, OutNextInputs);
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::GetNextOutputsInGraphRecursive(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete, TArray<TSharedPtr<FSignalFlowDashboardEntry>>& OutNextOutputs) const
	{
		// Iterate over this node's outputs - follow the branches until we reach the nearest nodes that are present in the current graph
		// Gather all of these together and add to OutNextOutputs
		for (const auto& [OutputKey, OutputData] : Entry.Outputs)
		{
			if (const TSharedPtr<FSignalFlowDashboardEntry>* DeviceDataEntry = DeviceData.Find(OutputKey))
			{
				if (!DeviceDataEntry->IsValid())
				{
					continue;
				}

				if (EntryIsInGraph(OutputKey, EntriesMarkedToDelete))
				{
					OutNextOutputs.Add(*DeviceDataEntry);
				}
				else
				{
					// This node is also not currently in the graph - we need to check it's outputs
					GetNextOutputsInGraphRecursive(**DeviceDataEntry, DeviceData, EntriesMarkedToDelete, OutNextOutputs);
				}
			}
		}
	}

	bool FSignalFlowDashboardViewFactory::EntryIsInGraph(const FSignalFlowEntryKey& EntryKey, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete) const
	{
		// An entry is considered in the graph if a corresponding node exists inside FilteredEntries and it hasn't been marked for deletion
		return FilteredEntries.Contains(EntryKey) && !EntriesMarkedToDelete.Contains(EntryKey);
	}

	void FSignalFlowDashboardViewFactory::CalculateFilteredGraphDepthStructure(TOptional<FSignalFlowEntryKey>& OutRootEntryKey, TArray<TreeDepthPair>& OutSortedTreeDepths) const
	{
		OutSortedTreeDepths.Reset();

		TreeDepthPair HighestTreeDepth = { ESignalFlowEntryType::InvalidLow, TNumericLimits<int32>::Lowest() };

		for (auto& [Key, Node] : FilteredEntries)
		{
			if (!Node.IsValid())
			{
				continue;
			}

			Node->ResetOrderID();

			const TreeDepthPair& NodeTreeDepth = Node->GetTreeDepth();
			if (!OutSortedTreeDepths.Contains(NodeTreeDepth))
			{
				OutSortedTreeDepths.Add(NodeTreeDepth);
			}

			if (HighestTreeDepth < NodeTreeDepth)
			{
				OutRootEntryKey = Key;
				HighestTreeDepth = NodeTreeDepth;
			}
		}

		auto SortByTreeDepth = [this](const TreeDepthPair& First, const TreeDepthPair& Second)
		{
			return First < Second;
		};

		if (!OutSortedTreeDepths.IsEmpty())
		{
			OutSortedTreeDepths.Sort(SortByTreeDepth);
		}
	}

	void FSignalFlowDashboardViewFactory::CreateDummyInputs(const TArray<TreeDepthPair>& TreeDepthStructure)
	{
		TArray<TSharedPtr<ISignalFlowNode>> Nodes;
		FilteredEntries.GenerateValueArray(Nodes);

		for (const TSharedPtr<ISignalFlowNode>& Node : Nodes)
		{
			const TreeDepthPair& CurrentTreeDepthPair = Node->GetTreeDepth();

			for (const FSignalFlowEntryKey& InputKey : Node->FilteredInputs)
			{
				const TSharedPtr<ISignalFlowNode>* FoundInput = FilteredEntries.Find(InputKey);
				if (FoundInput && FoundInput->IsValid())
				{
					ensure((*FoundInput)->IsRealNode());

					const TSharedPtr<FSignalFlowEntryNode> FoundInputSharedPtr = StaticCastSharedPtr<FSignalFlowEntryNode>(*FoundInput);
					if (!FoundInputSharedPtr.IsValid())
					{
						continue;
					}

					const TreeDepthPair& TargetTreeDepthPair = (*FoundInput)->GetTreeDepth();

					const int32 TargetTreeDepthIndex = TreeDepthStructure.IndexOfByKey(TargetTreeDepthPair);
					int32 CurrentTreeDepthIndex = TreeDepthStructure.IndexOfByKey(CurrentTreeDepthPair) - 1;

					ensure(CurrentTreeDepthIndex >= TargetTreeDepthIndex);

					const FSignalFlowEntryKey ConnectionOutputKey = Node->GetEntryKey();
					TSharedPtr<ISignalFlowNode> OutputNode = Node;

					// Add invisible "dummy connection" nodes between the input and output for every row between them
					while (CurrentTreeDepthIndex > TargetTreeDepthIndex)
					{
						OutputNode = CreateDummyConnectionNode(FoundInputSharedPtr, OutputNode, TreeDepthStructure[CurrentTreeDepthIndex], ConnectionOutputKey);

						CurrentTreeDepthIndex--;
					}
				}
			}
		}
	}

	TSharedPtr<FDummyConnectionNode> FSignalFlowDashboardViewFactory::CreateDummyConnectionNode(const TSharedPtr<FSignalFlowEntryNode>& InputNode, const TSharedPtr<ISignalFlowNode>& OutputNode, const TreeDepthPair& TreeDepthPair, const FSignalFlowEntryKey& ConnectionOutputNodeKey)
	{
		// Input nodes should only be real nodes in the graph, and should have valid entries
		// Output nodes can be real, or be dummy connections
		if (!InputNode.IsValid() || !InputNode->Entry.IsValid() || !OutputNode.IsValid())
		{
			return nullptr;
		}

		const FSignalFlowEntryKey InputKey = InputNode->GetEntryKey();
		const FSignalFlowEntryKey OutputKey = OutputNode->GetEntryKey();

		// It should be guarenteed for the OutputNode to contain the InputKey in it's filtered inputs
		if (!ensure(OutputNode->FilteredInputs.Contains(InputKey)))
		{
			return nullptr;
		}

		// Create a dummy node in between the input and output nodes
		// We can use these dummy connection nodes to help order and position real nodes in the graph and reduce connections crossing
		const FSignalFlowEntryKey DummyConnectionEntryKey = FSignalFlowEntryKey(InputKey.DeviceID, TreeDepthPair.LayerID, HashCombine(GetTypeHash(InputKey), GetTypeHash(OutputKey), TreeDepthPair.TreeDepth));
		TSharedPtr<FDummyConnectionNode> DummyConnectionNode = MakeShared<FDummyConnectionNode>(DummyConnectionEntryKey, TreeDepthPair.TreeDepth, TreeDepthPair.LayerID, InputNode->Entry->Timestamp);
		
		DummyConnectionNode->ConnectionOutputKey = ConnectionOutputNodeKey;
		DummyConnectionNode->ConnectionInputKey = InputKey;

		// Link the dummy node into the graph
		DummyConnectionNode->FilteredInputs.AddUnique(InputKey);
		DummyConnectionNode->FilteredOutputs.AddUnique(OutputKey);

		// Unlink the original nodes and point them towards the dummy
		InputNode->FilteredOutputs.Remove(OutputKey);
		InputNode->FilteredOutputs.AddUnique(DummyConnectionEntryKey);
		OutputNode->FilteredInputs[OutputNode->FilteredInputs.IndexOfByKey(InputKey)] = DummyConnectionEntryKey;

		// Add the new dummy node to the filtered entries map
		FilteredEntries.Add(DummyConnectionEntryKey, DummyConnectionNode);

		return DummyConnectionNode;
	}

	void FSignalFlowDashboardViewFactory::AssignNodeOrderRecursive(TSharedPtr<ISignalFlowNode> OutNode, const int32 Depth, int32& OutNodeProcessingOrder)
	{
		// Keep track of the order nodes were reached when iterating through tree inputs recursively
		// This helps keep nodes that are linked to the same input together in the graph
		// We sort the inputs before proceeding forward to keep a good order in the graph
		
		// Do not reprocess nodes that have already been processed
		if (OutNode->NodeOrderIsValid())
		{
			return;
		}

		OutNode->SetNodeOrderID(OutNodeProcessingOrder);
		OutNodeProcessingOrder++;

		SortNodeInputs(OutNode);

		for (const FSignalFlowEntryKey InputKey : OutNode->FilteredInputs)
		{
			const TSharedPtr<ISignalFlowNode>* FoundInput = FilteredEntries.Find(InputKey);
		
			if (FoundInput && FoundInput->IsValid())
			{
				AssignNodeOrderRecursive(*FoundInput, Depth + 1, OutNodeProcessingOrder);
			}
		}
	}

	void FSignalFlowDashboardViewFactory::SortNodeInputs(TSharedPtr<ISignalFlowNode> Node)
	{
		if (!Node.IsValid())
		{
			return;
		}

		auto SortInput = [this](const FSignalFlowEntryKey& First, const FSignalFlowEntryKey& Second)
		{
			const TSharedPtr<ISignalFlowNode>* FirstNode = FilteredEntries.Find(First);
			const TSharedPtr<ISignalFlowNode>* SecondNode = FilteredEntries.Find(Second);

			if (FirstNode && SecondNode && FirstNode->IsValid() && SecondNode->IsValid())
			{
				const TreeDepthPair FirstNodeTreeDepth = GetRealTreeDepthForNode((*FirstNode));
				const TreeDepthPair SecondNodeTreeDepth = GetRealTreeDepthForNode((*SecondNode));

				// First, sort by tree depth
				if (FirstNodeTreeDepth != SecondNodeTreeDepth)
				{
					return FirstNodeTreeDepth > SecondNodeTreeDepth;
				}

				// Next, try to order nodes by the minimum timestamp of their parent sound source/owner object nodes (if any exist)
				double FirstBranchMinTimestamp = TNumericLimits<double>::Max();
				TreeDepthPair FirstBranchMinTreeDepth = FirstNodeTreeDepth;
				GetBranchMinTimestampAndTreeDepthRecursive((*FirstNode)->FilteredInputs, FirstBranchMinTreeDepth, FirstBranchMinTimestamp);

				double SecondInputTimestamp = TNumericLimits<double>::Max();
				TreeDepthPair SecondBranchMinTreeDepth = SecondNodeTreeDepth;
				GetBranchMinTimestampAndTreeDepthRecursive((*SecondNode)->FilteredInputs, SecondBranchMinTreeDepth, SecondInputTimestamp);

				// Move branches that are not connected all the way up to the lowest tree depth to one side of the graph
				if (FirstBranchMinTreeDepth != SecondBranchMinTreeDepth)
				{
					return FirstBranchMinTreeDepth > SecondBranchMinTreeDepth;
				}

				// Sort by timestamp of the parent nodes
				if (!FMath::IsNearlyEqual(FirstBranchMinTimestamp, SecondInputTimestamp))
				{
					return FirstBranchMinTimestamp > SecondInputTimestamp;
				}

				// Try timestamp of this node
				if (NodeIsOwnerOrSound(**FirstNode) && NodeIsOwnerOrSound(**SecondNode))
				{
					if (!FMath::IsNearlyEqual((*FirstNode)->GetTimestamp(), (*SecondNode)->GetTimestamp()))
					{
						return (*FirstNode)->GetTimestamp() > (*SecondNode)->GetTimestamp();
					}
				}

				if ((*FirstNode)->GetPreviousNodeOrderID() != (*SecondNode)->GetPreviousNodeOrderID())
				{
					return (*FirstNode)->GetPreviousNodeOrderID() < (*SecondNode)->GetPreviousNodeOrderID();
				}
			}

			// Fallback on entry IDs
			return First.EntryID < Second.EntryID;
		};

		Node->FilteredInputs.Sort(SortInput);
	}

	TreeDepthPair FSignalFlowDashboardViewFactory::GetRealTreeDepthForNode(const TSharedPtr<ISignalFlowNode>& Node) const
	{
		// Either grab the tree depth for this node,
		// or if it's a dummy connection node, grab the tree depth of the original input to the dummy connection
		ensure(Node.IsValid());

		if (!Node->IsRealNode())
		{
			const TSharedPtr<FDummyConnectionNode> DummyNode = StaticCastSharedPtr<FDummyConnectionNode>(Node);
			const TSharedPtr<ISignalFlowNode>* DummyInputNode = FilteredEntries.Find(DummyNode->ConnectionInputKey);
			if (DummyInputNode && DummyInputNode->IsValid())
			{
				return (*DummyInputNode)->GetTreeDepth();
			}
		}

		return Node->GetTreeDepth();
	}

	void FSignalFlowDashboardViewFactory::GetBranchMinTimestampAndTreeDepthRecursive(const TArray<FSignalFlowEntryKey>& NodeKeys, TreeDepthPair& OutMinTreeDepth, double& OutMinTimestamp) const
	{
		for (const FSignalFlowEntryKey& NodeKey : NodeKeys)
		{
			const TSharedPtr<ISignalFlowNode>* Node = FilteredEntries.Find(NodeKey);

			if (Node && Node->IsValid())
			{
				OutMinTreeDepth = FMath::Min(OutMinTreeDepth, (*Node)->GetTreeDepth());

				// Only count entries that are transient when finding the minimum timestamp of a branch
				if (NodeIsOwnerOrSound(**Node))
				{
					OutMinTimestamp = FMath::Min((*Node)->GetTimestamp(), OutMinTimestamp);
				}
				
				GetBranchMinTimestampAndTreeDepthRecursive((*Node)->FilteredInputs, OutMinTreeDepth, OutMinTimestamp);
			}
		}
	}

	bool FSignalFlowDashboardViewFactory::NodeIsOwnerOrSound(const ISignalFlowNode& Node) const
	{
		return Node.GetEntryKey().EntryType == ESignalFlowEntryType::SoundSource || Node.GetEntryKey().EntryType == ESignalFlowEntryType::OwnerObject;
	}

	void FSignalFlowDashboardViewFactory::FixUnassignedNodeOrders()
	{
		// Run through again and find any nodes that did not get an order assigned
		// This can happen if a node is not the root node but has no outputs
		for (auto& [Key, Node] : FilteredEntries)
		{
			if (Node.IsValid() && !Node->NodeOrderIsValid())
			{
				// Force nodes with no outputs to one side of the graph
				Node->SetNodeOrderID(TNumericLimits<int32>::Lowest());
			}
		}
	}

	void FSignalFlowDashboardViewFactory::SortNodeOrder()
	{
		auto SortSignalNodes = [this](const TSharedPtr<ISignalFlowNode>& First, const TSharedPtr<ISignalFlowNode>& Second)
		{
			if (!First.IsValid() || !Second.IsValid())
			{
				return false;
			}

			// Sort by the depth in the signal node tree first
			if (First->GetTreeDepth() != Second->GetTreeDepth())
			{
				return First->GetTreeDepth() < Second->GetTreeDepth();
			}

			// Rows are sorted by the order a node was processed - this groups nodes that output to the same input together where possible
			// Inputs were sorted inside previous step FSignalFlowDashboardViewFactory::CreateFilteredNodeGraph
			if (First->GetNodeOrderID() != Second->GetNodeOrderID())
			{
				return First->GetNodeOrderID() < Second->GetNodeOrderID();
			}

			// Fallback to timestamp
			return First->GetTimestamp() > Second->GetTimestamp();
		};

		FilteredGraphNodes.Sort(SortSignalNodes);
	}

	void FSignalFlowDashboardViewFactory::CalcXPositions(const TSharedPtr<ISignalFlowNode>& GraphRootNode)
	{
		if (!GraphWidgetContents.IsValid())
		{
			return;
		}

		const EOrientation GraphOrientation = GraphWidgetContents->GetGraphOrientation();
		int32 GraphWidth = 0;

		TreeDepthPair CurrentTreeDepthPair{ ESignalFlowEntryType::InvalidMax, TNumericLimits<int32>::Max() };

		// Keep track of node indexes that we have processed
		// These are tracked as rows of indexes
		TArray<TArray<int32>> ProcessedNodeRows;
		ProcessedNodeRows.Reserve(FilteredGraphNodes.Num());

		int32 RowIndex = INDEX_NONE;

		// At this point, the nodes should be sorted from lowest to highest tree depth
		// We can iterate backwards (from audio device to source) to calc the positions
		for (int32 RevIndex = FilteredGraphNodes.Num() - 1; RevIndex >= 0; --RevIndex)
		{
			TSharedPtr<ISignalFlowNode> NextNodeToProcess = FilteredGraphNodes[RevIndex];
			if (!NextNodeToProcess.IsValid())
			{
				continue;
			}

			// If we have moved on to a new row, calculate the positions of the current structure
			if (CurrentTreeDepthPair != NextNodeToProcess->GetTreeDepth())
			{
				CalcXPositionsForRows(ProcessedNodeRows, GraphOrientation, GraphWidth);

				// Create the next row
				RowIndex = RowIndex == INDEX_NONE ? 0 : RowIndex + 1;

				TArray<int32> NextRow;
				NextRow.Reserve(FilteredGraphNodes.Num());
				ProcessedNodeRows.Add(NextRow);

				CurrentTreeDepthPair = NextNodeToProcess->GetTreeDepth();
			}

			ensure(ProcessedNodeRows.IsValidIndex(RowIndex));
			ProcessedNodeRows[RowIndex].Add(RevIndex);
		}

		// Do one final positioning process for the final row
		CalcXPositionsForRows(ProcessedNodeRows, GraphOrientation, GraphWidth);

		// Compact gaps between real nodes that were inflated by dummy node spacing
		CompactRealNodeXPositions();

		// Now shift the whole graph so the root focused node remains at X position 0
		MaintainFocusedNodeXPos(GraphRootNode);
		
	}

	void FSignalFlowDashboardViewFactory::CalcXPositionsForRows(const TArray<TArray<int32>>& NodeIndexRows, const EOrientation GraphOrientation, int32& OutGraphWidth)
	{
		if (NodeIndexRows.IsEmpty())
		{
			return;
		}

		const int32 RowToProcessIndex = NodeIndexRows.Num() - 1;

		const TArray<int32>& RowToProcess = NodeIndexRows[RowToProcessIndex];
		const int32 RowToProcessNumNodes = RowToProcess.Num();

		if (OutGraphWidth <= RowToProcessNumNodes)
		{
			OutGraphWidth = RowToProcessNumNodes;

			// new largest row, position the nodes with an x position of 1.0 apart
			PositionRowEquidistant(RowToProcess);

			// Propagate change down the hierarchy - moving nodes relative to their inputs
			for (int32 PreviousRowsIndex = RowToProcessIndex - 1; PreviousRowsIndex >= 0; --PreviousRowsIndex)
			{
				const TArray<int32> PrevRowOfIndexes = NodeIndexRows[PreviousRowsIndex];

				TArray<TSharedPtr<ISignalFlowNode>> PrevPositionedNodesInRow;
				for (int32 NodeIndexInRow = 0; NodeIndexInRow < PrevRowOfIndexes.Num(); ++NodeIndexInRow)
				{
					const TSharedPtr<ISignalFlowNode>& Node = FilteredGraphNodes[PrevRowOfIndexes[NodeIndexInRow]];

					if (Node.IsValid())
					{
						CalcXPositionForNode(Node, Node->FilteredInputs, NodeIndexInRow, PrevPositionedNodesInRow);
						PrevPositionedNodesInRow.Add(Node);
					}
				}
			}
		}
		else
		{
			// Process just this row - moving nodes relative to their outputs
			TArray<TSharedPtr<ISignalFlowNode>> PrevPositionedNodesInRow;
			for (int32 NodeIndexInRow = 0; NodeIndexInRow < RowToProcess.Num(); ++NodeIndexInRow)
			{
				TSharedPtr<ISignalFlowNode>& Node = FilteredGraphNodes[RowToProcess[NodeIndexInRow]];
				if (Node.IsValid())
				{
					CalcXPositionForNode(Node, Node->FilteredOutputs, NodeIndexInRow, PrevPositionedNodesInRow);
					PrevPositionedNodesInRow.Add(Node);
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::CompactRealNodeXPositions()
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;

		// Phase 1: Collect all unique real-node X positions, sorted ascending.
		TArray<float> RealXPositions;
		RealXPositions.Reserve(FilteredGraphNodes.Num());

		for (const TSharedPtr<ISignalFlowNode>& Node : FilteredGraphNodes)
		{
			if (Node.IsValid() && Node->IsRealNode())
			{
				RealXPositions.Add(Node->XPos);
			}
		}

		RealXPositions.Sort();

		// Remove duplicates (multiple real nodes on different rows can share the same X).
		if (RealXPositions.Num() > 1)
		{
			int32 WriteIndex = 0;
			for (int32 ReadIndex = 1; ReadIndex < RealXPositions.Num(); ++ReadIndex)
			{
				if (RealXPositions[ReadIndex] != RealXPositions[WriteIndex])
				{
					++WriteIndex;
					RealXPositions[WriteIndex] = RealXPositions[ReadIndex];
				}
			}
			RealXPositions.SetNum(WriteIndex + 1);
		}

		// Nothing to compact with fewer than 2 unique positions.
		if (RealXPositions.Num() < 2)
		{
			return;
		}

		// Phase 2: Build compacted position array, capping consecutive gaps to MaxRealNodeGap.
		const int32 NumPositions = RealXPositions.Num();
		TArray<float> CompactedPositions;
		CompactedPositions.SetNum(NumPositions);
		CompactedPositions[0] = RealXPositions[0];

		bool bAnyCompaction = false;

		for (int32 Index = 1; Index < NumPositions; ++Index)
		{
			const float OriginalGap = RealXPositions[Index] - RealXPositions[Index - 1];
			const float ClampedGap = FMath::Min(OriginalGap, MaxRealNodeGap);
			CompactedPositions[Index] = CompactedPositions[Index - 1] + ClampedGap;

			if (OriginalGap > MaxRealNodeGap)
			{
				bAnyCompaction = true;
			}
		}

		// All gaps already within threshold, nothing to do.
		if (!bAnyCompaction)
		{
			return;
		}

		// Phase 3: Remap real nodes only. Dummy nodes are left untouched
		// as their XPos is not used after CalcXPositions returns.
		for (const TSharedPtr<ISignalFlowNode>& Node : FilteredGraphNodes)
		{
			if (!Node.IsValid() || !Node->IsRealNode())
			{
				continue;
			}

			const int32 FoundIndex = Algo::LowerBound(RealXPositions, Node->XPos);
			if (RealXPositions.IsValidIndex(FoundIndex))
			{
				Node->XPos = CompactedPositions[FoundIndex];
			}
		}
	}

	void FSignalFlowDashboardViewFactory::MaintainFocusedNodeXPos(const TSharedPtr<ISignalFlowNode>& FallbackNode)
	{
		TOptional<float> XShift;

		if (XPosFocusNodeEntryKey.IsSet())
		{
			const FSignalFlowEntryKey EntryKey = XPosFocusNodeEntryKey.GetValue();
			TSharedPtr<ISignalFlowNode>* FilterFromNode = FilteredEntries.Find(EntryKey);

			if (FilterFromNode && FilterFromNode->IsValid())
			{
				XShift = (*FilterFromNode)->XPos;
			}
		}

		if (!XShift.IsSet() && FallbackNode.IsValid())
		{
			XShift = FallbackNode->XPos;
		}

		if (!XShift.IsSet())
		{
			return;
		}

		const float XShiftValue = XShift.GetValue();
		if (!FMath::IsNearlyZero(XShiftValue))
		{
			for (const TSharedPtr<ISignalFlowNode>& FilteredNode : FilteredGraphNodes)
			{
				if (FilteredNode.IsValid())
				{
					FilteredNode->XPos -= XShiftValue;
				}
			}
		}
	}

	void FSignalFlowDashboardViewFactory::PositionRowEquidistant(const TArray<int32>& NodeIndexRow)
	{
		// Space a row of nodes with a X distance of 1.0
		const int32 NumNodes = NodeIndexRow.Num();

		for (int32 NodeIndexInRow = 0; NodeIndexInRow < NodeIndexRow.Num(); ++NodeIndexInRow)
		{
			const TSharedPtr<ISignalFlowNode>& Node = FilteredGraphNodes[NodeIndexRow[NodeIndexInRow]];
			if (!Node.IsValid())
			{
				continue;
			}

			Node->XPos = NodeIndexInRow;
		}
	}

	void FSignalFlowDashboardViewFactory::CalcXPositionForNode(const TSharedPtr<ISignalFlowNode>& Node, const TArray<FSignalFlowEntryKey>& NodesToPositionRelativeTo, const int32 NodeIndexInRow, const TArray<TSharedPtr<ISignalFlowNode>>& PrevPositionedNodesInRow)
	{
		switch (GraphJustification)
		{
		case ESignalFlowJustification::Edge:
			AlignNodeTowardsEdge(Node, NodeIndexInRow);
			break;

		case ESignalFlowJustification::Center:
			AlignNodeTowardsCenter(Node, NodesToPositionRelativeTo, PrevPositionedNodesInRow);
			break;
		}
	}

	void FSignalFlowDashboardViewFactory::AlignNodeTowardsEdge(const TSharedPtr<ISignalFlowNode>& Node, const int32 NodeIndexInRow)
	{
		if (Node.IsValid())
		{
			Node->XPos = NodeIndexInRow;
		}
	}

	void FSignalFlowDashboardViewFactory::AlignNodeTowardsCenter(const TSharedPtr<ISignalFlowNode>& Node, const TArray<FSignalFlowEntryKey>& NodesToPositionRelativeTo, const TArray<TSharedPtr<ISignalFlowNode>>& PrevPositionedNodesInRow)
	{
		if (!Node.IsValid())
		{
			return;
		}

		if (NodesToPositionRelativeTo.IsEmpty())
		{
			// If no nodes are present to position this node relative to, it may be that this node has no inputs/outputs
			// In this case, position this just next to the previously placed node
			if (PrevPositionedNodesInRow.IsEmpty() || !PrevPositionedNodesInRow.Last().IsValid())
			{
				Node->XPos = 0.0f;
			}
			else
			{
				Node->XPos = PrevPositionedNodesInRow.Last()->XPos + 1.0f;
			}
		}
		else
		{
			// Position this node in the average position of the nodes we are positioning relative to
			float TotalXPositions = 0.0f;
			for (int32 AligningNodeIndex = 0; AligningNodeIndex < NodesToPositionRelativeTo.Num(); ++AligningNodeIndex)
			{
				TSharedPtr<ISignalFlowNode>* AliningNode = FilteredEntries.Find(NodesToPositionRelativeTo[AligningNodeIndex]);
				if (AliningNode && AliningNode->IsValid())
				{
					TotalXPositions += (*AliningNode)->XPos;
				}
			}

			// We use Floor(Av X 2) / 2 to snap the node's position on a grid with spacing 0.5 between cells
			const float AverageXPosition = FMath::Floor((TotalXPositions / NodesToPositionRelativeTo.Num()) * 2.0f) * 0.5f;
			Node->XPos = AverageXPosition;

			// Now ensure any nodes placed previously are shifted if they overlap the new node.
			// We want to maintain the order that nodes are drawn in the graph
			// Run backwards, ensuring at least a distance of 1.0 between nodes, until we don't detect any further overlaps
			float ShiftXPosition = AverageXPosition - 1.0f;
			for (int32 RevIndex = PrevPositionedNodesInRow.Num() - 1; RevIndex >= 0; --RevIndex)
			{
				const TSharedPtr<ISignalFlowNode> PrevNode = PrevPositionedNodesInRow[RevIndex];
				if (PrevNode.IsValid() && PrevNode->XPos > ShiftXPosition)
				{
					PrevNode->XPos = ShiftXPosition;
					ShiftXPosition -= 1.0f;
				}
				else
				{
					// No more overlaps detected, early out
					break;
				}
			}
		}
		
	}

#if WITH_EDITOR
	void FSignalFlowDashboardViewFactory::OnReadEditorSettings(const FSignalFlowSettings& InSettings)
	{
		using namespace FSignalFlowDashboardViewFactoryPrivate;
		using namespace SSignalFlowGraphStyle;

		if (GraphWidgetContents.IsValid())
		{
			const bool bGraphIsHorizontal = GraphWidgetContents->GetGraphOrientation() == EOrientation::Orient_Horizontal;
			if (InSettings.bHorizontalFlow != bGraphIsHorizontal)
			{
				GraphWidgetContents->ToggleOrientation();
			}

			GraphWidgetContents->SetWireAnimationSettings(InSettings.WireSplineAmplitudePowerFactor, InSettings.WireSplineMaxThicknessScalar);
		}

		GraphWidthRatio = FMath::Clamp(InSettings.GraphWidthRatio, MinGraphWidthRatio, MaxGraphWidthRatio);

		if (MainSplitter.IsValid())
		{
			MainSplitter->SlotAt(MainSplitterSelectionPanelSlotIndex).SetSizeValue(1.0f - GraphWidthRatio);
			MainSplitter->SlotAt(MainSplitterGraphSectionSlotIndex).SetSizeValue(GraphWidthRatio);
		}

		bPauseGraphOnSelect = InSettings.bPauseGraphOnSelect;
		GraphJustification = InSettings.GraphJustification;
		bShowNodeDetails = InSettings.bShowNodeDetails;
		NodeDetailFilters = InSettings.NodeDetailFilters;
		bDisplayAmpPeakInDb = InSettings.AmplitudeDisplayMode == EAudioAmplitudeDisplayMode::Decibels;
		bAnimateWires = InSettings.bAnimateWires;
		LargeNodePadding = FMath::Clamp(InSettings.LargeNodePadding, MinNodePadding, MaxNodePadding);
		SmallNodePadding = FMath::Clamp(InSettings.SmallNodePadding, MinNodePadding, MaxNodePadding);

		const TSharedPtr<FSignalFlowTraceProvider> Provider = FindProvider<FSignalFlowTraceProvider>();
		if (Provider.IsValid())
		{
			Provider->SetConnectionWiresAreAnimated(bAnimateWires);
		}

		ActiveEntryMenuExpansionSettings = InSettings.ActiveEntryMenuExpansionSettings;

		ApplyAllCategorySlotSizes();
	}

	void FSignalFlowDashboardViewFactory::OnWriteEditorSettings(FSignalFlowSettings& OutSettings)
	{
		if (GraphWidgetContents.IsValid())
		{
			const bool bGraphIsHorizontal = GraphWidgetContents->GetGraphOrientation() == EOrientation::Orient_Horizontal;
			OutSettings.bHorizontalFlow = bGraphIsHorizontal;
		}

		if (CategorySplitter.IsValid())
		{
			if (ActiveEntryMenuExpansionSettings.bShowActiveOwnerObjects)
			{
				ActiveEntryMenuExpansionSettings.OwnerObjectsSlotSize = CategorySplitter->SlotAt(static_cast<int32>(ESignalFlowEntryType::OwnerObject)).GetSizeValue();
			}

			if (ActiveEntryMenuExpansionSettings.bShowActiveSources)
			{
				ActiveEntryMenuExpansionSettings.SourcesSlotSize = CategorySplitter->SlotAt(static_cast<int32>(ESignalFlowEntryType::SoundSource)).GetSizeValue();
			}

			if (ActiveEntryMenuExpansionSettings.bShowActiveBuses)
			{
				ActiveEntryMenuExpansionSettings.BusesSlotSize = CategorySplitter->SlotAt(static_cast<int32>(ESignalFlowEntryType::AudioBus)).GetSizeValue();
			}

			if (ActiveEntryMenuExpansionSettings.bShowActiveSubmixes)
			{
				ActiveEntryMenuExpansionSettings.SubmixesSlotSize = CategorySplitter->SlotAt(static_cast<int32>(ESignalFlowEntryType::Submix)).GetSizeValue();
			}
		}

		OutSettings.GraphJustification = GraphJustification;
		OutSettings.bShowNodeDetails = bShowNodeDetails;
		OutSettings.bPauseGraphOnSelect = bPauseGraphOnSelect;
		OutSettings.NodeDetailFilters = NodeDetailFilters;
		OutSettings.ActiveEntryMenuExpansionSettings = ActiveEntryMenuExpansionSettings;
		OutSettings.AmplitudeDisplayMode = bDisplayAmpPeakInDb ? EAudioAmplitudeDisplayMode::Decibels : EAudioAmplitudeDisplayMode::Linear;
		OutSettings.bAnimateWires = bAnimateWires;
		OutSettings.GraphWidthRatio = GraphWidthRatio;
		OutSettings.LargeNodePadding = LargeNodePadding;
		OutSettings.SmallNodePadding = SmallNodePadding;

		const TSharedPtr<FSignalFlowTraceProvider> Provider = FindProvider<FSignalFlowTraceProvider>();
		if (Provider.IsValid())
		{
			Provider->SetConnectionWiresAreAnimated(bAnimateWires);
		}
	}

	TSharedPtr<SWidget> FSignalFlowDashboardViewFactory::OnConstructContextMenu()
	{
		return AssetContextMenuHelper.ContructContextMenuOptions();
	}
#endif // WITH_EDITOR

#if !WITH_EDITOR
	bool FSignalFlowDashboardViewFactory::IsReadingTraceFile() const
	{
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		const TSharedPtr<const FAudioInsightsComponent> AudioInsightsComponent = AudioInsightsModule.GetAudioInsightsComponent();
		const bool bIsReadingTraceFile = AudioInsightsComponent.IsValid() && !AudioInsightsComponent->GetIsLiveSession();
		return bIsReadingTraceFile;
	}
#endif // !WITH_EDITOR
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
