// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SignalFlowNodes.h"

#include "Messages/SignalFlowTraceMessages.h"

namespace UE::Audio::Insights
{
	FSignalFlowEntryKey FSignalFlowEntryNode::GetEntryKey() const
	{
		ensure(Entry.IsValid());
		return Entry.IsValid() ? Entry->GetSignalFlowEntryKey() : FSignalFlowEntryKey();
	}

	void ISignalFlowNode::ResetOrderID()
	{
		PreviousNodeOrderID = NodeOrderID;
		NodeOrderID = INVALID_NODE_ORDER_ID;
	}
} // namespace UE::Audio::Insights