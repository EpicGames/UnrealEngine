// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SoundDashboardViewFactory.h"

#include "Algo/Accumulate.h"
#include "Algo/AnyOf.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsUtils.h"
#include "Messages/SoundTraceMessages.h"
#include "Misc/EnumClassFlags.h"
#include "Providers/SoundTraceProvider.h"
#include "SoundDashboardCommands.h"
#include "Views/SAudioFilterBar.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"

#if WITH_EDITOR
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	/////////////////////////////////////////////////////////////////////////////////////////
	// FSoundDashboardViewFactoryPrivate
	namespace FSoundDashboardViewFactoryPrivate
	{
		const FSoundDashboardEntry& CastEntry(const IDashboardDataTreeViewEntry& InData)
		{
			return static_cast<const FSoundDashboardEntry&>(InData);
		};

		FSoundDashboardEntry& CastEntry(IDashboardDataTreeViewEntry& InData)
		{
			return static_cast<FSoundDashboardEntry&>(InData);
		};

		void OnDefaultPlotRangesChanged(bool bRequestWriteSettings = true)
		{
#if WITH_EDITOR
			if (bRequestWriteSettings)
			{
				FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
			}
#endif // WITH_EDITOR
		}

		bool SetFilteredVisibility(IDashboardDataTreeViewEntry& InEntry, const FString& InFilterString)
		{
			FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);

			bool bEntryMatchesTextFilter = SoundEntry.GetDisplayNameStr().Contains(InFilterString);

			if (bEntryMatchesTextFilter)
			{
				SoundEntry.bIsVisible = true;
			}
			else
			{
				bool bChildMatchesTextFilter = false;

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& SoundEntryChild : SoundEntry.Children)
				{
					if (SoundEntryChild.IsValid() && SetFilteredVisibility(*SoundEntryChild, InFilterString))
					{
						bChildMatchesTextFilter = true;
						break;
					}
				}

				SoundEntry.bIsVisible = bChildMatchesTextFilter;
			}

			return SoundEntry.bIsVisible;
		}

		void ResetVisibility(IDashboardDataTreeViewEntry& InEntry)
		{
			FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);

			SoundEntry.bIsVisible = true;

			for (const TSharedPtr<IDashboardDataTreeViewEntry>& SoundEntryChild : SoundEntry.Children)
			{
				if (SoundEntryChild.IsValid())
				{
					ResetVisibility(*SoundEntryChild);
				}
			}
		}

		bool IsCategoryItem(const IDashboardDataTreeViewEntry& InEntry)
		{
			const FSoundDashboardEntry& SoundEntry = CastEntry(InEntry);
			return SoundEntry.bIsCategory;
		}

		bool IsVisible(const IDashboardDataTreeViewEntry& InEntry, const bool bShowRecentlyStoppedSounds)
		{
			const FSoundDashboardEntry& SoundEntry = CastEntry(InEntry);

			return SoundEntry.bIsVisible && ((bShowRecentlyStoppedSounds || SoundEntry.TimeoutTimestamp == INVALID_TIMEOUT) || SoundEntry.bForceKeepEntryAlive);
		}

		FSlateColor GetIconColor(const TSharedPtr<IDashboardDataTreeViewEntry>& InEntry)
		{
			const FSoundDashboardEntry& SoundDashboardEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*InEntry);

			switch (SoundDashboardEntry.EntryType)
			{
				case ESoundDashboardEntryType::MetaSound:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.MetaSoundColor"));

				case ESoundDashboardEntryType::SoundCue:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.SoundCueColor"));

				case ESoundDashboardEntryType::ProceduralSource:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.ProceduralSourceColor"));

				case ESoundDashboardEntryType::SoundWave:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.SoundWaveColor"));

				case ESoundDashboardEntryType::SoundCueTemplate:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.SoundCueTemplateColor"));

				case ESoundDashboardEntryType::Pinned:
					return FSlateColor(FSlateStyle::Get().GetColor("SoundDashboard.PinnedColor"));

				case ESoundDashboardEntryType::None:
				default:
					return FSlateColor(FColor::White);
			}
		};

		void RefreshInitExpandStateRecursive(FSoundDashboardEntry& SoundEntry)
		{
			SoundEntry.bHasSetInitExpansion = false;

			for (TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : SoundEntry.Children)
			{
				if (ChildEntry.IsValid())
				{
					RefreshInitExpandStateRecursive(CastEntry(*ChildEntry));
				}
			}
		}
		
		bool EntryCanHaveChildren(const TSharedRef<IDashboardDataTreeViewEntry>& InEntry)
		{
			const FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*InEntry);

			const bool bIsSoundCueType = SoundEntry.EntryType == ESoundDashboardEntryType::SoundCue || SoundEntry.EntryType == ESoundDashboardEntryType::SoundCueTemplate;

			return IsCategoryItem(*InEntry) || bIsSoundCueType;
		}

		bool IsDescendant(const TSharedPtr<IDashboardDataTreeViewEntry>& InEntry, const TSharedPtr<IDashboardDataTreeViewEntry>& InChildCandidate)
		{
			if (InEntry.IsValid() && EntryCanHaveChildren(InEntry.ToSharedRef()))
			{
				for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : InEntry->Children)
				{
					if (ChildEntry == InChildCandidate || IsDescendant(ChildEntry, InChildCandidate))
					{
						return true;
					}
				}
			}

			return false;
		}

		bool HasPinEntryType(const IDashboardDataTreeViewEntry& InEntry, const FSoundDashboardEntry::EPinnedEntryType PinnedEntryType)
		{
			const FSoundDashboardEntry& SoundEntry = CastEntry(InEntry);
			return SoundEntry.PinnedEntryType == PinnedEntryType;
		}

		int32 GetNumChildrenWithoutPinEntryType(const IDashboardDataTreeViewEntry& InEntry, const FSoundDashboardEntry::EPinnedEntryType ExcludedPinnedEntryType, const bool bShowRecentlyStoppedSounds)
		{
			int32 NumChildrenWithoutType = 0;
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : InEntry.Children)
			{
				if (!Child.IsValid())
				{
					continue;
				}

				if (!HasPinEntryType(*Child, ExcludedPinnedEntryType) && IsVisible(*Child, bShowRecentlyStoppedSounds))
				{
					++NumChildrenWithoutType;
				}
			}

			return NumChildrenWithoutType;
		}

		int32 CountNumChildren(const IDashboardDataTreeViewEntry& InEntry, const bool bShowRecentlyStoppedSounds, const bool bIncludeTimingOutSounds = false)
		{
			const uint32 TotalNumChildren = Algo::Accumulate(InEntry.Children, 0,
				[bShowRecentlyStoppedSounds, bIncludeTimingOutSounds](uint32 Accum, TSharedPtr<IDashboardDataTreeViewEntry> InChild)
				{
					if (InChild.IsValid())
					{
						const FSoundDashboardEntry& SoundDashboardEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*InChild);

						if (bIncludeTimingOutSounds || SoundDashboardEntry.TimeoutTimestamp == INVALID_TIMEOUT)
						{
							if (HasPinEntryType(*InChild, FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry) || !SoundDashboardEntry.bIsVisible)
							{
								return Accum;
							}

							const int32 NumNestedChildren = GetNumChildrenWithoutPinEntryType(*InChild, FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry, bShowRecentlyStoppedSounds);

							if (NumNestedChildren > 0)
							{
								return Accum + NumNestedChildren;
							}

							return SoundDashboardEntry.bIsCategory ? Accum : Accum + 1;
						}
					}

					return Accum;
				});

			return TotalNumChildren;
		}

		void ClampMaxTimeoutRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const double NewTimeOutTimestamp)
		{
			if (!Entry.IsValid())
			{
				return;
			}

			FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

			// Clamp max timeout to the current max timeout from the settings
			if (SoundEntry.TimeoutTimestamp != INVALID_TIMEOUT && SoundEntry.TimeoutTimestamp > NewTimeOutTimestamp)
			{
				SoundEntry.TimeoutTimestamp = NewTimeOutTimestamp;
			}

			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : Entry->Children)
			{
				ClampMaxTimeoutRecursive(Child, NewTimeOutTimestamp);
			}
		}

#if WITH_EDITOR
		void SetMuteSolo(const IDashboardDataTreeViewEntry& InEntry, const EMuteSoloMode InMuteSoloMode, const bool bInOnOff)
		{
#if ENABLE_AUDIO_DEBUG
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				::Audio::FAudioDebugger& AudioDebugger = AudioDeviceManager->GetDebugger();

				const FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);

				// Skip setting mute/solo, a copy of this entry is currently in the Pinned category 
				if (SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry)
				{
					return;
				}

				const bool bIsSoundCueType = SoundEntry.EntryType == ESoundDashboardEntryType::SoundCue || SoundEntry.EntryType == ESoundDashboardEntryType::SoundCueTemplate;

				if (!bIsSoundCueType)
				{
					switch (InMuteSoloMode)
					{
						case EMuteSoloMode::Mute:
							AudioDebugger.SetMuteSoundWave(SoundEntry.GetDisplayFName(), bInOnOff);
							break;

						case EMuteSoloMode::Solo:
							AudioDebugger.SetSoloSoundWave(SoundEntry.GetDisplayFName(), bInOnOff);
							break;

						default:
							break;
					}
				}

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& SoundEntryChild : SoundEntry.Children)
				{
					if (SoundEntryChild.IsValid())
					{
						SetMuteSolo(*SoundEntryChild, InMuteSoloMode, bInOnOff);
					}
				}
			}
#endif // ENABLE_AUDIO_DEBUG
		}

		void ToggleMuteSolo(const IDashboardDataTreeViewEntry& InEntry, const EMuteSoloMode InMuteSoloMode)
		{
#if ENABLE_AUDIO_DEBUG
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				::Audio::FAudioDebugger& AudioDebugger = AudioDeviceManager->GetDebugger();

				const FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);

				const bool bIsSoundCueType = SoundEntry.EntryType == ESoundDashboardEntryType::SoundCue || SoundEntry.EntryType == ESoundDashboardEntryType::SoundCueTemplate;

				if (!bIsSoundCueType)
				{
					switch (InMuteSoloMode)
					{
						case EMuteSoloMode::Mute:
							AudioDebugger.ToggleMuteSoundWave(SoundEntry.GetDisplayFName());
							break;

						case EMuteSoloMode::Solo:
							AudioDebugger.ToggleSoloSoundWave(SoundEntry.GetDisplayFName());
							break;

						default:
							break;
					}
				}

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& SoundEntryChild : SoundEntry.Children)
				{
					if (SoundEntryChild.IsValid())
					{
						ToggleMuteSolo(*SoundEntryChild, InMuteSoloMode);
					}
				}
			}
#endif // ENABLE_AUDIO_DEBUG
		}

#if ENABLE_AUDIO_DEBUG
		bool IsMuteSolo(::Audio::FAudioDebugger& InAudioDebugger, const IDashboardDataTreeViewEntry& InEntry, const bool bInCheckChildren, const EMuteSoloMode InMuteSoloMode)
		{
			const FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);

			// Treat hidden original entries as muted/soloed to ensure the parent category reflects the correct state
			if (SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry)
			{
				return true;
			}

			const bool bIsSoundCueType = SoundEntry.EntryType == ESoundDashboardEntryType::SoundCue || SoundEntry.EntryType == ESoundDashboardEntryType::SoundCueTemplate;

			if (!bIsSoundCueType && !SoundEntry.bIsCategory)
			{
				bool bIsMuted = false;
				bool bIsSoloed = false;
				FString Reason;
				InAudioDebugger.QuerySoloMuteSoundWave(SoundEntry.GetDisplayNameStr(), bIsSoloed, bIsMuted, Reason);

				switch (InMuteSoloMode)
				{
					case EMuteSoloMode::Mute:
					{
						if (bIsMuted)
						{
							return true;
						}

						break;
					}

					case EMuteSoloMode::Solo:
					{
						if (bIsSoloed)
						{
							return true;
						}

						break;
					}

					default:
						break;
				}
			}

			if (bInCheckChildren)
			{
				uint32 NumChildrenMuteSolo = 0;

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& SoundEntryChild : SoundEntry.Children)
				{
					if (SoundEntryChild.IsValid() && IsMuteSolo(InAudioDebugger, *SoundEntryChild, true /*bInCheckChildren*/, InMuteSoloMode))
					{
						++NumChildrenMuteSolo;
					}
				}

				const bool bAllChildrenMuteSolo = !SoundEntry.Children.IsEmpty() && NumChildrenMuteSolo == SoundEntry.Children.Num();

				return bAllChildrenMuteSolo;
			}

			return false;
		}
#endif // ENABLE_AUDIO_DEBUG

		void ClearMutesAndSolos()
		{
#if ENABLE_AUDIO_DEBUG
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				AudioDeviceManager->GetDebugger().ClearMutesAndSolos();
			}
#endif
		}
#endif // WITH_EDITOR

		const FText FiltersName = LOCTEXT("SoundDashboard_Filter_CategoryText", "Filters");
		const FText FiltersTooltip = LOCTEXT("CurveFiltersToolTip", "Filters what kind of sounds types can be displayed.");

		const FText MetaSoundCategoryName = LOCTEXT("SoundDashboard_Filter_MetaSoundNameText", "MetaSound");
		const FText SoundCueCategoryName = LOCTEXT("SoundDashboard_Filter_SoundCueNameText", "Sound Cue");
		const FText ProceduralSourceCategoryName = LOCTEXT("SoundDashboard_Filter_ProceduralSourceNameText", "Procedural Source");
		const FText SoundWaveCategoryName = LOCTEXT("SoundDashboard_Filter_SoundWaveNameText", "Sound Wave");
		const FText SoundCueTemplateCategoryName = LOCTEXT("SoundDashboard_Filter_SoundCueTemplateNameText", "Sound Cue Template");
		const FText PinnedCategoryName = LOCTEXT("SoundDashboard_Filter_PinnedNameText", "Pinned");

		const FName MuteColumnName = "Mute";
		const FText MuteColumnDisplayName = LOCTEXT("SoundDashboard_MuteColumnDisplayName", "Mute");
		const FText MuteColumnTooltip = LOCTEXT("SoundDashboard_MuteColumnTooltip", "Mute a sound/category.\nMute will apply to all instances of an asset.");

		const FName SoloColumnName = "Solo";
		const FText SoloColumnDisplayName = LOCTEXT("SoundDashboard_SoloColumnDisplayName", "Solo");
		const FText SoloColumnTooltip = LOCTEXT("SoundDashboard_SoloColumnTooltip", "Solo a sound/category.\nMute will apply to all instances of an asset.");

		const FName PlotColumnName = "Plot";
		const FText PlotColumnDisplayName = LOCTEXT("SoundDashboard_PlotColumnDisplayName", "Plot");
		const FText PlotColumnTooltip = LOCTEXT("SoundDashboard_PlotColumnTooltip", "Plot property values for a category/sound instance/aggregate data for a parent active sound instance.\nPlots will appear in the Plots tab in Audio Insights.");

		const FName NameColumnName = "Name";
		const FText NameColumnDisplayName = LOCTEXT("SoundDashboard_NameColumnDisplayName", "Name");
		const FText NameColumnTooltip = LOCTEXT("SoundDashboard_NameColumnTooltip", "The name of the sound/category.");

		const FName PlayOrderColumnName = "PlayOrder";
		const FText PlayOrderColumnDisplayName = LOCTEXT("SoundDashboard_PlayOrderColumnDisplayName", "Play Order");
		const FText PlayOrderColumnTooltip = LOCTEXT("SoundDashboard_PlayOrderColumnTooltip", "The order in which a sound was played.\nFirst Num = Active Sound Play Order, Second Num = Wave Instance Play Order.");

		const FName PriorityColumnName = "Priority";
		const FText PriorityColumnDisplayName = LOCTEXT("SoundDashboard_PriorityColumnDisplayName", "Priority");
		const FText PriorityColumnTooltip = LOCTEXT("SoundDashboard_PriorityColumnTooltip", "The priority of a sound.\nUsed to determine whether sound can play or remain active if channel limit is met, where higher value is higher priority.");

		const FName DistanceColumnName = "Distance";
		const FText DistanceColumnDisplayName = LOCTEXT("SoundDashboard_DistanceColumnDisplayName", "Distance");
		const FText DistanceColumnTooltip = LOCTEXT("SoundDashboard_DistanceColumnTooltip", "The distance from the sound source to the listener.");

		const FName DistanceAttenuationColumnName = "DistanceAttenuation";
		const FText DistanceAttenuationColumnDisplayName = LOCTEXT("SoundDashboard_DistanceAttenuationColumnDisplayName", "Distance/Occlusion Attenuation");
		const FText DistanceAttenuationColumnTooltip = LOCTEXT("SoundDashboard_DistanceAttenuationColumnTooltip", "The amount of attenuation applied by the combined effect of distance and occlusion.");

		const FName AmplitudeColumnName = "Amplitude";
		const FText AmplitudeColumnDisplayName = LOCTEXT("SoundDashboard_AmplitudeColumnDisplayName", "Amp (Peak)");
		const FText AmplitudeColumnTooltip = LOCTEXT("SoundDashboard_AmplitudeColumnTooltip", "The measured peak amplitude of the sound.\nMeasured in either decibels or linear volume.\nThe units of measurement can be changed in the Sound tab settings.");

		const FName VolumeColumnName = "Volume";
		const FText VolumeColumnDisplayName = LOCTEXT("SoundDashboard_VolumeColumnDisplayName", "Volume");
		const FText VolumeColumnTooltip = LOCTEXT("SoundDashboard_VolumeColumnTooltip", "The volume multiplier applied to a sound.");

		const FName LPFFreqColumnName = "LPFFreq";
		const FText LPFFreqColumnDisplayName = LOCTEXT("SoundDashboard_LPFFreqColumnDisplayName", "LPF Freq (Hz)");
		const FText LPFFreqColumnTooltip = LOCTEXT("SoundDashboard_LPFFreqColumnTooltip", "The frequency of the Low Pass Filter applied to the sound, measured in Hertz.\nFrequencies above the value will be filtered.");

		const FName HPFFreqColumnName = "HPFFreq";
		const FText HPFFreqColumnDisplayName = LOCTEXT("SoundDashboard_HPFFreqColumnDisplayName", "HPF Freq (Hz)");
		const FText HPFFreqColumnTooltip = LOCTEXT("SoundDashboard_HPFColumnTooltip", "The frequency of the High Pass Filter applied to the sound, measured in Hertz.\nFrequencies below the value will be filtered.");

		const FName PitchColumnName = "Pitch";
		const FText PitchColumnDisplayName = LOCTEXT("SoundDashboard_PitchColumnDisplayName", "Pitch");
		const FText PitchColumnTooltip = LOCTEXT("SoundDashboard_PitchColumnTooltip", "The pitch multiplier applied to a sound.");

		const FName RelativeRenderCostColumnName = "RelativeRenderCost";
		const FText RelativeRenderCostColumnDisplayName = LOCTEXT("SoundDashboard_RelativeRenderCostColumnDisplayName", "Rel. Render Cost");
		const FText RelativeRenderCostColumnTooltip = LOCTEXT("SoundDashboard_RelativeRenderCostColumnTooltip", "The estimated relative render cost of the sound.");

		const FName ActorLabelColumnName = "ActorLabel";
		const FText ActorLabelColumnDisplayName = LOCTEXT("SoundDashboard_ActorLabelColumnDisplayName", "Actor Label");
		const FText ActorLabelColumnTooltip = LOCTEXT("SoundDashboard_ActorLabelColumnTooltip", "The label or name of the Actor this sound is playing on (if applicable).");

		const FName AudioComponentNameColumnName = "AudioComponentName";
		const FText AudioComponentNameColumnDisplayName = LOCTEXT("SoundDashboard_AudioComponentNameColumnDisplayName", "Audio Component");
		const FText AudioComponentNameColumnTooltip = LOCTEXT("SoundDashboard_AudioComponentNameColumnTooltip", "The name of the Audio Component playing this sound (if applicable).");

		const FName CategoryColumnName = "Category";
		const FText CategoryColumnDisplayName = LOCTEXT("SoundDashboard_CategoryColumnDisplayName", "Category");
		const FText CategoryColumnTooltip = LOCTEXT("SoundDashboard_CategoryColumnTooltip", "The type of sound asset this sound belongs to.");
	} // namespace FSoundDashboardViewFactoryPrivate

	/////////////////////////////////////////////////////////////////////////////////////////
	// FPinnedSoundEntryWrapperPrivate
	namespace FPinnedSoundEntryWrapperPrivate
	{
		bool CanBeDeleted(const TSharedPtr<FPinnedSoundEntryWrapper>& Entry)
		{
			using namespace FSoundDashboardViewFactoryPrivate;

			return !Entry->EntryIsValid() || (IsCategoryItem(*Entry->GetPinnedSectionEntry()) && Entry->PinnedWrapperChildren.IsEmpty());
		}

		void CopyDataToPinnedEntry(FSoundDashboardEntry& PinnedEntry, const FSoundDashboardEntry& OriginalEntry)
		{
			// Only copy data that has possibly changed from the other entry
			PinnedEntry.TimeoutTimestamp = OriginalEntry.TimeoutTimestamp;
			PinnedEntry.bIsVisible = OriginalEntry.bIsVisible;

			PinnedEntry.PriorityDataPoint = OriginalEntry.PriorityDataPoint;
			PinnedEntry.DistanceDataPoint = OriginalEntry.DistanceDataPoint;
			PinnedEntry.DistanceAttenuationDataPoint = OriginalEntry.DistanceAttenuationDataPoint;
			PinnedEntry.LPFFreqDataPoint = OriginalEntry.LPFFreqDataPoint;
			PinnedEntry.HPFFreqDataPoint = OriginalEntry.HPFFreqDataPoint;
			PinnedEntry.AmplitudeDataPoint = OriginalEntry.AmplitudeDataPoint;
			PinnedEntry.VolumeDataPoint = OriginalEntry.VolumeDataPoint;
			PinnedEntry.PitchDataPoint = OriginalEntry.PitchDataPoint;
			PinnedEntry.RelativeRenderCostDataPoint = OriginalEntry.RelativeRenderCostDataPoint;
		};
	} // namespace FPinnedSoundEntryWrapperPrivate

	/////////////////////////////////////////////////////////////////////////////////////////
	// FPinnedSoundEntryWrapper
	FPinnedSoundEntryWrapper::FPinnedSoundEntryWrapper(const TSharedPtr<IDashboardDataTreeViewEntry>& OriginalEntry)
		: OriginalDataEntry(OriginalEntry)
	{
		if (!OriginalEntry.IsValid())
		{
			return;
		}

		// Take a deep copy of the original entry to add to the pinned section of the dashboard
		// We need deep copies of any children too
		PinnedSectionEntry = MakeShared<FSoundDashboardEntry>(FSoundDashboardViewFactoryPrivate::CastEntry(*OriginalEntry));
		PinnedSectionEntry->Children.Empty();

		FSoundDashboardEntry& PinnedSectionSoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*PinnedSectionEntry);
		PinnedSectionSoundEntry.PinnedEntryType = FSoundDashboardEntry::EPinnedEntryType::PinnedCopy;
		PinnedSectionSoundEntry.bIsVisible = true;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : OriginalEntry->Children)
		{
			AddChildEntry(Child.ToSharedRef());
		}
	}

	TSharedPtr<FPinnedSoundEntryWrapper> FPinnedSoundEntryWrapper::AddChildEntry(const TSharedPtr<IDashboardDataTreeViewEntry> Child)
	{
		TSharedPtr<FPinnedSoundEntryWrapper> NewChild = MakeShared<FPinnedSoundEntryWrapper>(Child);

		FSoundDashboardEntry& NewChildSound = FSoundDashboardViewFactoryPrivate::CastEntry(*NewChild->GetPinnedSectionEntry());

		PinnedWrapperChildren.Add(NewChild);
		PinnedSectionEntry->Children.Add(NewChild->GetPinnedSectionEntry());

		return NewChild;
	}

	void FPinnedSoundEntryWrapper::UpdateParams()
	{
		// If we lose our handle to the original entry, we should stop updating
		if (!EntryIsValid())
		{
			OriginalDataEntry.Reset();
			PinnedSectionEntry.Reset();

			return;
		}

		// Only non-category entries have data to update
		if (OriginalDataEntry.IsValid())
		{
			FSoundDashboardEntry& Pinned = FSoundDashboardViewFactoryPrivate::CastEntry(*PinnedSectionEntry);
			const FSoundDashboardEntry& Original = FSoundDashboardViewFactoryPrivate::CastEntry(*OriginalDataEntry.Pin());

			FPinnedSoundEntryWrapperPrivate::CopyDataToPinnedEntry(Pinned, Original);
		}

		for (const TSharedPtr<FPinnedSoundEntryWrapper>& Child : PinnedWrapperChildren)
		{
			Child->UpdateParams();
		}
	}

	void FPinnedSoundEntryWrapper::CleanUp()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		// Remove any pinned items whose original data entries have been removed
		
		// Note: Active sounds restart with the same PlayOrderID when realizing after virtualizing, but WaveInstances start with new Play Order IDs, which creates new dashboard entries.
		// To fix this edge case, when a pinned entry loses it's original entry, double check that a new one hasn't appeared in it's place.
		// If it has, recreate the child entries.
		bool bCanBeRecovered = false;
		if (!IsCategoryItem(*PinnedSectionEntry) && OriginalDataEntry.IsValid())
		{
			const FSoundDashboardEntry& OriginalSoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*OriginalDataEntry.Pin());

			// A sound entry may be recovereable if it is still active, is not timing out, has child entries and it is currently pinned
			bCanBeRecovered = OriginalSoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry 
							&& OriginalSoundEntry.TimeoutTimestamp == INVALID_TIMEOUT
							&& OriginalSoundEntry.Children.Num() > 0;
		}

		bool bRecreateAfterClean = false;
		for (int Index = PinnedWrapperChildren.Num() - 1; Index >= 0; --Index)
		{
			const TSharedPtr<FPinnedSoundEntryWrapper> Child = PinnedWrapperChildren[Index];
			if (FPinnedSoundEntryWrapperPrivate::CanBeDeleted(Child))
			{
				if (bCanBeRecovered)
				{
					// If the parent sound is still alive, but the child is no longer valid, destroy and recreate all pinned child entries
					PinnedSectionEntry->Children.Empty();
					PinnedWrapperChildren.Empty();
					bRecreateAfterClean = true;
					break;
				}

				TSharedPtr<IDashboardDataTreeViewEntry> DataEntry = Child->GetPinnedSectionEntry();
				PinnedSectionEntry->Children.Remove(DataEntry);
				PinnedWrapperChildren.Remove(Child);
			}
			else
			{
				Child->CleanUp();
			}
		}

		if (bRecreateAfterClean)
		{
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : OriginalDataEntry.Pin()->Children)
			{
				AddChildEntry(Child);
			}
		}
	}

	void FPinnedSoundEntryWrapper::MarkToDelete()
	{
		OriginalDataEntry.Reset();
	}

	bool FPinnedSoundEntryWrapper::EntryIsValid() const
	{
		return PinnedSectionEntry.IsValid() && (OriginalDataEntry.IsValid() || FSoundDashboardViewFactoryPrivate::IsCategoryItem(*PinnedSectionEntry));
	}

	TSharedPtr<IDashboardDataTreeViewEntry> FPinnedSoundEntryWrapper::FindOriginalEntryInChildren(const TSharedPtr<IDashboardDataTreeViewEntry>& PinnedEntry)
	{
		if (!PinnedEntry.IsValid())
		{
			return nullptr;
		}

		if (GetPinnedSectionEntry() == PinnedEntry)
		{
			return GetOriginalDataEntry();
		}

		for (const TSharedPtr<FPinnedSoundEntryWrapper>& ChildPinnedEntryWrapper : PinnedWrapperChildren)
		{
			TSharedPtr<IDashboardDataTreeViewEntry> FoundEntry = ChildPinnedEntryWrapper->FindOriginalEntryInChildren(PinnedEntry);
			if (FoundEntry.IsValid())
			{
				return FoundEntry;
			}
		}
		return nullptr;
	}

	/////////////////////////////////////////////////////////////////////////////////////////
	// FSoundDashboardViewFactory
	FSoundDashboardViewFactory::FSoundDashboardViewFactory()
	{
		FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());

		const TSharedPtr<FSoundTraceProvider> SoundsTraceProvider = MakeShared<FSoundTraceProvider>();
		SoundsTraceProvider->OnProcessPlotData.AddRaw(this, &FSoundDashboardViewFactory::ProcessPlotData);

		AudioInsightsTraceModule.AddTraceProvider(SoundsTraceProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			SoundsTraceProvider
		};
		
		FSoundDashboardCommands::Register();

		BindCommands();

#if WITH_EDITOR
		FSoundDashboardSettings::OnReadSettings.AddRaw(this, &FSoundDashboardViewFactory::OnReadEditorSettings);
		FSoundDashboardSettings::OnWriteSettings.AddRaw(this, &FSoundDashboardViewFactory::OnWriteEditorSettings);
#endif
	}

	FSoundDashboardViewFactory::~FSoundDashboardViewFactory()
	{
		const TSharedPtr<FSoundTraceProvider> Provider = FindProvider<FSoundTraceProvider>();
		if (Provider.IsValid())
		{
			Provider->OnProcessPlotData.RemoveAll(this);
		}

		FSoundDashboardCommands::Unregister();

#if WITH_EDITOR
		FSoundDashboardSettings::OnReadSettings.RemoveAll(this);
		FSoundDashboardSettings::OnWriteSettings.RemoveAll(this);
#endif
	}

	FName FSoundDashboardViewFactory::GetName() const
	{
		return "Sounds";
	}

	FText FSoundDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_Sounds_DisplayName", "Sounds");
	}

	FSlateIcon FSoundDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Tab");
	}

	EDefaultDashboardTabStack FSoundDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Analysis;
	}

	void FSoundDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FSoundDashboardCommands& Commands = FSoundDashboardCommands::Get();

		CommandList->MapAction(Commands.GetPinCommand(), FExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::PinSound));
		CommandList->MapAction(Commands.GetUnpinCommand(), FExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::UnpinSound));
#if WITH_EDITOR
		CommandList->MapAction(Commands.GetBrowseCommand(), FExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::BrowseSoundAsset), FCanExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::SelectedItemsIncludesAnAsset));
		CommandList->MapAction(Commands.GetEditCommand(),   FExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::OpenSoundAsset), FCanExecuteAction::CreateRaw(this, &FSoundDashboardViewFactory::SelectedItemsIncludesAnAsset));
#endif // WITH_EDITOR

		CommandList->MapAction(
			Commands.GetViewFullTreeCommand(),
			FExecuteAction::CreateLambda([this]() { ChangeTreeViewingMode(ESoundDashboardTreeViewingOptions::FullTree); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return TreeViewingMode == ESoundDashboardTreeViewingOptions::FullTree; }));

		CommandList->MapAction(
			Commands.GetViewActiveSoundsCommand(),
			FExecuteAction::CreateLambda([this]() { ChangeTreeViewingMode(ESoundDashboardTreeViewingOptions::ActiveSounds); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return TreeViewingMode == ESoundDashboardTreeViewingOptions::ActiveSounds; }));

		CommandList->MapAction(
			Commands.GetViewFlatList(),
			FExecuteAction::CreateLambda([this]() { ChangeTreeViewingMode(ESoundDashboardTreeViewingOptions::FlatList); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return TreeViewingMode == ESoundDashboardTreeViewingOptions::FlatList; }));

		CommandList->MapAction(
			Commands.GetAutoExpandCategoriesCommand(),
			FExecuteAction::CreateLambda([this]() { ChangeAutoExpandMode(ESoundDashboardAutoExpandOptions::Categories); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return AutoExpandMode == ESoundDashboardAutoExpandOptions::Categories; }));

		CommandList->MapAction(
			Commands.GetAutoExpandEverythingCommand(),
			FExecuteAction::CreateLambda([this]() { ChangeAutoExpandMode(ESoundDashboardAutoExpandOptions::Everything); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return AutoExpandMode == ESoundDashboardAutoExpandOptions::Everything; }));

		CommandList->MapAction(
			Commands.GetAutoExpandNothingCommand(),
			FExecuteAction::CreateLambda([this]() { ChangeAutoExpandMode(ESoundDashboardAutoExpandOptions::Nothing); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return AutoExpandMode == ESoundDashboardAutoExpandOptions::Nothing; }));
	}

#if WITH_EDITOR
	TSharedRef<SWidget> FSoundDashboardViewFactory::MakeMuteSoloWidget()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const bool bIsNotFilteredMuteOrSolo = !(bIsMuteFilteredMode || bIsSoloFilteredMode);
		TFunction<FSlateColor()> MuteColorFunc = [this]() { return bIsMuteFilteredMode ? FSlateColor(FColor::Green) : FSlateColor::UseForeground(); };
		TFunction<FText()> MuteTextFunc = [bNotMuteSolo = bIsNotFilteredMuteOrSolo, &FilteredList = FilteredEntriesListView]()
			{
				return FilteredList.IsValid() && bNotMuteSolo && FilteredList->GetNumItemsSelected() > 0 ?
					LOCTEXT("SoundDashboard_MuteButtonTextSelect", "Mute Selected") :
					LOCTEXT("SoundDashboard_MuteButtonTextFilter", "Mute by Text");
			};
		TFunction<FSlateColor()> SoloColorFunc = [this]() { return bIsSoloFilteredMode ? FSlateColor(FColor::Green) : FSlateColor::UseForeground(); };
		TFunction<FText()> SoloTextFunc = [bNotMuteSolo = bIsNotFilteredMuteOrSolo, &FilteredList = FilteredEntriesListView]()
			{
				return FilteredList.IsValid() && bNotMuteSolo && FilteredList->GetNumItemsSelected() > 0 ?
					LOCTEXT("SoundDashboard_SoloButtonTextSelect", "Solo Selected") :
					LOCTEXT("SoundDashboard_SoloButtonTextFilter", "Solo by Text");
			};

		return SNew(SHorizontalBox)
			// Mute button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundDashboard_MuteTooltipText", "Toggles the mute state of the selected items."))
				.OnClicked_Lambda([this]()
				{
					if (FilteredEntriesListView.IsValid())
					{
						if (FilteredEntriesListView->GetNumItemsSelected() > 0 && !bIsMuteFilteredMode)
						{
							ToggleMuteSoloEntries(FilteredEntriesListView->GetSelectedItems(), EMuteSoloMode::Mute);
						}
						else
						{
							bIsMuteFilteredMode = !bIsMuteFilteredMode;
							MuteSoloFilteredEntries();
						}
					}

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().CreateToggleButtonContent("AudioInsights.Icon.SoundDashboard.Mute", "SmallButtonText", MuteColorFunc, MuteTextFunc)
				]
			]
			// Solo button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundDashboard_SoloTooltipText", "Toggles the solo state of the selected items."))
				.OnClicked_Lambda([this]()
				{
					if (FilteredEntriesListView.IsValid())
					{
						if (FilteredEntriesListView->GetNumItemsSelected() > 0 && !bIsSoloFilteredMode)
						{
							ToggleMuteSoloEntries(FilteredEntriesListView->GetSelectedItems(), EMuteSoloMode::Solo);
						}
						else
						{
							bIsSoloFilteredMode = !bIsSoloFilteredMode;
							MuteSoloFilteredEntries();
						}
					}

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().CreateToggleButtonContent("AudioInsights.Icon.SoundDashboard.Solo", "SmallButtonText", SoloColorFunc, SoloTextFunc)
				]
			]
			// Clear Mutes/Solos button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundsDashboard_ClearMutesAndSolosTooltipText", "Clears all assigned mute/solo states."))
				.OnClicked_Lambda([this]()
				{
					ClearMutesAndSolos();
					bIsMuteFilteredMode = false;
					bIsSoloFilteredMode = false;

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().FSlateStyle::Get().CreateButtonContentWidget("AudioInsights.Icon.SoundDashboard.Reset", LOCTEXT("SoundsDashboard_ClearMutesAndSolosButtonText", "Clear All Mutes/Solos"), "SmallButtonText")
				]
			];
	}
#endif // WITH_EDITOR

	TSharedPtr<SWidget> FSoundDashboardViewFactory::GetFilterBarWidget()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!SoundsFilterBar.IsValid())
		{
			const TSharedPtr<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(FiltersName, FiltersTooltip);
			
			TArray<TSharedRef<FFilterBase<ESoundDashboardFilterFlags>>> Filters{
				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::MetaSound,
				"MetaSound",
				MetaSoundCategoryName,
				"AudioInsights.Icon.SoundDashboard.MetaSound",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.MetaSoundColor"),
				FilterCategory),

				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::SoundCue,
				"SoundCue",
				SoundCueCategoryName,
				"AudioInsights.Icon.SoundDashboard.SoundCue",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.SoundCueColor"),
				FilterCategory),

				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::ProceduralSource,
				"ProceduralSource",
				ProceduralSourceCategoryName,
				"AudioInsights.Icon.SoundDashboard.ProceduralSource",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.ProceduralSourceColor"),
				FilterCategory),

				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::SoundWave,
				"SoundWave",
				SoundWaveCategoryName,
				"AudioInsights.Icon.SoundDashboard.SoundWave",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.SoundWaveColor"),
				FilterCategory),

				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::SoundCueTemplate,
				"SoundCueTemplate",
				SoundCueTemplateCategoryName,
				"AudioInsights.Icon.SoundDashboard.SoundCue",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.SoundCueTemplateColor"),
				FilterCategory),

				MakeShared<FSoundDashboardFilter>(
				ESoundDashboardFilterFlags::Pinned,
				"Pinned",
				PinnedCategoryName,
				"AudioInsights.Icon.SoundDashboard.Pin",
				FText::GetEmpty(),
				FSlateStyle::Get().GetColor("SoundDashboard.PinnedColor"),
				FilterCategory)
			};

			SAssignNew(SoundsFilterBar, SAudioFilterBar<ESoundDashboardFilterFlags>)
			.CustomFilters(Filters)
			.OnFilterChanged_Lambda([this, Filters]()
			{
				auto GetActiveFilterFlags = [&Filters]()
				{
					ESoundDashboardFilterFlags ActiveFilterFlags = ESoundDashboardFilterFlags::None;

					for (const TSharedRef<FFilterBase<ESoundDashboardFilterFlags>>& Filter : Filters)
					{
						TSharedRef<FSoundDashboardFilter> SoundDashboardFilter = StaticCastSharedRef<FSoundDashboardFilter>(Filter);

						if (SoundDashboardFilter->IsActive())
						{
							ActiveFilterFlags |= SoundDashboardFilter->GetFlags();
						}
					}

					// By default, if there are no active filters selected it means that all filters are enabled
					return ActiveFilterFlags != ESoundDashboardFilterFlags::None ? ActiveFilterFlags : AllFilterFlags;
				};

				SelectedFilterFlags = GetActiveFilterFlags();
				bIsPinnedCategoryFilterEnabled = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::Pinned);

				UpdateFilterReason = EProcessReason::FilterUpdated;
			});
		}

		return SoundsFilterBar;
	}

	TSharedPtr<SWidget> FSoundDashboardViewFactory::GetFilterBarButtonWidget()
	{
		if (!SoundsFilterBarButton.IsValid())
		{
			if (!SoundsFilterBar.IsValid())
			{
				GetFilterBarWidget();
			}

			SoundsFilterBarButton = SBasicFilterBar<ESoundDashboardFilterFlags>::MakeAddFilterButton(StaticCastSharedPtr<SAudioFilterBar<ESoundDashboardFilterFlags>>(SoundsFilterBar).ToSharedRef()).ToSharedPtr();
		}

		return SoundsFilterBarButton;
	}

	void FSoundDashboardViewFactory::SetSearchBoxFilterText(const FText& NewText)
	{
		FTraceTreeDashboardViewFactory::SetSearchBoxFilterText(NewText);

#if WITH_EDITOR
#if ENABLE_AUDIO_DEBUG
		MuteSoloFilteredEntries();
#endif // ENABLE_AUDIO_DEBUG
#endif // WITH_EDITOR

		if (bIsPlotFilteredMode)
		{
			FilterText(); // Sets the entry visibility, required for use within PlotFilteredEntries
			ResetPlots();

			if (!NewText.IsEmpty())
			{
				PlotFilteredEntries();
			}
		}
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
#if WITH_EDITOR
		FSoundDashboardSettings::OnRequestReadSettings.Broadcast();
#endif // WITH_EDITOR

		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().OnTimingViewTimeMarkerChanged.AddSP(this, &FSoundDashboardViewFactory::OnTimingViewTimeMarkerChanged);
		AudioInsightsModule.GetTimingViewExtender().OnTimeControlMethodReset.AddSP(this, &FSoundDashboardViewFactory::HandleOnTimeControlMethodReset);

		const TSharedRef<SWidget> SoundDashboardWidget = SNew(SVerticalBox)
#if WITH_EDITOR
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					MakeMuteSoloWidget()
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					MakeShowPlotWidget()
				]
				
				// Empty Spacing
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				// Settings button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					MakeSettingsButtonWidget()
				]
			]
#else
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					MakeShowPlotWidget()
				]
				// Empty Spacing
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				// Settings button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					MakeSettingsButtonWidget()
				]
			]
#endif // WITH_EDITOR
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				FTraceTreeDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs)
			];

		if (HeaderRowWidget.IsValid())
		{
			const FSoundDashboardVisibleColumns VisibleColumns = InitVisibleColumnSettings.IsSet() ? InitVisibleColumnSettings.GetValue() : FSoundDashboardVisibleColumns();
			VisibleColumnsSettingsMenu = MakeShared<FVisibleColumnsSettingsMenu<FSoundDashboardVisibleColumns>>(HeaderRowWidget.ToSharedRef(), VisibleColumns);

			VisibleColumnsSettingsMenu->OnVisibleColumnsSettingsUpdated.AddSPLambda(this, []()
			{
#if WITH_EDITOR
				FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR	
			});
		}

		return SoundDashboardWidget->AsShared();
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::GenerateWidgetForRootColumn(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& InColumn, const FText& InValueText)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const FColumnData& ColumnData = GetColumns()[InColumn];

		if (InColumn == NameColumnName)
		{
			const FName IconName = ColumnData.GetIconName.IsSet() ? ColumnData.GetIconName(InRowData.Get()) : NAME_None;

			return SNew(SHorizontalBox)
				.Visibility_Lambda([InRowWidget]()
				{
					return InRowWidget->GetVisibility();
				})
				// Tree expander arrow
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, InRowWidget)
				]
				// Icon
				+ SHorizontalBox::Slot()
				.Padding(IconName != NAME_None ? 2.0f : 0.0f, 2.0f)
				.AutoWidth()
				[
					IconName != NAME_None 
						? SNew(SImage)
						.ColorAndOpacity_Lambda([InRowData]() 
						{
							return GetIconColor(InRowData);
						})
						.Image(FSlateStyle::Get().GetBrush(IconName)) 
						: SNullWidget::NullWidget
				]
				// Text
				+ SHorizontalBox::Slot()
				.Padding(IconName != NAME_None ? 10.0f : 0.0f, 2.0f, 0.0f, 2.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font_Lambda([this, InRowData]()
					{
						const FSoundDashboardEntry& SoundDashboardEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*InRowData);
						return SoundDashboardEntry.bIsCategory ? FCoreStyle::Get().GetFontStyle("BoldFont") : FCoreStyle::Get().GetFontStyle("NormalFont");
					})
					.Text(InValueText)
					.ToolTipText(InValueText)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				// Number of children text
				+ SHorizontalBox::Slot()
				.Padding(6.0f, 2.0f, 0.0f, 2.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor(FColor::White.WithAlpha(128)))
					.Text_Lambda([this, InRowData]()
					{
						const uint32 TotalNumChildren = CountNumChildren(*InRowData, bShowRecentlyStoppedSounds);

						if (TreeViewingMode != ESoundDashboardTreeViewingOptions::FullTree && TotalNumChildren == 0)
						{
							return FText::GetEmpty();
						}

						return FText::FromString("(" + FString::FromInt(TotalNumChildren) + ")");
					})
				];
		}

		return SNullWidget::NullWidget;
	}

	bool FSoundDashboardViewFactory::IsRootItem(const TSharedRef<IDashboardDataTreeViewEntry>& InEntry) const
	{
		return FilteredEntriesListView.IsValid() && FilteredEntriesListView->GetRootItems().Contains(InEntry);
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::GenerateWidgetForColumn(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& InColumn)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const FColumnData& ColumnData = GetColumns()[InColumn];

		const FText ValueText  = ColumnData.GetDisplayValue.IsSet() ? ColumnData.GetDisplayValue(InRowData.Get()) : FText::GetEmpty();
		const FName& ValueIcon = ColumnData.GetIconName.IsSet() ? ColumnData.GetIconName(InRowData.Get()) : NAME_None;

		if (ValueText.IsEmpty() && ValueIcon.IsNone())
		{
			return SNullWidget::NullWidget;
		}

#if WITH_EDITOR
		if (InColumn == MuteColumnName)
		{
			return CreateMuteSoloButton(InRowWidget, InRowData, InColumn,
				[this](const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries)
				{
					ToggleMuteSoloEntries(InEntries, EMuteSoloMode::Mute);
				},
				[](const IDashboardDataTreeViewEntry& InEntry, const bool bInCheckChildren)
				{
					FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
#if ENABLE_AUDIO_DEBUG
					return AudioDeviceManager ? IsMuteSolo(AudioDeviceManager->GetDebugger(), InEntry, bInCheckChildren, EMuteSoloMode::Mute) : false;
#else
					return false;
#endif // ENABLE_AUDIO_DEBUG
				});
		}
		else if (InColumn == SoloColumnName)
		{
			return CreateMuteSoloButton(InRowWidget, InRowData, InColumn,
				[this](const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries)
				{
					ToggleMuteSoloEntries(InEntries, EMuteSoloMode::Solo);
				}, 
				[](const IDashboardDataTreeViewEntry& InEntry, const bool bInCheckChildren)
				{
					FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
#if ENABLE_AUDIO_DEBUG
					return AudioDeviceManager ? IsMuteSolo(AudioDeviceManager->GetDebugger(), InEntry, bInCheckChildren, EMuteSoloMode::Solo) : false;
#else
					return false;
#endif // ENABLE_AUDIO_DEBUG
				});
		}
		else
#endif // WITH_EDITOR
		if (InColumn == PlotColumnName)
		{
			return CreateShowPlotColumnButton(InRowWidget, InRowData, InColumn,
				[this](const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries, const bool bActivatePlot)
				{
					ToggleShowPlotEntries(InEntries, bActivatePlot);
				},
				[](const IDashboardDataTreeViewEntry& InEntry)
				{
					return CastEntry(InEntry).bIsPlotActive;
				});
		}
		else if (InColumn == NameColumnName)
		{
			if (IsRootItem(InRowData))
			{
				return GenerateWidgetForRootColumn(InRowWidget, InRowData, InColumn, ValueText);
			}

			return SNew(SHorizontalBox)
				.Visibility_Lambda([InRowWidget]()
				{
					return InRowWidget->GetVisibility();
				})
				// Tree expander arrow
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, InRowWidget)
				]
				// Icon
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 2.0f, 2.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this, InRowData]()
					{
						if (IsRootItem(InRowData))
						{
							return GetIconColor(InRowData);
						}
						return FSlateColor(FColor::White);
					})
					.Image(FSlateStyle::Get().GetBrush(ColumnData.GetIconName(InRowData.Get())))
				]
				// Text
				+ SHorizontalBox::Slot()
				.Padding(10.0f, 2.0f, 2.0f, 2.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(ValueText)
					.ToolTipText(ValueText)
				]
				// Number of children text (if it is not leaf item)
				+ SHorizontalBox::Slot()
				.Padding(6.0f, 2.0f, 0.0f, 2.0f)
				.AutoWidth()
				[
					!InRowData->Children.IsEmpty()
						? SNew(STextBlock)
						  .ColorAndOpacity(FSlateColor(FColor::White.WithAlpha(128)))
						  .Text_Lambda([this, InRowData]()
						  {
							  const uint32 TotalNumChildren = CountNumChildren(*InRowData, bShowRecentlyStoppedSounds);
							  return FText::FromString("(" + FString::FromInt(TotalNumChildren) + ")");
						  })
						: SNullWidget::NullWidget
				];
		}
		else
		{
			if (IsRootItem(InRowData) && TreeViewingMode == ESoundDashboardTreeViewingOptions::FullTree)
			{
				return SNullWidget::NullWidget;
			}

			return SNew(SHorizontalBox)
				.Visibility_Lambda([InRowWidget]()
				{
					return InRowWidget->GetVisibility();
				})
				// Text
				+ SHorizontalBox::Slot()
				.Padding(10.0f, 2.0f, 2.0f, 2.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([InRowData, ColumnData]()
					{
						return ColumnData.GetDisplayValue(InRowData.Get());
					})
					.ToolTipText_Lambda([InRowData, ColumnData]()
					{
						return ColumnData.GetDisplayValue(InRowData.Get());
					})
				];
		}
	}

	TSharedRef<ITableRow> FSoundDashboardViewFactory::OnGenerateRow(TSharedPtr<IDashboardDataTreeViewEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(*Item);

		return SNew(SRowWidget, OwnerTable, Item, AsShared())
			.RenderOpacity_Lambda([Item]
			{
				const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(*Item);
				return SoundDashboardEntry.TimeoutTimestamp == INVALID_TIMEOUT && !SoundDashboardEntry.bForceKeepEntryAlive ? 1.0f : 0.25f;
			})
			.Visibility_Lambda([this, Item]()
			{
				const FSoundDashboardEntry& SoundEntry = CastEntry(*Item);

				if (TreeViewingMode != ESoundDashboardTreeViewingOptions::FullTree && SoundEntry.bIsCategory && SoundEntry.EntryType != ESoundDashboardEntryType::Pinned)
				{
					return EVisibility::Collapsed;
				}

				if (SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::PinnedCopy)
				{
					if (!bIsPinnedCategoryFilterEnabled || (SoundEntry.bIsCategory && CountNumChildren(*Item, bShowRecentlyStoppedSounds, true /*bIncludeTimingOutSounds*/) == 0))
					{
						return EVisibility::Collapsed;
					}
				}

				const int32 NumUnpinnedChildren = GetNumChildrenWithoutPinEntryType(*Item, FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry, bShowRecentlyStoppedSounds);

				const bool bRowShouldBeVisible = SoundEntry.bIsVisible 
												&& ((bShowRecentlyStoppedSounds || SoundEntry.TimeoutTimestamp == INVALID_TIMEOUT) || SoundEntry.bForceKeepEntryAlive)
												&& SoundEntry.PinnedEntryType != FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry 
												&& (!IsCategoryItem(*Item) || (NumUnpinnedChildren > 0));

				return bRowShouldBeVisible ? EVisibility::Visible : EVisibility::Collapsed;
			});
	}

	void FSoundDashboardViewFactory::ProcessEntries(FTraceTreeDashboardViewFactory::EProcessReason InReason)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (bTreeViewModeChanged)
		{
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
			{
				FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
				RefreshInitExpandStateRecursive(SoundEntry);
			}
		}

		// Filter by category
		FTraceTreeDashboardViewFactory::FilterEntries<FSoundTraceProvider>([this](IDashboardDataTreeViewEntry& InEntry)
		{
			if (SelectedFilterFlags == AllFilterFlags)
			{
				return true;
			}

			FSoundDashboardEntry& SoundCategoryEntry = FSoundDashboardViewFactoryPrivate::CastEntry(InEntry);
			bool bEntryTypePassesFilter = false;

			switch (SoundCategoryEntry.EntryType)
			{
				case ESoundDashboardEntryType::MetaSound:
					bEntryTypePassesFilter = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::MetaSound);
					break;
				case ESoundDashboardEntryType::SoundCue:
					bEntryTypePassesFilter = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::SoundCue);
					break;
				case ESoundDashboardEntryType::ProceduralSource:
					bEntryTypePassesFilter = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::ProceduralSource);
					break;
				case ESoundDashboardEntryType::SoundWave:
					bEntryTypePassesFilter = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::SoundWave);
					break;
				case ESoundDashboardEntryType::SoundCueTemplate:
					bEntryTypePassesFilter = EnumHasAnyFlags(SelectedFilterFlags, ESoundDashboardFilterFlags::SoundCueTemplate);
					break;
				default:
					break;
			}

			if (!bEntryTypePassesFilter)
			{
				RefreshInitExpandStateRecursive(SoundCategoryEntry);
			}

			return bEntryTypePassesFilter;
		});

		if (InReason == FTraceTreeDashboardViewFactory::EProcessReason::EntriesUpdated)
		{
#if WITH_EDITOR
			const FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			if (AudioInsightsModule.GetTimingViewExtender().GetMessageCacheAndProcessingStatus() != ECacheAndProcess::Latest)
			{
				return;
			}
#endif // WITH_EDITOR

			PlotFilteredEntries();
			ProcessPlotData();
		}
	}

	void FSoundDashboardViewFactory::ChangeTreeViewingMode(const ESoundDashboardTreeViewingOptions SelectedMode, const bool bUpdateEditorSettings /*= true*/)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (TreeViewingMode == SelectedMode)
		{
			return;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
			RefreshInitExpandStateRecursive(SoundEntry);
		}

		TreeViewingMode = SelectedMode;
		UpdateFilterReason = EProcessReason::FilterUpdated;
		ClearSelection();

		bTreeViewModeChanged = true;

#if WITH_EDITOR
		if (bUpdateEditorSettings)
		{
			FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
		}
#endif
	}

	void FSoundDashboardViewFactory::BuildViewOptionsMenuContent(FMenuBuilder& MenuBuilder)
	{
		const FSoundDashboardCommands& Commands = FSoundDashboardCommands::Get();

		MenuBuilder.AddMenuEntry(Commands.GetViewFullTreeCommand());
		MenuBuilder.AddMenuEntry(Commands.GetViewActiveSoundsCommand());
		MenuBuilder.AddMenuEntry(Commands.GetViewFlatList());
	}

	void FSoundDashboardViewFactory::ChangeAutoExpandMode(const ESoundDashboardAutoExpandOptions SelectedMode, const bool bUpdateEditorSettings /*= true*/)
	{
		if (AutoExpandMode == SelectedMode)
		{
			return;
		}

		AutoExpandMode = SelectedMode;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			RefreshExpansionChecks(Entry);
		}

#if WITH_EDITOR
		if (bUpdateEditorSettings)
		{
			FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
		}
#endif
	}

	void FSoundDashboardViewFactory::RefreshExpansionChecks(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
		SoundEntry.bHasSetInitExpansion = false;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : Entry->Children)
		{
			RefreshExpansionChecks(Child);
		}
	}

	void FSoundDashboardViewFactory::BuildAutoExpandMenuContent(FMenuBuilder& MenuBuilder)
	{
		const FSoundDashboardCommands& Commands = FSoundDashboardCommands::Get();

		MenuBuilder.AddMenuEntry(Commands.GetAutoExpandCategoriesCommand());
		MenuBuilder.AddMenuEntry(Commands.GetAutoExpandEverythingCommand());
		MenuBuilder.AddMenuEntry(Commands.GetAutoExpandNothingCommand());
	}

	void FSoundDashboardViewFactory::BuildVisibleColumnsMenuContent(FMenuBuilder& MenuBuilder)
	{
		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->BuildVisibleColumnsMenuContent(MenuBuilder);
		}
	}

	void FSoundDashboardViewFactory::FilterText()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		// Filter by text
		const FString FilterString = GetSearchFilterText().ToString();
		const bool bFilterHasText  = !FilterString.IsEmpty();

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& CategoryEntry : DataViewEntries)
		{
			if (!CategoryEntry.IsValid())
			{
				continue;
			}

			ResetVisibility(*CategoryEntry);

			if (TreeViewingMode == ESoundDashboardTreeViewingOptions::FullTree)
			{
				for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : CategoryEntry->Children)
				{
					if (Entry.IsValid() && bFilterHasText)
					{
						SetFilteredVisibility(*Entry, FilterString);
					}
				}
			}
			else if (bFilterHasText)
			{
				SetFilteredVisibility(*CategoryEntry, FilterString);
			}
		}
	}

	TSharedPtr<SWidget> FSoundDashboardViewFactory::OnConstructContextMenu()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const FSoundDashboardCommands& Commands = FSoundDashboardCommands::Get();

		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.BeginSection("SoundDashboardActions", LOCTEXT("SoundDashboard_Actions_HeaderText", "Sound Options"));

		{
			MenuBuilder.AddMenuEntry(
				Commands.GetPinCommand(),
				NAME_None,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Pin"),
				NAME_None,
				TAttribute<EVisibility>::CreateLambda([this]() { return SelectionIncludesUnpinnedItem() ? EVisibility::Visible : EVisibility::Collapsed; })
			);
			
			MenuBuilder.AddMenuEntry(
				Commands.GetUnpinCommand(),
				NAME_None,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Pin"),
				NAME_None,
				TAttribute<EVisibility>::CreateLambda([this]() { return SelectionIncludesUnpinnedItem() ? EVisibility::Collapsed : EVisibility::Visible; })
			);

#if WITH_EDITOR
			MenuBuilder.AddMenuEntry(Commands.GetBrowseCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Browse"));
			MenuBuilder.AddMenuEntry(Commands.GetEditCommand(),   NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Edit"));
#endif // WITH_EDITOR
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void FSoundDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataTreeViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		if (!FilteredEntriesListView.IsValid())
		{
			return;
		}

		OnUpdatePlotSelection.Broadcast(FilteredEntriesListView->GetSelectedItems());

#if WITH_EDITOR
		FAudioInsightsDetailsSelectionManager& SelectionManager = IAudioInsightsModule::GetChecked().GetDetailsSelectionManager();
		const TArray<TObjectPtr<UObject>> Assets = GetSelectedEditableAssets();
		if (Assets.Num() > 0)
		{
			SelectionManager.SetSelectedAsset(Assets[0]);
		}
		else if (SelectInfo != ESelectInfo::Type::Direct)
		{
			SelectionManager.ClearSelection();
		}
#endif // WITH_EDITOR
	}

	FReply FSoundDashboardViewFactory::OnDataRowKeyInput(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const
	{
		return (CommandList && CommandList->ProcessCommandBindings(InKeyEvent)) ? FReply::Handled() : FReply::Unhandled();
	}

	const TMap<FName, FTraceTreeDashboardViewFactory::FHeaderRowColumnData>& FSoundDashboardViewFactory::GetHeaderRowColumns() const
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		FSoundDashboardVisibleColumns VisibleColumns = InitVisibleColumnSettings.IsSet() ? InitVisibleColumnSettings.GetValue() : FSoundDashboardVisibleColumns();

		static const TMap<FName, FTraceTreeDashboardViewFactory::FHeaderRowColumnData> HeaderRowColumnData =
		{
#if WITH_EDITOR
			{
				MuteColumnName,
				{
					.DisplayName	  = MuteColumnDisplayName,
					.IconName		  = "AudioInsights.Icon.SoundDashboard.Mute",
					.TooltipText	  = MuteColumnTooltip,
					.bShowDisplayName = false,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(MuteColumnName),
					.FixedWidth       = 30.0f,
					.Alignment        = EHorizontalAlignment::HAlign_Center
				}
			},
			{
				SoloColumnName,
				{
					.DisplayName      = SoloColumnDisplayName,
					.IconName         = "AudioInsights.Icon.SoundDashboard.Solo",
					.TooltipText	  = SoloColumnTooltip,
					.bShowDisplayName = false,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(SoloColumnName),
					.FixedWidth       = 30.0f,
					.Alignment        = EHorizontalAlignment::HAlign_Center
				}
			},
#endif
			{
				PlotColumnName,
				{
					.DisplayName	  = PlotColumnDisplayName,
					.IconName		  = "AudioInsights.Icon.SoundDashboard.Plots",
					.TooltipText	  = PlotColumnTooltip,
					.bShowDisplayName = false,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(PlotColumnName),
					.FixedWidth		  = 30.0f,
					.Alignment		  = EHorizontalAlignment::HAlign_Center
				}
			},
			{
				NameColumnName,
				{
					.DisplayName      = NameColumnDisplayName,
					.IconName         = NAME_None,
					.TooltipText	  = NameColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(NameColumnName),
					.FillWidth        = 0.5f,
					.Alignment        = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				PlayOrderColumnName,
				{
					.DisplayName	  = PlayOrderColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = PlayOrderColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(PlayOrderColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				PriorityColumnName,
				{
					.DisplayName	  = PriorityColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = PriorityColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden	  = !VisibleColumns.GetIsVisible(PriorityColumnName),
					.FillWidth		  = 0.08f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				DistanceColumnName,
				{
					.DisplayName      = DistanceColumnDisplayName,
					.IconName         = NAME_None,
					.TooltipText	  = DistanceColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(DistanceColumnName),
					.FillWidth        = 0.1f,
					.Alignment        = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				DistanceAttenuationColumnName,
				{
					.DisplayName	  = DistanceAttenuationColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = DistanceAttenuationColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(DistanceAttenuationColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				AmplitudeColumnName,
				{
					.DisplayName      = AmplitudeColumnDisplayName,
					.IconName         = NAME_None,
					.TooltipText	  = AmplitudeColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(AmplitudeColumnName),
					.FillWidth        = 0.12f,
					.Alignment        = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				VolumeColumnName,
				{
					.DisplayName      = VolumeColumnDisplayName,
					.IconName         = NAME_None,
					.TooltipText	  = VolumeColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(VolumeColumnName),
					.FillWidth        = 0.1f,
					.Alignment        = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				LPFFreqColumnName,
				{
					.DisplayName	  = LPFFreqColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = LPFFreqColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(LPFFreqColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				HPFFreqColumnName,
				{
					.DisplayName	  = HPFFreqColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = HPFFreqColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(HPFFreqColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				PitchColumnName,
				{
					.DisplayName      = PitchColumnDisplayName,
					.IconName         = NAME_None,
					.TooltipText	  = PitchColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(PitchColumnName),
					.FillWidth        = 0.1f,
					.Alignment        = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				RelativeRenderCostColumnName,
				{
					.DisplayName	  = RelativeRenderCostColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = RelativeRenderCostColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(RelativeRenderCostColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				ActorLabelColumnName,
				{
					.DisplayName	  = ActorLabelColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = ActorLabelColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(ActorLabelColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				AudioComponentNameColumnName,
				{
					.DisplayName	  = AudioComponentNameColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = AudioComponentNameColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(AudioComponentNameColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			},
			{
				CategoryColumnName,
				{
					.DisplayName	  = CategoryColumnDisplayName,
					.IconName		  = NAME_None,
					.TooltipText	  = CategoryColumnTooltip,
					.bShowDisplayName = true,
					.bDefaultHidden   = !VisibleColumns.GetIsVisible(CategoryColumnName),
					.FillWidth		  = 0.1f,
					.Alignment		  = EHorizontalAlignment::HAlign_Left
				}
			}
		};

		return HeaderRowColumnData;
	}

	void FSoundDashboardViewFactory::OnHiddenColumnsListChanged()
	{
		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->OnHiddenColumnsListChanged();
		}
	}

	const TMap<FName, FTraceTreeDashboardViewFactory::FColumnData>& FSoundDashboardViewFactory::GetColumns() const
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		static const TMap<FName, FTraceTreeDashboardViewFactory::FColumnData> ColumnData =
		{
#if WITH_EDITOR
			{
				MuteColumnName,
				{
					.GetIconName = [](const IDashboardDataTreeViewEntry& InData)
					{
						return "AudioInsights.Icon.SoundDashboard.Mute";
					}
				}
			},
			{
				SoloColumnName,
				{
					.GetIconName = [](const IDashboardDataTreeViewEntry& InData)
					{
						return "AudioInsights.Icon.SoundDashboard.Solo";
					}
				}
			},
#endif
			{
				PlotColumnName,
				{
					.GetIconName = [this](const IDashboardDataTreeViewEntry& InData)
					{
						return "AudioInsights.Icon.SoundDashboard.Plots";
					}
				}
			},
			{
				NameColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);

						return SoundDashboardEntry.GetDisplayName();
					},
					.GetIconName = [](const IDashboardDataTreeViewEntry& InData) -> FName
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);

						switch (SoundDashboardEntry.EntryType)
						{
							case ESoundDashboardEntryType::MetaSound:
								return FName("AudioInsights.Icon.SoundDashboard.MetaSound");
							case ESoundDashboardEntryType::SoundCue:
								return FName("AudioInsights.Icon.SoundDashboard.SoundCue");
							case ESoundDashboardEntryType::ProceduralSource:
								return FName("AudioInsights.Icon.SoundDashboard.ProceduralSource");
							case ESoundDashboardEntryType::SoundWave:
								return FName("AudioInsights.Icon.SoundDashboard.SoundWave");
							case ESoundDashboardEntryType::SoundCueTemplate:
								return FName("AudioInsights.Icon.SoundDashboard.SoundCue");
							case ESoundDashboardEntryType::Pinned:
								return FName("AudioInsights.Icon.SoundDashboard.Pin");
							case ESoundDashboardEntryType::None:
							default:
								break;
						}

						return NAME_None;
					}
				}
			},
			{
				PlayOrderColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						if (SoundDashboardEntry.WaveInstancePlayOrder == INDEX_NONE)
						{
							return FText::AsNumber(SoundDashboardEntry.ActiveSoundPlayOrder);
						}

						return FText::Format(LOCTEXT("AudioDashboard_Sounds_WaveInstancePlayOrderFormat", "{0}, {1}"), SoundDashboardEntry.ActiveSoundPlayOrder, SoundDashboardEntry.WaveInstancePlayOrder);
					}
				}
			},
			{
				PriorityColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						const float PriorityValue = SoundDashboardEntry.PriorityDataPoint;

						return SoundMessageUtils::IsPriorityValueSetToMax(PriorityValue)
							? LOCTEXT("AudioDashboard_Sounds_Max", "MAX") 
							: FText::AsNumber(PriorityValue, FSlateStyle::Get().GetLinearVolumeFloatFormat());
					}
				}
			},
			{
				DistanceColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.DistanceDataPoint, FSlateStyle::Get().GetDistanceFloatFormat());
					}
				}
			},
			{
				DistanceAttenuationColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.DistanceAttenuationDataPoint, FSlateStyle::Get().GetLinearVolumeFloatFormat());
					}
				}
			},
			{
				AmplitudeColumnName,
				{
					.GetDisplayValue = [this](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						const float AmplitudeValue = SoundDashboardEntry.AmplitudeDataPoint;

						return bDisplayAmpPeakInDb 
							? AudioInsightsUtils::ConvertToDecibelsText(AmplitudeValue) 
							: FText::AsNumber(AmplitudeValue, FSlateStyle::Get().GetLinearVolumeFloatFormat());
					}
				}
			},
			{
				VolumeColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.VolumeDataPoint, FSlateStyle::Get().GetLinearVolumeFloatFormat());
					}
				}
			},
			{
				LPFFreqColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.LPFFreqDataPoint, FSlateStyle::Get().GetFreqFloatFormat());
					}
				}
			},
			{
				HPFFreqColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.HPFFreqDataPoint, FSlateStyle::Get().GetFreqFloatFormat());
					}
				}
			},
			{
				PitchColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.PitchDataPoint, FSlateStyle::Get().GetPitchFloatFormat());
					}
				}
			},
			{
				RelativeRenderCostColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);
						if (SoundDashboardEntry.bIsCategory)
						{
							return FText::GetEmpty();
						}

						return FText::AsNumber(SoundDashboardEntry.RelativeRenderCostDataPoint, FSlateStyle::Get().GetRelativeRenderCostFormat());
					}
				}
			},
			{
				ActorLabelColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);

						return SoundDashboardEntry.ActorLabel;
					}
				}
			},
			{
				AudioComponentNameColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);

						return SoundDashboardEntry.AudioComponentName;
					}
				}
			},
			{
				CategoryColumnName,
				{
					.GetDisplayValue = [](const IDashboardDataTreeViewEntry& InData)
					{
						const FSoundDashboardEntry& SoundDashboardEntry = CastEntry(InData);

						return SoundDashboardEntry.CategoryName;
					}
				}
			}
		};

		return ColumnData;
	}

	void FSoundDashboardViewFactory::SortTable()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		FilterText();

		UpdatePinnedSection();

		auto SortByPlayOrder = [](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			if (First.ActiveSoundPlayOrder == Second.ActiveSoundPlayOrder)
			{
				return First.WaveInstancePlayOrder < Second.WaveInstancePlayOrder;
			}

			return First.ActiveSoundPlayOrder < Second.ActiveSoundPlayOrder;
		};
		
		auto SortByName = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const int32 Comparison = First.GetDisplayName().CompareToCaseIgnored(Second.GetDisplayName());
			if (Comparison == 0)
			{
				return SortByPlayOrder(First, Second);
			}
			
			return Comparison < 0;
		};

		auto SortByPriority = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.PriorityDataPoint - Second.PriorityDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByDistance = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.DistanceDataPoint- Second.DistanceDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByDistanceAttenuation = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.DistanceAttenuationDataPoint - Second.DistanceAttenuationDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByLPFFreq = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.LPFFreqDataPoint - Second.LPFFreqDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByHPFFreq = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.HPFFreqDataPoint - Second.HPFFreqDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByAmplitude = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.AmplitudeDataPoint - Second.AmplitudeDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByVolume = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.VolumeDataPoint - Second.VolumeDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByPitch = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.PitchDataPoint - Second.PitchDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByRelativeRenderCost = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const float ComparisonDiff = First.RelativeRenderCostDataPoint - Second.RelativeRenderCostDataPoint;
			if (FMath::IsNearlyZero(ComparisonDiff, UE_KINDA_SMALL_NUMBER))
			{
				return SortByPlayOrder(First, Second);
			}

			return ComparisonDiff < 0.0f;
		};

		auto SortByActor = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const int32 Comparison = First.ActorLabel.CompareToCaseIgnored(Second.ActorLabel);
			if (Comparison == 0)
			{
				return SortByPlayOrder(First, Second);
			}
			
			return Comparison < 0;
		};

		auto SortByAudioComponentName = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const int32 Comparison = First.AudioComponentName.CompareToCaseIgnored(Second.AudioComponentName);
			if (Comparison == 0)
			{
				return SortByPlayOrder(First, Second);
			}

			return Comparison < 0;
		};

		auto SortByCategory = [&SortByPlayOrder](const FSoundDashboardEntry& First, const FSoundDashboardEntry& Second)
		{
			const int32 Comparison = First.CategoryName.CompareToCaseIgnored(Second.CategoryName);
			if (Comparison == 0)
			{
				return SortByPlayOrder(First, Second);
			}
			
			return Comparison < 0;
		};

		if (SortByColumn == NameColumnName)
		{
			SortByPredicate(SortByName);
		}
		else if (SortByColumn == PlayOrderColumnName)
		{
			SortByPredicate(SortByPlayOrder);
		}
		else if (SortByColumn == PriorityColumnName)
		{
			SortByPredicate(SortByPriority);
		}
		else if (SortByColumn == DistanceColumnName)
		{
			SortByPredicate(SortByDistance);
		}
		else if (SortByColumn == DistanceAttenuationColumnName)
		{
			SortByPredicate(SortByDistanceAttenuation);
		}
		else if (SortByColumn == AmplitudeColumnName)
		{
			SortByPredicate(SortByAmplitude);
		}
		else if (SortByColumn == VolumeColumnName)
		{
			SortByPredicate(SortByVolume);
		}
		else if (SortByColumn == LPFFreqColumnName)
		{
			SortByPredicate(SortByLPFFreq);
		}
		else if (SortByColumn == HPFFreqColumnName)
		{
			SortByPredicate(SortByHPFFreq);
		}
		else if (SortByColumn == PitchColumnName)
		{
			SortByPredicate(SortByPitch);
		}
		else if (SortByColumn == RelativeRenderCostColumnName)
		{
			SortByPredicate(SortByRelativeRenderCost);
		}
		else if (SortByColumn == ActorLabelColumnName)
		{
			SortByPredicate(SortByActor);
		}
		else if (SortByColumn == AudioComponentNameColumnName)
		{
			SortByPredicate(SortByAudioComponentName);
		}
		else if (SortByColumn == CategoryColumnName)
		{
			SortByPredicate(SortByCategory);
		}

		FullTree.Reset();

		if (PinnedItemEntries.IsValid())
		{
			FullTree.Add(PinnedItemEntries->GetPinnedSectionEntry());
		}
		
		FullTree.Append(DataViewEntries);

		if (bTreeViewModeChanged)
		{
			FilteredEntriesListView->RebuildList();
			bTreeViewModeChanged = false;
		}
	}

	bool FSoundDashboardViewFactory::IsColumnSortable(const FName& ColumnId) const
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		return ColumnId != SoloColumnName
			&& ColumnId != MuteColumnName
			&& ColumnId != PlotColumnName;
	}

	bool FSoundDashboardViewFactory::ResetTreeData()
	{
		bool bDataReset = false;
		if (!DataViewEntries.IsEmpty())
		{
			DataViewEntries.Empty();
			bDataReset = true;
		}

		if (PinnedItemEntries.IsValid())
		{
			PinnedItemEntries.Reset();
			bDataReset = true;
		}

		if (!FullTree.IsEmpty())
		{
			FullTree.Empty();
			bDataReset = true;
		}

		return bDataReset;
	}

	bool FSoundDashboardViewFactory::ShouldAutoExpand(const TSharedPtr<IDashboardDataTreeViewEntry>& Item) const
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!Item.IsValid())
		{
			return false;
		}

		switch (AutoExpandMode)
		{
			case ESoundDashboardAutoExpandOptions::Everything:
				return true;
			case ESoundDashboardAutoExpandOptions::Nothing:
				return false;
			case ESoundDashboardAutoExpandOptions::Categories:
			default:
				FSoundDashboardEntry& Entry = CastEntry(*Item);
				return Entry.bIsCategory;
		}
	}

	void FSoundDashboardViewFactory::RecursiveSort(TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& OutTree, TFunctionRef<bool(const FSoundDashboardEntry&, const FSoundDashboardEntry&)> Predicate)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		for (TSharedPtr<IDashboardDataTreeViewEntry>& Entry : OutTree)
		{
			if (Entry->Children.Num() > 0)
			{
				RecursiveSort(Entry->Children, Predicate);
			}
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : OutTree)
		{
			const FSoundDashboardEntry& EntryData = CastEntry(*Entry);
			if (IsCategoryItem(*Entry))
			{
				return;
			}
		}

		auto SortDashboardEntries = [this](const TSharedPtr<IDashboardDataTreeViewEntry>& First, const TSharedPtr<IDashboardDataTreeViewEntry>& Second, TFunctionRef<bool(const FSoundDashboardEntry&, const FSoundDashboardEntry&)> Predicate)
		{
			return Predicate(CastEntry(*First), CastEntry(*Second));
		};

		if (SortMode == EColumnSortMode::Ascending)
		{
			OutTree.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataTreeViewEntry>& A, const TSharedPtr<IDashboardDataTreeViewEntry>& B)
			{
				return SortDashboardEntries(A, B, Predicate);
			});
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			OutTree.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataTreeViewEntry>& A, const TSharedPtr<IDashboardDataTreeViewEntry>& B)
			{
				return SortDashboardEntries(B, A, Predicate);
			});
		}
	}

	void FSoundDashboardViewFactory::SortByPredicate(TFunctionRef<bool(const FSoundDashboardEntry&, const FSoundDashboardEntry&)> Predicate)
	{
		if (PinnedItemEntries.IsValid())
		{
			RecursiveSort(PinnedItemEntries->GetPinnedSectionEntry()->Children, Predicate);
		}

		RecursiveSort(DataViewEntries, Predicate);
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::OnGetSettingsMenuContent()
	{
		FMenuBuilder MenuBuilder(false /*bShouldCloseWindowAfterMenuSelection*/, CommandList);

		MenuBuilder.BeginSection("SoundDashboardSettingActions", LOCTEXT("SoundDashboard_SettingActions_HeaderText", "Settings"));

		{
			// Amp (Peak) Display Mode Display Mode toggle
			MenuBuilder.AddMenuEntry(
				FUIAction(FExecuteAction::CreateLambda([this]()
				{
					bDisplayAmpPeakInDb = !bDisplayAmpPeakInDb;
#if WITH_EDITOR
					FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
				})),
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0, 4.0, 5.0, 0.0))
					.Text(LOCTEXT("SoundsDashboard_AmpPeakDisplayMode_Text", "Amp (Peak) Display Mode"))
				]
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				[
					SNew(SSegmentedControl<bool>)
					.OnValueChanged_Lambda([this](bool bInValue)
					{
						bDisplayAmpPeakInDb = bInValue;
#if WITH_EDITOR
						FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
					})
					.Value_Lambda([this]
					{
						return bDisplayAmpPeakInDb;
					})
					// Decibels mode
					+SSegmentedControl<bool>::Slot(true)
					.Text(LOCTEXT("SoundsDashboard_AmpDisplayMode_dB_Text", "dB"))
					.ToolTip(LOCTEXT("SoundsDashboard_AmpDisplayMode_dB_ToolTipText", "Displays amplitude values in decibels."))
					// Linear mode
					+ SSegmentedControl<bool>::Slot(false)
					.Text(LOCTEXT("SoundsDashboard_AmpDisplayMode_Linear_Text", "Lin"))
					.ToolTip(LOCTEXT("SoundsDashboard_AmpDisplayMode_Linear_TooltipText", "Displays amplitude values in linear scale."))
				],
				NAME_None /*InExtensionHook*/,
				LOCTEXT("SoundsDashboard_AmpPeakDisplayMode_ToolTipText", "Switches the amplitude display mode between decibels or linear scale."),
				EUserInterfaceActionType::CollapsedButton
			);

			// View options
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("SoundsDashboard_ViewOptions", "View"),
				LOCTEXT("SoundsDashboard_ViewOptionsTooltip", "Choose how items in the Sounds Dashboard is displayed."),
				FNewMenuDelegate::CreateRaw(this, &FSoundDashboardViewFactory::BuildViewOptionsMenuContent),
				false
			);

			// Auto-Expand options
			MenuBuilder.AddSubMenu
			(
				LOCTEXT("SoundsDashboard_AutoExpand", "Auto-Expand"),
				LOCTEXT("SoundsDashboard_AutoExpandTooltip", "Choose whether new items in the Sounds Dashboard are automatically expanded or not."),
				FNewMenuDelegate::CreateRaw(this, &FSoundDashboardViewFactory::BuildAutoExpandMenuContent),
				false
			);

			// Visible columns
			MenuBuilder.AddSubMenu(
				LOCTEXT("SoundsDashboard_VisibleColumns", "Visible Columns"),
				LOCTEXT("SoundsDashboard_VisibleColumnsTooltip", "Show/hide columns"),
				FNewMenuDelegate::CreateRaw(this, &FSoundDashboardViewFactory::BuildVisibleColumnsMenuContent)
			);

			// Show Stopped Sounds checkbox
			MenuBuilder.AddMenuEntry(
				FUIAction(FExecuteAction::CreateLambda([this]()
				{
					bShowRecentlyStoppedSounds = !bShowRecentlyStoppedSounds;
#if WITH_EDITOR
					FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
#endif // WITH_EDITOR
				})),
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SoundsDashboard_ShowStoppedSounds", "Show Stopped Sounds"))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(27.0, 0.0f, 0.0f, 0.0f)
				[
					SNew(SImage)
				 	.Image_Lambda([this]()
				 	{
				 		const FName IconName = bShowRecentlyStoppedSounds ? "AudioInsights.Icon.SoundDashboard.Visible" : "AudioInsights.Icon.SoundDashboard.Invisible";
		 
				 		return FSlateStyle::Get().GetBrush(IconName);
				 	})
				],
				NAME_None /*InExtensionHook*/,
				LOCTEXT("SoundsDashboard_ShowStoppedSoundsTooltip", "Shows sounds that have recently stopped playing"),
				EUserInterfaceActionType::CollapsedButton
			);

			// Recently Stopped Sound Timeout
			MenuBuilder.AddMenuEntry(
				FUIAction(),
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0, 4.0, 5.0, 0.0))
					.Text(LOCTEXT("SoundsDashboard_RecentlyStoppedSoundTimeout", "Recently Stopped Timeout"))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SNumericEntryBox<float>)
					.MinDesiredValueWidth(30)
					.MinValue(0.0f)
					.MaxValue(TOptional<float>())
					.MaxSliderValue(TOptional<float>())
					.MaxFractionalDigits(2)
					.AllowWheel(true)
					.WheelStep(0.1f)
					.AllowSpin(true)
					.BroadcastValueChangesPerKey(true)
					.Value_Lambda([this]()
					{
						return RecentlyStoppedSoundTimeout;
					})
					.OnValueChanged_Lambda([this](float Value)
					{
						RecentlyStoppedSoundTimeout = Value;
						UpdateMaxStoppedSoundTimeout();

#if !WITH_EDITOR
						const TSharedPtr<FSoundTraceProvider> Provider = FindProvider<FSoundTraceProvider>();
						if (Provider.IsValid())
						{
							Provider->SetDashboardTimeoutTime(RecentlyStoppedSoundTimeout);
						}
#endif // !WITH_EDITOR
					})
#if WITH_EDITOR
					.OnValueCommitted_Lambda([](float /*Value*/, ETextCommit::Type)
					{
						// Persist the in-memory value (already updated per keystroke) to .ini on commit.
						FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
					})
#endif // WITH_EDITOR
				],
				NAME_None /*InExtensionHook*/,
				LOCTEXT("SoundsDashboard_RecentlyStoppedSoundTimeoutTooltip", "How long recently stopped sounds stay greyed out in the Sounds Dashboard before being removed (in seconds)")
			);
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::MakeSettingsButtonWidget()
	{
		return SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &FSoundDashboardViewFactory::OnGetSettingsMenuContent)
#if WITH_EDITOR
			.OnMenuOpenChanged_Lambda([](bool bIsOpen)
			{
				// Flush in-memory changes to .ini when the menu closes (covers click-outside where commit may not fire).
				if (!bIsOpen)
				{
					FSoundDashboardSettings::OnRequestWriteSettings.Broadcast();
				}
			})
#endif // WITH_EDITOR
			.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
			.HasDownArrow(false)
			.ToolTipText(LOCTEXT("SoundsDashboard_Settings_TooltipText", "Sound dashboard settings"))
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Settings"))
				]
			];
	}

	void FSoundDashboardViewFactory::UpdateMaxStoppedSoundTimeout()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		const double NewTimeoutTimestamp = FPlatformTime::Seconds() + static_cast<double>(RecentlyStoppedSoundTimeout);

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			ClampMaxTimeoutRecursive(Entry, NewTimeoutTimestamp);
		}
	}

#if WITH_EDITOR
	void FSoundDashboardViewFactory::OnReadEditorSettings(const FSoundDashboardSettings& InSettings)
	{
		ChangeTreeViewingMode(InSettings.TreeViewMode, false /*bUpdateEditorSettings*/);
		ChangeAutoExpandMode(InSettings.AutoExpandMode, false /*bUpdateEditorSettings*/);

		bShowRecentlyStoppedSounds = InSettings.bShowStoppedSounds;
		bDisplayAmpPeakInDb = InSettings.AmplitudeDisplayMode == EAudioAmplitudeDisplayMode::Decibels;

		// If VisibleColumnsSettingsMenu is already created, update it with the desired visible columns:
		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->ReadFromSettings(InSettings.VisibleColumns);
		}

		if (!InitVisibleColumnSettings.IsSet())
		{
			InitVisibleColumnSettings = InSettings.VisibleColumns;
		}

		DefaultPlotRanges = InSettings.DefaultPlotRanges;

		if (RecentlyStoppedSoundTimeout != InSettings.StoppedSoundTimeoutTime)
		{
			RecentlyStoppedSoundTimeout = InSettings.StoppedSoundTimeoutTime;

			UpdateMaxStoppedSoundTimeout();
		}
	}

	void FSoundDashboardViewFactory::OnWriteEditorSettings(FSoundDashboardSettings& OutSettings)
	{
		OutSettings.TreeViewMode = TreeViewingMode;
		OutSettings.AutoExpandMode = AutoExpandMode;
		OutSettings.bShowStoppedSounds = bShowRecentlyStoppedSounds;
		OutSettings.AmplitudeDisplayMode = bDisplayAmpPeakInDb ? EAudioAmplitudeDisplayMode::Decibels : EAudioAmplitudeDisplayMode::Linear;

		if (VisibleColumnsSettingsMenu.IsValid())
		{
			VisibleColumnsSettingsMenu->WriteToSettings(OutSettings.VisibleColumns);
		}

		OutSettings.DefaultPlotRanges = DefaultPlotRanges;

		if (OutSettings.StoppedSoundTimeoutTime != RecentlyStoppedSoundTimeout)
		{
			OutSettings.StoppedSoundTimeoutTime = RecentlyStoppedSoundTimeout;
			
			const TSharedPtr<FSoundTraceProvider> Provider = FindProvider<FSoundTraceProvider>();
			if (Provider.IsValid())
			{
				Provider->SetDashboardTimeoutTime(RecentlyStoppedSoundTimeout);
			}
		}
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::CreateMuteSoloButton(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, 
		const TSharedRef<IDashboardDataTreeViewEntry>& InRowData,
		const FName& InColumn, 
		TFunction<void(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>&)> MuteSoloToggleFunc, 
		TFunctionRef<bool(const IDashboardDataTreeViewEntry&, const bool)> IsMuteSoloFunc)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		return SNew(SHorizontalBox)
			.Visibility_Lambda([this, InRowWidget, InRowData]()
			{
				return InRowWidget->GetVisibility();
			})
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SoundDashboard.MuteSoloButton"))
				.IsChecked(ECheckBoxState::Unchecked)
				.OnCheckStateChanged_Lambda([InRowData, MuteSoloToggleFunc](ECheckBoxState NewState)
				{
					MuteSoloToggleFunc({ InRowData });
				})
				[
					SNew(SImage)
					.Image_Lambda([this, InRowData, InColumn, IsMuteSoloFunc]()
					{
						const FColumnData& ColumnData = GetColumns()[InColumn];
						const FName IconName = IsMuteSoloFunc(*InRowData, EntryCanHaveChildren(InRowData) /*bInCheckChildren*/) ? ColumnData.GetIconName(InRowData.Get()) : "AudioInsights.Icon.SoundDashboard.Transparent";

						return FSlateStyle::Get().GetBrush(IconName);
					})
				]
			];
	}

	void FSoundDashboardViewFactory::ToggleMuteSoloEntries(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries, const EMuteSoloMode InMuteSoloMode)
	{
#if ENABLE_AUDIO_DEBUG
		using namespace FSoundDashboardViewFactoryPrivate;

		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (AudioDeviceManager == nullptr)
		{
			return;
		}

		::Audio::FAudioDebugger& AudioDebugger = AudioDeviceManager->GetDebugger();

		TArray<TSharedPtr<IDashboardDataTreeViewEntry>> EntriesToMuteSolo;

		// In multiple selection we need to discard children entries to avoid double mute/solo toggling
		if (InEntries.Num() > 1)
		{
			EntriesToMuteSolo.Reserve(InEntries.Num());

			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : InEntries)
			{
				if (!Entry.IsValid())
				{
					continue;
				}

				bool bIsTopLevelEntry = true;

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& Other : InEntries)
				{
					if (Other != Entry && IsDescendant(Other, Entry))
					{
						bIsTopLevelEntry = false;
						break;
					}
				}

				if (bIsTopLevelEntry)
				{
					EntriesToMuteSolo.Add(Entry);
				}
			}
		}
		else
		{
			EntriesToMuteSolo = InEntries;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : EntriesToMuteSolo)
		{
			if (!Entry.IsValid())
			{
				continue;
			}

			if (EntryCanHaveChildren(Entry.ToSharedRef()))
			{
				const bool bAreChildrenMuteSolo = FSoundDashboardViewFactoryPrivate::IsMuteSolo(AudioDebugger, *Entry, true /*bInCheckChildren*/, InMuteSoloMode);

				for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
				{
					if (ChildEntry.IsValid())
					{
						FSoundDashboardViewFactoryPrivate::SetMuteSolo(*ChildEntry, InMuteSoloMode, !bAreChildrenMuteSolo);
					}
				}
			}
			else
			{
				FSoundDashboardViewFactoryPrivate::ToggleMuteSolo(*Entry, InMuteSoloMode);
			}
		}
#endif // ENABLE_AUDIO_DEBUG
	}

	void FSoundDashboardViewFactory::MuteSoloFilteredEntries()
	{
#if ENABLE_AUDIO_DEBUG
		using namespace FSoundDashboardViewFactoryPrivate;

		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (AudioDeviceManager == nullptr)
		{
			return;
		}

		ClearMutesAndSolos();

		if (!bIsMuteFilteredMode && !bIsSoloFilteredMode)
		{
			return;
		}

		const FText& FilterText = GetSearchFilterText();
		if (!FilterText.IsEmpty())
		{
			const EMuteSoloMode Mode = bIsSoloFilteredMode ? EMuteSoloMode::Solo : EMuteSoloMode::Mute;
			::Audio::FAudioDebugger& AudioDebugger = AudioDeviceManager->GetDebugger();

			if (Mode == EMuteSoloMode::Solo)
			{
				AudioDebugger.SetSoloSoundWave(FName(FilterText.ToString()), true);
			}
			else
			{
				AudioDebugger.SetMuteSoundWave(FName(FilterText.ToString()), true);
			}
		}
#endif //ENABLE_AUDIO_DEBUG
	}

	TArray<TObjectPtr<UObject>> FSoundDashboardViewFactory::GetSelectedEditableAssets() const
	{
		TArray<TObjectPtr<UObject>> Objects;

		if (!FilteredEntriesListView.IsValid())
		{
			return Objects;
		}

		const TArray<TSharedPtr<IDashboardDataTreeViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

		Algo::TransformIf(SelectedItems, Objects,
			[](const TSharedPtr<IDashboardDataTreeViewEntry>& SelectedItem)
			{
				if (SelectedItem.IsValid())
				{
					IObjectTreeDashboardEntry& RowData = *StaticCastSharedPtr<IObjectTreeDashboardEntry>(SelectedItem).Get();

					if (TObjectPtr<UObject> Object = RowData.GetObject())
					{
						return Object->IsAsset();
					}
				}

				return false;
			},
			[](const TSharedPtr<IDashboardDataTreeViewEntry>& SelectedItem) -> TObjectPtr<UObject>
			{
				if (SelectedItem.IsValid())
				{
					IObjectTreeDashboardEntry& RowData = *StaticCastSharedPtr<IObjectTreeDashboardEntry>(SelectedItem).Get();
					return RowData.GetObject();
				}

				return nullptr;
			}
		);

		return Objects;
	}
#endif // WITH_EDITOR

	void FSoundDashboardViewFactory::OnTimingViewTimeMarkerChanged(double InTimeMarker)
	{
		OnUpdatePlotVisibility.Broadcast(DataViewEntries);
	}

	void FSoundDashboardViewFactory::HandleOnTimeControlMethodReset()
	{
		bPinnedSectionNeedsRebuild = true;
	}

	void FSoundDashboardViewFactory::PlotFilteredEntries()
	{
		if (!bIsPlotFilteredMode)
		{
			return;
		}

		const FText& FilterText = GetSearchFilterText();
		if (FilterText.IsEmpty())
		{
			return; // Intentionally don't plot everything when the filter is empty
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			SetShowPlotVisibleRecursive(Entry); // Plot what is visible via filter
		}

		BroadcastPlotUpdate();
	}

	void FSoundDashboardViewFactory::ResetPlots()
	{
		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			SetShowPlotRecursive(Entry, false);
		}

		BroadcastPlotUpdate();
	}

	void FSoundDashboardViewFactory::BroadcastPlotUpdate()
	{
		if (FilteredEntriesListView.IsValid())
		{
			OnProcessPlotData.Broadcast(DataViewEntries, FilteredEntriesListView->GetSelectedItems(), true /*bForceUpdate*/);
		}

		OnUpdatePlotVisibility.Broadcast(DataViewEntries);
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::MakeShowPlotWidget()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		return SNew(SHorizontalBox)
			// Toggle Selected Plots button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundDashboard_SelectedPlotsToggleText", "Toggles whether plots are enabled on the selected items."))
				.OnClicked_Lambda([this]()
				{
					if (FilteredEntriesListView.IsValid())
					{
						bool bToggleOn = false;
						for (const TSharedPtr<IDashboardDataTreeViewEntry>& SelectedEntry : FilteredEntriesListView->GetSelectedItems())
						{
							if (!SelectedEntry.IsValid())
							{
								continue;
							}

							const FSoundDashboardEntry& SoundEntry = CastEntry(*SelectedEntry);

							if (!SoundEntry.bIsPlotActive)
							{
								bToggleOn = true;
								break;
							}
						}

						const bool bIgnoreCategories = SelectedItemsIncludesAnAsset();
						ToggleShowPlotEntries(FilteredEntriesListView->GetSelectedItems(), bToggleOn, bIgnoreCategories);
					}

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().CreateButtonContentWidget("AudioInsights.Icon.SoundDashboard.Plots", LOCTEXT("SoundDashboard_PlotsButtonText", "Plot Selected"), "SmallButtonText")
				]
			]
			// Plot by Text button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundDashboard_PlotByTextTooltipText", "Toggles plotting on entries matching the search text."))
				.OnClicked_Lambda([this]()
				{
					bIsPlotFilteredMode = !bIsPlotFilteredMode;

					ResetPlots(); // Clear plots when we change modes

					if (bIsPlotFilteredMode)
					{
						PlotFilteredEntries(); // Since we might be paused, also plot filtered entries here
					}

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().CreateToggleButtonContent("AudioInsights.Icon.SoundDashboard.Plots",
						"SmallButtonText",
						[this]() { return bIsPlotFilteredMode ? FSlateColor(FColor::Green) : FSlateColor::UseForeground(); },
						[]() { return LOCTEXT("SoundDashboard_PlotByTextButtonText", "Plot by Text"); })
				]
			]
			// Clear All Plots button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 6.0f)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("SoundsDashboard_ClearPlotsTooltipText", "Clears all enabled plots."))
				.OnClicked_Lambda([this]()
				{
					bIsPlotFilteredMode = false;
					ResetPlots();

					return FReply::Handled();
				})
				[
					FSlateStyle::Get().CreateButtonContentWidget("AudioInsights.Icon.SoundDashboard.Reset", LOCTEXT("SoundsDashboard_ClearAllPlotsButtonText", "Clear All Plots"), "SmallButtonText")
				]
			];
	}

	TSharedRef<SWidget> FSoundDashboardViewFactory::CreateShowPlotColumnButton(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& InColumn, TFunction<void(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>&, const bool)> ShowPlotToggleFunc, TFunctionRef<bool(const IDashboardDataTreeViewEntry&)> IsPlotActiveFunc)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		return SNew(SHorizontalBox)
			.Visibility_Lambda([this, InRowWidget, InRowData]()
			{
				return InRowWidget->GetVisibility();
			})
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.Style(&FSlateStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SoundDashboard.MuteSoloButton"))
				.IsChecked(ECheckBoxState::Unchecked)
				.IsChecked_Lambda([this, InRowData]()
				{
					return CastEntry(*InRowData).bIsPlotActive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this, InRowData, ShowPlotToggleFunc](ECheckBoxState NewState)
				{
					ShowPlotToggleFunc({ InRowData }, NewState == ECheckBoxState::Checked);
				})
				[
					SNew(SImage)
					.Image_Lambda([this, InRowData, InColumn, IsPlotActiveFunc]()
					{
						const FColumnData& ColumnData = GetColumns()[InColumn];
						const FName IconName = IsPlotActiveFunc(*InRowData) ? ColumnData.GetIconName(InRowData.Get()) : "AudioInsights.Icon.SoundDashboard.Transparent";

						return FSlateStyle::Get().GetBrush(IconName);
					})
					.ColorAndOpacity_Lambda([InRowData]() { return InRowData->GetEntryColor(); })
				]
			];
	}

	void FSoundDashboardViewFactory::ToggleShowPlotEntries(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries, const bool bShowPlot, const bool bIgnoreCategories /*= false*/)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		TSet<uint32> ActiveSoundsToggledOff;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : InEntries)
		{
			if (!Entry.IsValid())
			{
				continue;
			}

			FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
			if (SoundEntry.bIsCategory && bIgnoreCategories)
			{
				continue;
			}
			else
			{
				SetShowPlotRecursive(Entry, bShowPlot);

				if (!bShowPlot && !SoundEntry.bIsCategory)
				{
					ActiveSoundsToggledOff.Add(SoundEntry.ActiveSoundPlayOrder);
				}
			}
		}

		// Clean up any parent Is Plotting states if a child has been disabled
		if (!bShowPlot && ActiveSoundsToggledOff.Num() > 0)
		{
			CleanUpParentPlotEnabledStates(ActiveSoundsToggledOff);
		}

		BroadcastPlotUpdate();
	}

	void FSoundDashboardViewFactory::SetShowPlotRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const bool bShowPlot)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		SetShowPlot(Entry, bShowPlot);

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
		{
			SetShowPlotRecursive(ChildEntry, bShowPlot);
		}
	}

	void FSoundDashboardViewFactory::SetShowPlotVisibleRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		const FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
		SetShowPlot(Entry, SoundEntry.bIsVisible);

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
		{
			SetShowPlotVisibleRecursive(ChildEntry);
		}
	}

	void FSoundDashboardViewFactory::SetShowPlot(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const bool bShowPlot)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

		// Special case: do not enable toggling plots on the Pinned category header
		if (SoundEntry.EntryType == ESoundDashboardEntryType::Pinned)
		{
			return;
		}

		SoundEntry.bIsPlotActive = bShowPlot;

		// If this is a Pinned entry, make sure to replicate the value in the original copy
		if (PinnedItemEntries.IsValid() && SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::PinnedCopy)
		{
			const TSharedPtr<IDashboardDataTreeViewEntry>& OriginalEntry = PinnedItemEntries->FindOriginalEntryInChildren(Entry);
			if (OriginalEntry.IsValid())
			{
				FSoundDashboardEntry& OriginalSoundEntry = CastEntry(*OriginalEntry);

				OriginalSoundEntry.bIsPlotActive = bShowPlot;
			}
		}
	}

	void FSoundDashboardViewFactory::CleanUpParentPlotEnabledStates(const TSet<uint32>& ActiveSoundsToggledOff)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& CategoryEntry : FullTree)
		{
			if (!CategoryEntry.IsValid())
			{
				continue;
			}

			const FSoundDashboardEntry& CategorySoundEntry = CastEntry(*CategoryEntry);

			if (CategorySoundEntry.EntryType == ESoundDashboardEntryType::Pinned)
			{
				for (const TSharedPtr<IDashboardDataTreeViewEntry>& PinnedCategoryEntry : CategoryEntry->Children)
				{
					if (!PinnedCategoryEntry.IsValid())
					{
						continue;
					}

					for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : PinnedCategoryEntry->Children)
					{
						if (!ChildEntry.IsValid())
						{
							continue;
						}

						const FSoundDashboardEntry& ActiveSoundEntry = CastEntry(*ChildEntry);
						if (ActiveSoundsToggledOff.Contains(ActiveSoundEntry.ActiveSoundPlayOrder))
						{
							SetShowPlot(CategoryEntry, false);
							SetShowPlot(PinnedCategoryEntry, false);
							SetShowPlot(ChildEntry, false);
						}
					}
				}
			}
			else
			{
				for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : CategoryEntry->Children)
				{
					if (!ChildEntry.IsValid())
					{
						continue;
					}

					const FSoundDashboardEntry& ActiveSoundEntry = CastEntry(*ChildEntry);
					if (ActiveSoundsToggledOff.Contains(ActiveSoundEntry.ActiveSoundPlayOrder))
					{
						SetShowPlot(CategoryEntry, false);
						SetShowPlot(ChildEntry, false);
					}
				}
			}
		}
	}

	void FSoundDashboardViewFactory::ProcessPlotData()
	{
#if WITH_EDITOR
		if (FilteredEntriesListView.IsValid())
		{
			OnProcessPlotData.Broadcast(DataViewEntries, FilteredEntriesListView->GetSelectedItems(), false /*bForceUpdate*/);
		}
#else
		if (FilteredEntriesListView.IsValid())
		{
			OnProcessPlotData.Broadcast(DataViewEntries, FilteredEntriesListView->GetSelectedItems(), false /*bForceUpdate*/);
		}
		else
		{
			// In standalone, we still want to process entries as the trace file loads in
			// Send an empty array for selected items and broadcast the data
			const TArray<TSharedPtr<IDashboardDataTreeViewEntry>> EmptySelectedEntriesContainer;
			OnProcessPlotData.Broadcast(DataViewEntries, EmptySelectedEntriesContainer, false /*bForceUpdate*/);
		}
#endif // WITH_EDITOR
	}

	TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo> FSoundDashboardViewFactory::GetPlotColumnInfo()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		auto CreatePlotColumnInfo = [this]()
		{
			return TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo>
			{
				{
					AmplitudeColumnName,
					{
						AmplitudeColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).AmplitudeDataRange; },
						[this](const float InValue) { return bDisplayAmpPeakInDb ? ::Audio::ConvertToDecibels(InValue) : InValue; },
						[this]() { return bDisplayAmpPeakInDb ? FSlateStyle::Get().GetAmpFloatFormat() : FSlateStyle::Get().GetLinearVolumeFloatFormat(); },
						[this]()
						{
							if (bDisplayAmpPeakInDb)
							{
								return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Amplitude_dB));
							}
							else
							{
								return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Amplitude_Linear));
							}
						},
						[this]() { return bDisplayAmpPeakInDb ? DefaultPlotRanges.bUseCustomAmplitude_dBRange : DefaultPlotRanges.bUseCustomAmplitude_LinearRange; },
						[this](bool bUseCustomYRange)
						{
							if (bDisplayAmpPeakInDb)
							{
								DefaultPlotRanges.bUseCustomAmplitude_dBRange = bUseCustomYRange;
							}
							else
							{
								DefaultPlotRanges.bUseCustomAmplitude_LinearRange = bUseCustomYRange;
							}
							OnDefaultPlotRangesChanged();
						},
						[this]() { return bDisplayAmpPeakInDb ? DefaultPlotRanges.Amplitude_dB : DefaultPlotRanges.Amplitude_Linear; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType)
						{
							if (bDisplayAmpPeakInDb)
							{
								DefaultPlotRanges.Amplitude_dB = Range;
							}
							else
							{
								DefaultPlotRanges.Amplitude_Linear = Range;
							}
							OnDefaultPlotRangesChanged(CommitType.IsSet());
						}
					}
				},
				{
					VolumeColumnName,
					{
						VolumeColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).VolumeDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetLinearVolumeFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Volume)); },
						[this]() { return DefaultPlotRanges.bUseCustomVolumeRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomVolumeRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.Volume; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.Volume = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					DistanceColumnName,
					{
						DistanceColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).DistanceDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetFreqFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Distance)); },
						[this]() { return DefaultPlotRanges.bUseCustomDistanceRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomDistanceRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.Distance; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.Distance = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					DistanceAttenuationColumnName,
					{
						DistanceAttenuationColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).DistanceAttenuationDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetLinearVolumeFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, DistanceAttenuation)); },
						[this]() { return DefaultPlotRanges.bUseCustomDistanceAttenuationRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomDistanceAttenuationRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.DistanceAttenuation; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.DistanceAttenuation = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					PitchColumnName,
					{
						PitchColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).PitchDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetPitchFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Pitch)); },
						[this]() { return DefaultPlotRanges.bUseCustomPitchRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomPitchRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.Pitch; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.Pitch = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					PriorityColumnName,
					{
						PriorityColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).PriorityDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetLinearVolumeFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, Priority)); },
						[this]() { return DefaultPlotRanges.bUseCustomPriorityRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomPriorityRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.Priority; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.Priority = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					LPFFreqColumnName,
					{
						LPFFreqColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).LPFFreqDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetFreqFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, LPFFreq)); },
						[this]() { return DefaultPlotRanges.bUseCustomLPFFreqRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomLPFFreqRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.LPFFreq; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.LPFFreq = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				},
				{
					HPFFreqColumnName,
					{
						HPFFreqColumnDisplayName,
						[](const IDashboardDataTreeViewEntry& InEntry) -> const ::Audio::TCircularAudioBuffer<FDataPoint>& { return CastEntry(InEntry).HPFFreqDataRange; },
						nullptr,
						[this]() { return FSlateStyle::Get().GetFreqFloatFormat(); },
						[]() { return FSoundPlotsDashboardPlotRanges::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FSoundPlotsDashboardPlotRanges, HPFFreq)); },
						[this]() { return DefaultPlotRanges.bUseCustomHPFFreqRange; },
						[this](bool bUseCustomYRange) { DefaultPlotRanges.bUseCustomHPFFreqRange = bUseCustomYRange; OnDefaultPlotRangesChanged(); },
						[this]() { return DefaultPlotRanges.HPFFreq; },
						[this](const FFloatInterval& Range, TOptional<ETextCommit::Type> CommitType) { DefaultPlotRanges.HPFFreq = Range; OnDefaultPlotRangesChanged(CommitType.IsSet()); }
					}
				}
			};
		};

		return CreatePlotColumnInfo();
	}

	bool FSoundDashboardViewFactory::SelectedItemsIncludesAnAsset() const
	{
		using namespace FSoundDashboardViewFactoryPrivate;
		if (!FilteredEntriesListView.IsValid())
		{
			return false;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& SelectedEntry : FilteredEntriesListView->GetSelectedItems())
		{
			if (!IsCategoryItem(*SelectedEntry))
			{
				return true;
			}
		}

		return false;
	}

	bool FSoundDashboardViewFactory::SelectionIncludesUnpinnedItem() const
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!FilteredEntriesListView.IsValid())
		{
			return false;
		}

		const TArray<TSharedPtr<IDashboardDataTreeViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();
		for (const TSharedPtr<IDashboardDataTreeViewEntry>& SelectedItem : SelectedItems)
		{
			if (!SelectedItem.IsValid())
			{
				continue;
			}

			const FSoundDashboardEntry& SelectedSoundEntry = CastEntry(*SelectedItem);
			if (SelectedSoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::None)
			{
				return true;
			}
		}

		return false;
	}

	void FSoundDashboardViewFactory::ClearSelection()
	{
		if (FilteredEntriesListView == nullptr)
		{
			return;
		}

		// Make sure to clear the selection and the internal selector in SListView
		// to ensure no shared references keep the entry alive past the point
		// it has been removed from the dashboard
		FilteredEntriesListView->ClearSelection();
		FilteredEntriesListView->SetSelection(nullptr);
	}

	void FSoundDashboardViewFactory::PinSound()
	{
		if (FilteredEntriesListView.IsValid())
		{
			const TArray<TSharedPtr<IDashboardDataTreeViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();
			const bool bSelectionContainsAssets = SelectedItemsIncludesAnAsset();

			for (TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
			{
				// If only categories are selected, pin the entire category
				if (SelectedItems.Contains(Entry) && !bSelectionContainsAssets)
				{
					for (TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
					{
						MarkBranchAsPinned(ChildEntry, true /*bIsPinned*/);
						CreatePinnedEntry(ChildEntry);
					}
				}
				else
				{
					PinSelectedItems(Entry, SelectedItems);
				}
			}
			
			ClearSelection();
		}
	}

	void FSoundDashboardViewFactory::UnpinSound()
	{
		if (FilteredEntriesListView.IsValid() && PinnedItemEntries.IsValid())
		{
			const TArray<TSharedPtr<IDashboardDataTreeViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();
			const bool bSelectionContainsAssets = SelectedItemsIncludesAnAsset();

			// If the user has only selected the pinned item row, unpin everything
			if (SelectedItems.Num() == 1 && SelectedItems[0] == PinnedItemEntries->GetPinnedSectionEntry())
			{
				for (TSharedPtr<IDashboardDataTreeViewEntry>& OriginalChildEntry : DataViewEntries)
				{
					MarkBranchAsPinned(OriginalChildEntry, false /*bIsPinned*/);
				}
				
				PinnedItemEntries.Reset();
			}
			else
			{
				UnpinSelectedItems(PinnedItemEntries, SelectedItems, bSelectionContainsAssets);
			}
			
			ClearSelection();
		}
	}

	void FSoundDashboardViewFactory::PinSelectedItems(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedItems)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (SelectedItems.Contains(Entry) && (!IsRootItem(Entry.ToSharedRef()) || TreeViewingMode != ESoundDashboardTreeViewingOptions::FullTree))
		{
			const FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

			if (SoundEntry.PinnedEntryType != FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry)
			{
				MarkBranchAsPinned(Entry, true /*bIsPinned*/);
				CreatePinnedEntry(Entry);
			}
		}
		else 
		{
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : Entry->Children)
			{
				if (SelectedItems.Contains(Child))
				{
					if (TreeViewingMode == ESoundDashboardTreeViewingOptions::FullTree && IsRootItem(Entry.ToSharedRef()))
					{
						MarkBranchAsPinned(Child, true /*bIsPinned*/);
						CreatePinnedEntry(Child);
					}
					else
					{
						MarkBranchAsPinned(Entry, true /*bIsPinned*/);
						CreatePinnedEntry(Entry);
					}
				}
				else
				{
					PinSelectedItems(Child, SelectedItems);
				}
			}
		}
	}

	void FSoundDashboardViewFactory::UnpinSelectedItems(const TSharedPtr<FPinnedSoundEntryWrapper>& PinnedWrapperEntry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedItems, const bool bSelectionContainsAssets)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		// Run through all the pinned items in the dashboard and check if they are in the list of selected items.
		// If the child of a non-category parent is selected, we will move the parent and all of it's children back to the unpinned section
		for (const TSharedPtr<FPinnedSoundEntryWrapper>& PinnedWrapperChild : PinnedWrapperEntry->PinnedWrapperChildren)
		{
			const TSharedPtr<IDashboardDataTreeViewEntry> OriginalChildEntry = PinnedWrapperChild->GetOriginalDataEntry();
			if (!OriginalChildEntry.IsValid())
			{
				UnpinSelectedItems(PinnedWrapperChild, SelectedItems, bSelectionContainsAssets);
				continue;
			}

			const FSoundDashboardEntry& OriginalChildSoundEntry = CastEntry(*OriginalChildEntry);

			const TSharedPtr<IDashboardDataTreeViewEntry>* FoundMatchingSelectedEntry = SelectedItems.FindByPredicate([&OriginalChildSoundEntry, bSelectionContainsAssets](const TSharedPtr<IDashboardDataTreeViewEntry> SelectedEntry)
			{
				if (!SelectedEntry.IsValid())
				{
					return false;
				}

				const FSoundDashboardEntry& SelectedSoundEntry = CastEntry(*SelectedEntry);
				if (SelectedSoundEntry.PinnedEntryType != FSoundDashboardEntry::EPinnedEntryType::PinnedCopy)
				{
					return false;
				}

				if (SelectedSoundEntry.bIsCategory && !bSelectionContainsAssets)
				{
					return OriginalChildSoundEntry.EntryType == SelectedSoundEntry.EntryType;
				}
				
				return OriginalChildSoundEntry.GetEntryID() == SelectedSoundEntry.GetEntryID();
			});

			if (FoundMatchingSelectedEntry != nullptr)
			{
				if (IsCategoryItem(*PinnedWrapperEntry->GetPinnedSectionEntry()))
				{
					// If the current parent pinned item is a category, move the child to the unpinned area and continue
					MarkBranchAsPinned(OriginalChildEntry, false /*bIsPinned*/);
					PinnedWrapperChild->MarkToDelete();
				}
				else
				{
					// If the current parent pinned item is not a category, move it and all of it's children to unpinned.
					// There is no need to check the other children so break.
					const TWeakPtr<IDashboardDataTreeViewEntry> OriginalPinnedEntry = PinnedWrapperEntry->GetOriginalDataEntry();
					if (OriginalPinnedEntry.IsValid())
					{
						MarkBranchAsPinned(OriginalPinnedEntry.Pin(), false /*bIsPinned*/);
						PinnedWrapperEntry->MarkToDelete();
						break;
					}
				}
			}
			else
			{
				// If this child item is not selected, check it's children.
				UnpinSelectedItems(PinnedWrapperChild, SelectedItems, bSelectionContainsAssets);
			}
		}
	}

	void FSoundDashboardViewFactory::RecreatePinnedEntries(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*Entry);

		if (Entry->Children.IsEmpty() && SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry)
		{
			MarkBranchAsPinned(Entry, true /*bIsPinned*/);
			CreatePinnedEntry(Entry);
		}
		else
		{
			const bool bSoundEntryIsPinned = SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry;
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : Entry->Children)
			{
				if (bSoundEntryIsPinned)
				{
					if (TreeViewingMode == ESoundDashboardTreeViewingOptions::FullTree && SoundEntry.bIsCategory)
					{
						MarkBranchAsPinned(Child, true /*bIsPinned*/);
						CreatePinnedEntry(Child);
					}
					else
					{
						MarkBranchAsPinned(Entry, true /*bIsPinned*/);
						CreatePinnedEntry(Entry);
						break;
					}
				}
				else
				{
					const FSoundDashboardEntry& ChildEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*Child);
					const bool bChildEntryIsPinned = ChildEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry;

					if (bChildEntryIsPinned && TreeViewingMode != ESoundDashboardTreeViewingOptions::FlatList && !SoundEntry.bIsCategory)
					{
						MarkBranchAsPinned(Entry, true /*bIsPinned*/);
						CreatePinnedEntry(Entry);
						break;
					}
					else
					{
						RecreatePinnedEntries(Child);
					}
				}
			}
		}
	}

	void FSoundDashboardViewFactory::MarkBranchAsPinned(const TSharedPtr<IDashboardDataTreeViewEntry> Entry, const bool bIsPinned)
	{
		FSoundDashboardEntry& SoundEntry = FSoundDashboardViewFactoryPrivate::CastEntry(*Entry);
		SoundEntry.PinnedEntryType = bIsPinned ? FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry : FSoundDashboardEntry::EPinnedEntryType::None;

		for (TSharedPtr<IDashboardDataTreeViewEntry> Child : Entry->Children)
		{
			MarkBranchAsPinned(Child, bIsPinned);
		}
	}

	void FSoundDashboardViewFactory::InitPinnedItemEntries()
	{
		if (PinnedItemEntries.IsValid())
		{
			return;
		}

		TSharedPtr<FSoundDashboardEntry> PinnedCategory = MakeShared<FSoundDashboardEntry>();
		PinnedCategory->SetName(FSoundDashboardViewFactoryPrivate::PinnedCategoryName.ToString());
		PinnedCategory->EntryType = ESoundDashboardEntryType::Pinned;
		PinnedCategory->PinnedEntryType = FSoundDashboardEntry::EPinnedEntryType::PinnedCopy;
		PinnedCategory->bIsCategory = true;

		PinnedItemEntries = MakeShared<FPinnedSoundEntryWrapper>(PinnedCategory);
	}

	void FSoundDashboardViewFactory::CreatePinnedEntry(TSharedPtr<IDashboardDataTreeViewEntry> Entry)
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		// If we have at least one entry that is pinned, ensure the pinned section has been created
		// The pinned area will delete itself once empty
		InitPinnedItemEntries();

		if (TreeViewingMode != ESoundDashboardTreeViewingOptions::FullTree)
		{
			PinnedItemEntries->AddChildEntry(Entry);
		}
		else
		{
			const FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);

			// Check if category is already in the list, if so we need to merge
			bool bFoundExistingCategory = false;
			for (TSharedPtr<FPinnedSoundEntryWrapper> PinnedCategoryEntry : PinnedItemEntries->PinnedWrapperChildren)
			{
				if (!PinnedCategoryEntry.IsValid())
				{
					continue;
				}

				const FSoundDashboardEntry& PinnedCategorySoundDashboardEntry = CastEntry(*PinnedCategoryEntry->GetPinnedSectionEntry());

				if (PinnedCategorySoundDashboardEntry.EntryType == SoundEntry.EntryType)
				{
					bFoundExistingCategory = true;

					const TSharedPtr<IDashboardDataTreeViewEntry>* FoundExisingEntry = PinnedCategorySoundDashboardEntry.Children.FindByPredicate([&SoundEntry](const TSharedPtr<IDashboardDataTreeViewEntry> PinnedEntry)
					{
						if (!PinnedEntry.IsValid())
						{
							return false;
						}

						const FSoundDashboardEntry& ExistingPinnedEntry = CastEntry(*PinnedEntry);
						return ExistingPinnedEntry.GetEntryID() == SoundEntry.GetEntryID();
					});

					// If we didn't find this entry already inside the pinned area, add it here
					if (FoundExisingEntry == nullptr)
					{
						PinnedCategoryEntry->AddChildEntry(Entry);
					}
					break;
				}
			}

			// if we haven't found an existing pinned category, create a new one and add this item
			if (!bFoundExistingCategory)
			{
				for (const TSharedPtr<IDashboardDataTreeViewEntry>& DataCategoryEntry : DataViewEntries)
				{
					if (SoundEntry.EntryType == CastEntry(*DataCategoryEntry).EntryType)
					{
						TSharedPtr<FSoundDashboardEntry> NewPinnedCategoryEntry = MakeShared<FSoundDashboardEntry>(CastEntry(*DataCategoryEntry));
						NewPinnedCategoryEntry->Children.Reset();

						TSharedPtr<FPinnedSoundEntryWrapper> PinnedCategory = PinnedItemEntries->AddChildEntry(NewPinnedCategoryEntry);
						PinnedCategory->AddChildEntry(Entry);

						break;
					}
				}
			}
		}
	}

	void FSoundDashboardViewFactory::UpdatePinnedSection()
	{
		if (bPinnedSectionNeedsRebuild)
		{
			if (!PinnedItemEntries.IsValid())
			{
				InitPinnedItemEntries();
			}

			RebuildPinnedSection();

			const int32 RebuiltChildren = PinnedItemEntries.IsValid() ? PinnedItemEntries->PinnedWrapperChildren.Num() : 0;

			// After OnTimeControlMethodReset, the provider re-creates entries asynchronously from the realtime trace stream.
			// The first UpdatePinnedSection ticks may run with DataViewEntries containing only the parent category (no children yet), so RebuildPinnedSection finds nothing to pin.
			// Keep the latch set across ticks until the rebuild actually produces PinnedWrapperChildren > 0, i.e. the restored entry has finally arrived in DataViewEntries.
			// Also clear the latch if the provider has nothing cached to restore (e.g. after un-pin), so we don't loop on empty rebuilds.
			if (RebuiltChildren > 0)
			{
				bPinnedSectionNeedsRebuild = false;
			}
			else if (const TSharedPtr<const FSoundTraceProvider> Provider = FindProvider<const FSoundTraceProvider>(false /*bEnsureIfMissing*/))
			{
				if (!Provider->HasCachedPinnedEntries())
				{
					bPinnedSectionNeedsRebuild = false;
				}
			}

			return;
		}

		if (!PinnedItemEntries.IsValid())
		{
			return;
		}

		if (bTreeViewModeChanged)
		{
			RebuildPinnedSection();
		}

		PinnedItemEntries->CleanUp();

		if (FPinnedSoundEntryWrapperPrivate::CanBeDeleted(PinnedItemEntries))
		{
			PinnedItemEntries.Reset();

			// Scrub may have temporarily destroyed pinned entries that will reappear when the time marker moves back.
			// Re-trigger the latch only if the provider still has cached pins to restore; otherwise (e.g. the user just un-pinned everything) leave it cleared.
			if (const TSharedPtr<const FSoundTraceProvider> Provider = FindProvider<const FSoundTraceProvider>(false /*bEnsureIfMissing*/))
			{
				if (Provider->HasCachedPinnedEntries())
				{
					bPinnedSectionNeedsRebuild = true;
				}
			}
		}
		else
		{
			PinnedItemEntries->UpdateParams();
		}
	}

	void FSoundDashboardViewFactory::RebuildPinnedSection()
	{
		using namespace FSoundDashboardViewFactoryPrivate;

		if (!PinnedItemEntries.IsValid())
		{
			return;
		}

		PinnedItemEntries.Reset();
		InitPinnedItemEntries();

		for (TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			FSoundDashboardEntry& SoundEntry = CastEntry(*Entry);
			if (SoundEntry.PinnedEntryType == FSoundDashboardEntry::EPinnedEntryType::HiddenOriginalEntry && SoundEntry.bIsCategory)
			{
				for (TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
				{
					MarkBranchAsPinned(ChildEntry, true /*bIsPinned*/);
					CreatePinnedEntry(ChildEntry);
				}
			}
			else
			{
				RecreatePinnedEntries(Entry);
			}
		}
	}

#if WITH_EDITOR
	void FSoundDashboardViewFactory::BrowseSoundAsset() const
	{
		if (GEditor)
		{
			TArray<UObject*> EditableAssets = GetSelectedEditableAssets();
			GEditor->SyncBrowserToObjects(EditableAssets);
		}
	}

	void FSoundDashboardViewFactory::OpenSoundAsset() const
	{
		if (GEditor && FilteredEntriesListView.IsValid())
		{
			TArray<UObject*> Objects = GetSelectedEditableAssets();
			if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetSubsystem->OpenEditorForAssets(Objects);
			}
		}
	}
#endif // WITH_EDITOR
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
