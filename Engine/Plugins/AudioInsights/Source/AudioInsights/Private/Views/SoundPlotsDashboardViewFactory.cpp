// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SoundPlotsDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "Messages/SoundTraceMessages.h"
#include "Views/SoundDashboardViewFactory.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	/////////////////////////////////////////////////////////////////////////////////////////
	// FSoundPlotsDashboardViewFactoryPrivate
	namespace FSoundPlotsDashboardViewFactoryPrivate
	{
		const FSoundDashboardEntry& CastEntry(const IDashboardDataTreeViewEntry& InData)
		{
			return static_cast<const FSoundDashboardEntry&>(InData);
		};
	}

	/////////////////////////////////////////////////////////////////////////////////////////
	// FSoundPlotsDashboardViewFactory
	FName FSoundPlotsDashboardViewFactory::GetName() const
	{
		return "Plots";
	}

	FText FSoundPlotsDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_PlotsTab_Name", "Plots");
	}

	EDefaultDashboardTabStack FSoundPlotsDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Plots;
	}

	FSlateIcon FSoundPlotsDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Plots");
	}

	TSharedRef<SWidget> FSoundPlotsDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		return PlotsWidget ? PlotsWidget->MakeWidget() : SNullWidget::NullWidget;
	}

	void FSoundPlotsDashboardViewFactory::InitPlots(const TSharedRef<FSoundDashboardViewFactory> SoundsDashboard)
	{
		using namespace FSoundPlotsDashboardViewFactoryPrivate;

		TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo> ColumnInfo = SoundsDashboard->GetPlotColumnInfo();
		PlotsWidget = MakeShared<FSoundPlotsWidgetView>(MoveTemp(ColumnInfo), [](const IDashboardDataTreeViewEntry& InEntry) -> bool { return CastEntry(InEntry).bIsPlotActive; });

		OnProcessPlotData = SoundsDashboard->OnProcessPlotData.AddSP(this, &FSoundPlotsDashboardViewFactory::ProcessPlotData);
		OnUpdatePlotVisibility = SoundsDashboard->OnUpdatePlotVisibility.AddSP(this, &FSoundPlotsDashboardViewFactory::UpdatePlotVisibility);
		OnUpdatePlotSelection = SoundsDashboard->OnUpdatePlotSelection.AddSP(this, &FSoundPlotsDashboardViewFactory::UpdatePlotSelection);
	}

	void FSoundPlotsDashboardViewFactory::ProcessPlotData(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, const bool bForceUpdate /* = false */)
	{
		if (PlotsWidget.IsValid())
		{
			PlotsWidget->ProcessPlotData(DataViewEntries, SelectedEntries, bForceUpdate);
		}
	}

	void FSoundPlotsDashboardViewFactory::UpdatePlotVisibility(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries)
	{
		if (PlotsWidget.IsValid())
		{
			PlotsWidget->UpdatePlotVisibility(DataViewEntries);
		}
	}

	void FSoundPlotsDashboardViewFactory::UpdatePlotSelection(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries)
	{
		if (PlotsWidget.IsValid())
		{
			PlotsWidget->UpdatePlotSelection(SelectedEntries);
		}
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
