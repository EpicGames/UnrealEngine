// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SoundPlotsWidgetView.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "Features/IModularFeatures.h"
#include "Misc/AssertionMacros.h"
#include "SoundPlotsWidgetCommands.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"

#if WITH_EDITOR
#include "Editor.h"
#else
#include "AudioInsightsComponent.h"
#include "AudioInsightsTimingViewExtender.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FSoundPlotsWidgetViewPrivate
	{
		constexpr float DefaultPlotThickness = 1.0f;
		constexpr float SelectedPlotThickness = 4.0f;
	}

	FSoundPlotsWidgetView::FSoundPlotsWidgetView(TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo>&& InColumnInfo, const TFunction<bool(const IDashboardDataTreeViewEntry& InEntry)> InIsPlotEnabledForEntryFunc)
		: ColumnInfo(MoveTemp(InColumnInfo))
		, IsPlotEnabledForEntryFunc(InIsPlotEnabledForEntryFunc)
	{
		FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
		AudioInsightsTraceModule.OnAnalysisStarting.AddRaw(this, &FSoundPlotsWidgetView::OnAnalysisStarting);

		FSoundPlotsWidgetCommands::Register();
		BindCommands();

#if !WITH_EDITOR
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		const TSharedPtr<FAudioInsightsComponent> AudioInsightsComponent = AudioInsightsModule.GetAudioInsightsComponent();
		if (AudioInsightsComponent.IsValid())
		{
			AudioInsightsComponent->OnSessionAnalysisCompleted.AddRaw(this, &FSoundPlotsWidgetView::OnSessionAnalysisCompleted);

			if (!AudioInsightsComponent->GetIsLiveSession())
			{
				GameState = EGameState::LoadingTraceFile;
			}
		}
#endif // !WITH_EDITOR
	}

	FSoundPlotsWidgetView::~FSoundPlotsWidgetView()
	{
		FSoundPlotsWidgetCommands::Unregister();

		if (FModuleManager::Get().IsModuleLoaded("AudioInsights") && IModularFeatures::Get().IsModularFeatureAvailable(TraceServices::ModuleFeatureName))
		{
			FTraceModule& TraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());
			TraceModule.OnAnalysisStarting.RemoveAll(this);

#if !WITH_EDITOR
			FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			const TSharedPtr<FAudioInsightsComponent> AudioInsightsComponent = AudioInsightsModule.GetAudioInsightsComponent();
			if (AudioInsightsComponent.IsValid())
			{
				AudioInsightsComponent->OnSessionAnalysisCompleted.RemoveAll(this);
			}
#endif // !WITH_EDITOR
		}
	}

	TSharedRef<SWidget> FSoundPlotsWidgetView::MakeWidget()
	{
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().OnTimingViewTimeMarkerChanged.AddSP(this, &FSoundPlotsWidgetView::OnTimingViewTimeMarkerChanged);
		AudioInsightsModule.GetTimingViewExtender().OnTimeControlMethodReset.AddSP(this, &FSoundPlotsWidgetView::OnTimeControlMethodReset);

#if WITH_EDITOR
		FEditorDelegates::PostPIEStarted.AddSP(this, &FSoundPlotsWidgetView::OnPIEStarted);
		FEditorDelegates::EndPIE.AddSP(this, &FSoundPlotsWidgetView::OnPIEStopped);
		FEditorDelegates::PausePIE.AddSP(this, &FSoundPlotsWidgetView::OnPIEPaused);
		FEditorDelegates::ResumePIE.AddSP(this, &FSoundPlotsWidgetView::OnPIEResumed);

		FTraceAuxiliary::OnTraceStopped.AddSP(this, &FSoundPlotsWidgetView::OnTraceStopped);
#else
		const TSharedPtr<FAudioInsightsComponent> AudioInsightsComponent = AudioInsightsModule.GetAudioInsightsComponent();
		if (AudioInsightsComponent.IsValid())
		{
			AudioInsightsComponent->OnTabSpawn.AddSP(this, &FSoundPlotsWidgetView::OnAudioInsightsComponentTabSpawn);
		}
#endif // WITH_EDITOR

		return MakePlotsWidget();
	}

	void FSoundPlotsWidgetView::ProcessPlotData(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, const bool bForceUpdate /* = false */)
	{
		if (DataViewEntries.IsEmpty() || !PlotWidgetMetadataPerCurve.IsValid())
		{
			return;
		}

		if (!bForceUpdate)
		{
			if (GameState == EGameState::ControlledByTimemarker && !bTimeMarkerMoved)
			{
				return;
			}

			if (GameState == EGameState::Paused)
			{
				return;
			}
		}

		// Process new data
		bool bHasNewMetadata = false;
		for (const TSharedPtr<IDashboardDataTreeViewEntry>& DataEntry : DataViewEntries)
		{
			CollectDataRecursive(DataEntry, SelectedEntries, bHasNewMetadata);
		}

		// Set metadata for each widget if updated
		if (bHasNewMetadata)
		{
			PlotWidget->SetCurvesMetadata(PlotWidgetMetadataPerCurve);
		}

		// Remove old points and set curve data for each widget
		const double PlotDrawMinLimitTimestamp = CurrentTimestamp - BeginTimestamp - (FAudioInsightsTimingViewExtender::MaxPlottingHistorySeconds + FAudioInsightsTimingViewExtender::PlottingMarginSeconds /* extra grace time to avoid curve cuts being displayed */);

		const TSharedPtr<FPointDataPerCurveMap>* const FoundPointDataPerCurveElement = PlotWidgetCurveIdToPointDataMapPerColumn.Find(SelectedPlotColumnName);

		if (FoundPointDataPerCurveElement && FoundPointDataPerCurveElement->IsValid())
		{
			// Remove points that are older than max history limit from the most recent timestamp
			for (auto& [CurveId, CurvePoints] : **FoundPointDataPerCurveElement)
			{
				const int32 FoundMinIndex = CurvePoints.IndexOfByPredicate([&PlotDrawMinLimitTimestamp](const FDataPoint& InDataPoint)
				{
					return InDataPoint.Key >= PlotDrawMinLimitTimestamp;
				});

				if (FoundMinIndex > 0)
				{
					CurvePoints.RemoveAt(0, FoundMinIndex, EAllowShrinking::No);
				}
			}

			UpdatePlot();
		}

#if !WITH_EDITOR
		bTimeMarkerMoved = false;
#endif // !WITH_EDITOR
	}

	void FSoundPlotsWidgetView::UpdatePlotVisibility(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries)
	{
		if (!PlotWidgetMetadataPerCurve.IsValid() || DataViewEntries.IsEmpty())
		{
			return;
		}

		for (auto& [ID, PlotMetaData] : *PlotWidgetMetadataPerCurve)
		{
			PlotMetaData.CurveColor.A = 0.0f;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : DataViewEntries)
		{
			UpdatePlotVisibilityRecursive(Entry);
		}
	}

	void FSoundPlotsWidgetView::UpdatePlotSelection(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries)
	{
		using namespace FSoundPlotsWidgetViewPrivate;

		if (!PlotWidgetMetadataPerCurve.IsValid())
		{
			return;
		}

		for (auto& [SourceID, Metadata] : *PlotWidgetMetadataPerCurve)
		{
			Metadata.PlotThickness = DefaultPlotThickness;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Entry : SelectedEntries)
		{
			if (!Entry.IsValid())
			{
				continue;
			}

			if (SAudioCurveView::FCurveMetadata* Metadata = PlotWidgetMetadataPerCurve->Find(Entry->GetEntryID()))
			{
				Metadata->PlotThickness = SelectedPlotThickness;
			}
		}
	}

	void FSoundPlotsWidgetView::OnAnalysisStarting(const double Timestamp)
	{
		ResetPlots();

#if WITH_EDITOR
		BeginTimestamp = Timestamp - GStartTime;
		GameState = EGameState::Running;
		bIsTraceActive = true;
#else
		BeginTimestamp = 0.0;
#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	void FSoundPlotsWidgetView::OnPIEStarted(bool bSimulating)
	{
		GameState = EGameState::Running;
		bIsPIEPaused = false;
	}

	void FSoundPlotsWidgetView::OnPIEStopped(bool bSimulating)
	{
		GameState = EGameState::Stopped;
		bIsPIEPaused = false;
	}

	void FSoundPlotsWidgetView::OnPIEPaused(bool bSimulating)
	{
		GameState = EGameState::Paused;
		bIsPIEPaused = true;
	}

	void FSoundPlotsWidgetView::OnPIEResumed(bool bSimulating)
	{
		GameState = EGameState::Running;
		bIsPIEPaused = false;
	}

	void FSoundPlotsWidgetView::OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
	{
		GameState = EGameState::Paused;
		bIsTraceActive = false;
	}
#else
	void FSoundPlotsWidgetView::OnAudioInsightsComponentTabSpawn()
	{
		if (IAudioInsightsModule::IsLiveSession())
		{
			GameState = EGameState::Running;
		}
	}

	void FSoundPlotsWidgetView::OnSessionAnalysisCompleted()
	{
		if (GameState == EGameState::LoadingTraceFile)
		{
			if (PlotWidgetMetadataPerCurve.IsValid())
			{
				for (auto& [SourceID, MetaData] : *PlotWidgetMetadataPerCurve)
				{
					MetaData.CurveColor.A = 0.0f;
				}

				if (PlotWidget.IsValid())
				{
					PlotWidget->SetCurvesMetadata(PlotWidgetMetadataPerCurve);
				}
			}
		}
		else
		{
			GameState = EGameState::Stopped;
		}
	}
#endif // WITH_EDITOR

	void FSoundPlotsWidgetView::OnTimingViewTimeMarkerChanged(double InTimeMarker)
	{
		CurrentTimestamp = InTimeMarker;
		GameState = EGameState::ControlledByTimemarker;
		bTimeMarkerMoved = true;
	}

	void FSoundPlotsWidgetView::OnTimeControlMethodReset()
	{
		bTimeMarkerMoved = false;

#if WITH_EDITOR
		GameState = bIsTraceActive && !bIsPIEPaused ? EGameState::Running : EGameState::Paused;
#else
		if (IAudioInsightsModule::IsLiveSession())
		{
			GameState = EGameState::Running;
		}
#endif // WITH_EDITOR
	}

	FReply FSoundPlotsWidgetView::OnHandleKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
	{
		return (CommandList && CommandList->ProcessCommandBindings(KeyEvent)) ? FReply::Handled() : FReply::Unhandled();
	}

	TSharedRef<SWidget> FSoundPlotsWidgetView::MakePlotsWidget()
	{
		if (!PlotWidget.IsValid() || !ComboBox.IsValid())
		{
			ConstructPlotsWidgetContents();
		}

		static FSpinBoxStyle TransparentFillSpinBoxStyle = FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox");
		TransparentFillSpinBoxStyle.ActiveFillBrush = FSlateColorBrush(FLinearColor::Transparent);
		TransparentFillSpinBoxStyle.HoveredFillBrush = FSlateColorBrush(FLinearColor::Transparent);
		TransparentFillSpinBoxStyle.InactiveFillBrush = FSlateColorBrush(FLinearColor::Transparent);

		return SNew(SVerticalBox)
			.Clipping(EWidgetClipping::ClipToBounds)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SSimpleTimeSlider)
				.ViewRange_Lambda([this]() { return GetViewRange(); })
				.ClampRangeHighlightSize(0.0f) // Hide clamp range
				.ScrubPosition_Lambda([]() { return TNumericLimits<double>::Lowest(); }) // Hide scrub
				.PixelSnappingMethod(EWidgetPixelSnapping::Disabled)
				.OnFormatTickLabel_Lambda([](double InSeconds) { return FSlateStyle::Get().FormatTimestamp(InSeconds).ToString(); })
				.MinPixelsPerDisplayTick(10)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MinWidth(105.0f)
				[
					ComboBox.ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				[
					SNew(STextBlock)
						.Margin(FMargin(10.0, 4.0, 5.0, 0.0))
						.Text(LOCTEXT("SoundPlotsWidgetView_YAxisRange", "Y Axis Range:"))
				]
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				[
					SNew(SSegmentedControl<bool>)
						.OnValueChanged_Lambda([this](bool bInValue) { SetPlotColumnUseCustomYRange(SelectedPlotColumnName, bInValue); })
						.Value_Lambda([this] { return GetPlotColumnUseCustomYRange(SelectedPlotColumnName); })
						+ SSegmentedControl<bool>::Slot(false)
						.Text(LOCTEXT("SoundPlotsWidgetView_YAxisRange_Auto_Text", "Auto"))
						.ToolTip(LOCTEXT("SoundPlotsWidgetView_YAxisRange_Auto_ToolTipText", "Auto range Y axis."))
						+ SSegmentedControl<bool>::Slot(true)
						.Text(LOCTEXT("SoundPlotsWidgetView_YAxisRange_Custom_Text", "Custom"))
						.ToolTip(LOCTEXT("SoundPlotsWidgetView_YAxisRange_Custom_ToolTipText", "Custom range Y axis."))
				]

				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				[
					SNew(SNumericEntryBox<float>)
					.MinDesiredValueWidth(50.0f)
					.MinSliderValue_Lambda([this]() { return GetPlotColumnYRangeFloatMetaData(SelectedPlotColumnName, TEXT("UIMin")); })
					.MaxSliderValue_Lambda([this]() { return GetPlotColumnYRangeFloatMetaData(SelectedPlotColumnName, TEXT("UIMax")); })
					.MaxFractionalDigits(2)
					.AllowSpin(true)
					.SpinBoxStyle(&TransparentFillSpinBoxStyle)
					.IsEnabled_Lambda([this]() { return GetPlotColumnUseCustomYRange(SelectedPlotColumnName); })
					.Value_Lambda([this]()
					{
						const FFloatInterval DefaultRange = PlotWidget->GetYAxisRange();
						return GetPlotColumnCustomizedYRange(SelectedPlotColumnName).Get(DefaultRange).Min;
					})
					.OnValueChanged_Lambda([this](float Value) { SetPlotColumnCustomYRangeMin(SelectedPlotColumnName, Value); })
					.OnValueCommitted_Lambda([this](float Value, ETextCommit::Type CommitType) { SetPlotColumnCustomYRangeMin(SelectedPlotColumnName, Value, CommitType); })
					.Label()
					[
						SNumericEntryBox<float>::BuildLabel(LOCTEXT("SoundPlotsWidgetView_YAxisRangeMinLabel", "Min"), FLinearColor::White, FLinearColor::Transparent)
					]
				]

				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				[
					SNew(SNumericEntryBox<float>)
					.MinDesiredValueWidth(50.0f)
					.MinSliderValue_Lambda([this]() { return GetPlotColumnYRangeFloatMetaData(SelectedPlotColumnName, TEXT("UIMin")); })
					.MaxSliderValue_Lambda([this]() { return GetPlotColumnYRangeFloatMetaData(SelectedPlotColumnName, TEXT("UIMax")); })
					.MaxFractionalDigits(2)
					.AllowSpin(true)
					.SpinBoxStyle(&TransparentFillSpinBoxStyle)
					.IsEnabled_Lambda([this]() { return GetPlotColumnUseCustomYRange(SelectedPlotColumnName); })
					.Value_Lambda([this]()
					{
						const FFloatInterval DefaultRange = PlotWidget->GetYAxisRange();
						return GetPlotColumnCustomizedYRange(SelectedPlotColumnName).Get(DefaultRange).Max;
					})
					.OnValueChanged_Lambda([this](float Value) { SetPlotColumnCustomYRangeMax(SelectedPlotColumnName, Value); })
					.OnValueCommitted_Lambda([this](float Value, ETextCommit::Type CommitType) { SetPlotColumnCustomYRangeMax(SelectedPlotColumnName, Value, CommitType); })
					.Label()
					[
						SNumericEntryBox<float>::BuildLabel(LOCTEXT("SoundPlotsWidgetView_YAxisRangeMaxLabel", "Max"), FLinearColor::White, FLinearColor::Transparent)
					]
				]
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				PlotWidget.ToSharedRef()
			];
	}

	void FSoundPlotsWidgetView::ConstructPlotsWidgetContents()
	{
		// Initialize column options and initially selected column
		ColumnInfo.GenerateKeyArray(ColumnNames);

		if (ColumnNames.IsEmpty())
		{
			return;
		}

		SelectedPlotColumnName = ColumnNames[0];

		SAssignNew(PlotWidget, SAudioCurveView)
			.AutoRangeYAxis_Lambda([this]() { return !GetPlotColumnUseCustomYRange(SelectedPlotColumnName); })
			.ViewRange_Lambda([this]() { return GetViewRange(); })
			.PixelSnappingMethod(EWidgetPixelSnapping::Disabled)
			.AdditionalToolTipText_Lambda([this]() { return GetAdditionalTooltipText(); })
			.OnScrubPositionChanged_Lambda([this](const double TimestampPosition, const float YValue) { SetTimestampFromCurveView(TimestampPosition); })
			.OnKeyDown_Lambda([this](const FGeometry& Geometry, const FKeyEvent& KeyEvent) { return OnHandleKeyDown(Geometry, KeyEvent); });

		// Initialize curve data and metadata
		if (!PlotWidgetMetadataPerCurve.IsValid())
		{
			PlotWidgetMetadataPerCurve = MakeShared<TMap<uint64, SAudioCurveView::FCurveMetadata>>();
			for (const FName& ColumnName : ColumnNames)
			{
				TSharedPtr<FPointDataPerCurveMap> PointDataPerCurveMap = MakeShared<FPointDataPerCurveMap>();
				PlotWidgetCurveIdToPointDataMapPerColumn.Emplace(ColumnName, MoveTemp(PointDataPerCurveMap));
				ColumnNameToLatestTimestamp.Emplace(ColumnName, MakeShared<FLatestTimestampPerCurveMap>());
			}

			PlotWidget->SetCurvesMetadata(PlotWidgetMetadataPerCurve);
		}

		UpdatePlot();

		SAssignNew(ComboBox, SComboBox<FName>)
			.ToolTipText(LOCTEXT("SoundPlotsWidgetView_SelectPlotColumnDescription", "Select a column from the table to plot."))
			.OptionsSource(&ColumnNames)
			.OnGenerateWidget_Lambda([this](const FName& ColumnName)
			{
				return SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(GetPlotColumnDisplayName(ColumnName));
			})
			.OnSelectionChanged_Lambda([this](FName NewColumnName, ESelectInfo::Type)
			{
				SelectedPlotColumnName = NewColumnName;
				UpdatePlot();
#if !WITH_EDITOR
				bTimeMarkerMoved = true;
#endif
			})
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([this]()
				{
					return GetPlotColumnDisplayName(SelectedPlotColumnName);
				})
			];
	}

	void FSoundPlotsWidgetView::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FSoundPlotsWidgetCommands& Commands = FSoundPlotsWidgetCommands::Get();
		CommandList->MapAction(Commands.GetResetInspectTimestampCommand(), FExecuteAction::CreateLambda([this]() { ResetInspectTimestamp(); }));
	}

	void FSoundPlotsWidgetView::ResetInspectTimestamp()
	{
		FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().ResumeTimeMarker();
	}

	void FSoundPlotsWidgetView::CollectDataRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, bool& bOutHasNewMetadata)
	{
		using namespace FSoundPlotsWidgetViewPrivate;

		if (!Entry.IsValid())
		{
			return;
		}

		const uint64 SourceId = Entry->GetEntryID();
		const bool bPlotIsEnabled = GetIsPlotEnabledForEntry(*Entry);

		if (SourceId == INDEX_NONE || !bPlotIsEnabled)
		{
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
			{
				CollectDataRecursive(ChildEntry, SelectedEntries, bOutHasNewMetadata);
			}

			return;
		}

		// For each column, get the array for this data point's source id and add the value to that data array
		for (const auto& [ColumnName, DataMap] : PlotWidgetCurveIdToPointDataMapPerColumn)
		{
			// Get the data point array for this source id, or add new point
			TArray<FPlotCurvePoint>* DataPoints = DataMap->Find(SourceId);

			if (DataPoints == nullptr)
			{
				DataPoints = &DataMap->Add(SourceId);

				TSharedPtr<FLatestTimestampPerCurveMap>* LatestTimestamp = ColumnNameToLatestTimestamp.Find(ColumnName);
				if (LatestTimestamp && LatestTimestamp->IsValid() && !(*LatestTimestamp)->Contains(SourceId))
				{
					(*LatestTimestamp)->Add(SourceId);
				}
			}
			else if (GameState == EGameState::ControlledByTimemarker)
			{
				DataPoints->Empty(DataPoints->Num());
			}

			auto DataFunc = GetPlotColumnDataFunc(ColumnName);
			if (!DataFunc.IsSet())
			{
				continue;
			}

			const ::Audio::TCircularAudioBuffer<FDataPoint>& TimeStampedValues = (DataFunc)(*Entry);

			const ::Audio::DisjointedArrayView<const FDataPoint> TimeStampedValuesDisjointedArrayView = TimeStampedValues.PeekInPlace(TimeStampedValues.Num());

			const TFunction<float(const float)> UnitConversionFunc = GetPlotColumnUnitConversionFunc(ColumnName);

			for (const auto& [Timestamp, Value] : TimeStampedValuesDisjointedArrayView.FirstBuffer)
			{
				TSharedPtr<FLatestTimestampPerCurveMap>* LatestTimestampMap = ColumnNameToLatestTimestamp.Find(ColumnName);
				if (LatestTimestampMap && LatestTimestampMap->IsValid())
				{
					double* LatestColumnTimestep = (*LatestTimestampMap)->Find(SourceId);
					if (LatestColumnTimestep)
					{
						if (DataPoints->IsEmpty())
						{
							*LatestColumnTimestep = Timestamp;
						}
						else if (Timestamp <= *LatestColumnTimestep)
						{
							continue;
						}
						else
						{
							*LatestColumnTimestep = Timestamp;
						}
					}
				}

				if (GameState != EGameState::ControlledByTimemarker && GameState != EGameState::Paused)
				{
					CurrentTimestamp = FMath::Max(CurrentTimestamp, Timestamp);
				}

#if WITH_EDITOR
				const double DataPointTime = Timestamp - BeginTimestamp;
#else
				const double DataPointTime = Timestamp;
#endif // WITH_EDITOR

				if (DataPointTime >= 0.0)
				{
					DataPoints->Emplace(DataPointTime, UnitConversionFunc.IsSet() ? UnitConversionFunc(Value) : Value);
				}
			}
		}

		// Create metadata for this curve if necessary
		if (SAudioCurveView::FCurveMetadata* MetaData = PlotWidgetMetadataPerCurve->Find(SourceId))
		{
			const FLinearColor EntryColor = Entry->GetEntryColor();
			if (MetaData->CurveColor.A > 0.0f && MetaData->CurveColor != EntryColor)
			{
				Entry->SetEntryColor(MetaData->CurveColor);
			}

			const bool bPlotWasSelected = MetaData->PlotThickness != DefaultPlotThickness;
			if (bPlotWasSelected != SelectedEntries.Contains(Entry))
			{
				MetaData->PlotThickness = !bPlotWasSelected ? SelectedPlotThickness : DefaultPlotThickness;
				bOutHasNewMetadata = true;
			}
		}
		else
		{
			FPlotCurveMetadata& NewMetadata = PlotWidgetMetadataPerCurve->Add(SourceId);
			NewMetadata.CurveColor = Entry->GetEntryColor();
			NewMetadata.DisplayName = Entry->GetDisplayName();
			NewMetadata.PlotThickness = SelectedEntries.Contains(Entry) ? SelectedPlotThickness : DefaultPlotThickness;
			bOutHasNewMetadata = true;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& ChildEntry : Entry->Children)
		{
			CollectDataRecursive(ChildEntry, SelectedEntries, bOutHasNewMetadata);
		}
	}

	void FSoundPlotsWidgetView::UpdatePlot()
	{
		const TSharedPtr<FPointDataPerCurveMap>* const DataMap = PlotWidgetCurveIdToPointDataMapPerColumn.Find(SelectedPlotColumnName);
		if (!DataMap || !DataMap->IsValid())
		{
			ensureMsgf(false, TEXT("Invalid FPointDataPerCurveMap in FSoundPlotsWidgetView::UpdatePlot"));
			return;
		}

		const FNumberFormattingOptions* const PlotColumnNumberFormat = GetPlotColumnNumberFormat(SelectedPlotColumnName);
		if (!PlotColumnNumberFormat)
		{
			ensureMsgf(false, TEXT("Invalid FNumberFormattingOptions in FSoundPlotsWidgetView::UpdatePlot"));
			return;
		}

		if (!PlotWidget.IsValid())
		{
			ensureMsgf(false, TEXT("Invalid SAudioCurveView in FSoundPlotsWidgetView::UpdatePlot"));
			return;
		}

		PlotWidget->SetCurvesPointData(*DataMap);
		PlotWidget->SetYValueFormattingOptions(*PlotColumnNumberFormat);
		PlotWidget->SetYAxisRange(GetPlotColumnCustomizedOrDefaultYRange(SelectedPlotColumnName));
	}

	void FSoundPlotsWidgetView::ResetPlots()
	{
		for (const auto& [ColumnName, PointDataPerCurveMap] : PlotWidgetCurveIdToPointDataMapPerColumn)
		{
			PointDataPerCurveMap->Empty();
		}

		for (auto& [ColumnName, LatestTimestampsPerPlot] : ColumnNameToLatestTimestamp)
		{
			LatestTimestampsPerPlot->Empty();
		}

		if (PlotWidgetMetadataPerCurve.IsValid())
		{
			PlotWidgetMetadataPerCurve->Empty();
		}

		BeginTimestamp = TNumericLimits<double>::Max();
		CurrentTimestamp = TNumericLimits<double>::Lowest();
	}

	void FSoundPlotsWidgetView::UpdatePlotVisibilityRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry)
	{
		if (!Entry.IsValid())
		{
			return;
		}

		if (FPlotCurveMetadata* PlotMetaData = PlotWidgetMetadataPerCurve->Find(Entry->GetEntryID()))
		{
			PlotMetaData->CurveColor.A = GetIsPlotEnabledForEntry(*Entry) ? 1.0f : 0.0f;
		}

		for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : Entry->Children)
		{
			UpdatePlotVisibilityRecursive(Child);
		}
	}

	const TFunction<const::Audio::TCircularAudioBuffer<FDataPoint>& (const IDashboardDataTreeViewEntry& InData)> FSoundPlotsWidgetView::GetPlotColumnDataFunc(const FName& ColumnName)
	{
		return ColumnInfo.Find(ColumnName)->DataFunc;
	}

	const TFunction<float(const float)> FSoundPlotsWidgetView::GetPlotColumnUnitConversionFunc(const FName& ColumnName)
	{
		return ColumnInfo.Find(ColumnName)->UnitConversionFunc;
	}

	const FNumberFormattingOptions* FSoundPlotsWidgetView::GetPlotColumnNumberFormat(const FName& ColumnName)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->GetFormatOptionsFunc.IsSet())
			{
				return PlotColumnInfo->GetFormatOptionsFunc();
			}
		}
		return nullptr;
	}

	FFloatInterval FSoundPlotsWidgetView::GetPlotColumnCustomizedOrDefaultYRange(const FName& ColumnName)
	{
		// First get values from the range property metadata if available, or 0 to 1 if not available:
		const float DefaultMin = GetPlotColumnYRangeFloatMetaData(ColumnName, TEXT("UIMin")).Get(0.0f);
		const float DefaultMax = GetPlotColumnYRangeFloatMetaData(ColumnName, TEXT("UIMax")).Get(1.0f);

		// Use the customized Y range, or the default values otherwise:
		return GetPlotColumnCustomizedYRange(ColumnName).Get({ DefaultMin, DefaultMax });
	}

	const FText FSoundPlotsWidgetView::GetPlotColumnDisplayName(const FName& ColumnName)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			return PlotColumnInfo->ColumnDisplayName;
		}
		return FText::GetEmpty();
	}

	FText FSoundPlotsWidgetView::GetAdditionalTooltipText() const
	{
		FAudioInsightsModule& AudioInsightsModule = UE::Audio::Insights::FAudioInsightsModule::GetChecked();
		if (AudioInsightsModule.GetTimingViewExtender().GetMessageCacheAndProcessingStatus() != ECacheAndProcess::Latest)
		{
			return LOCTEXT("SoundPlotsWidgetView_ResetTimeControlMethod", "Audio Insights has been paused. Press Esc to resume.");
		}

		return LOCTEXT("SoundPlotsWidgetView_InspectTimestampHint", "Left Click to inspect timestamp.");
	}

	TOptional<float> FSoundPlotsWidgetView::GetPlotColumnYRangeFloatMetaData(const FName& ColumnName, const FName& MetaDataKey)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->GetYRangePropertyFunc.IsSet())
			{
				if (const FProperty* YRangeProperty = PlotColumnInfo->GetYRangePropertyFunc())
				{
					if (YRangeProperty->HasMetaData(MetaDataKey))
					{
						return YRangeProperty->GetFloatMetaData(MetaDataKey);
					}
				}
			}
		}
		return NullOpt;
	}

	TOptional<FFloatInterval> FSoundPlotsWidgetView::GetPlotColumnCustomizedYRange(const FName& ColumnName)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->GetCustomYRangeFunc.IsSet() && GetPlotColumnUseCustomYRange(ColumnName))
			{
				return PlotColumnInfo->GetCustomYRangeFunc();
			}
		}
		return NullOpt;
	}

	bool FSoundPlotsWidgetView::GetPlotColumnUseCustomYRange(const FName& ColumnName)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			return (PlotColumnInfo->GetUseCustomYRangeFunc.IsSet() && PlotColumnInfo->GetUseCustomYRangeFunc());
		}
		return true;
	}

	void FSoundPlotsWidgetView::SetPlotColumnUseCustomYRange(const FName& ColumnName, bool bUseCustomYRange)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->SetUseCustomYRangeFunc.IsSet())
			{
				PlotColumnInfo->SetUseCustomYRangeFunc(bUseCustomYRange);
			}
		}
	}

	void FSoundPlotsWidgetView::SetPlotColumnCustomYRangeMin(const FName& ColumnName, float Value, TOptional<ETextCommit::Type> CommitType)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->SetCustomYRangeFunc.IsSet() && PlotColumnInfo->GetCustomYRangeFunc.IsSet())
			{
				FFloatInterval Range = PlotColumnInfo->GetCustomYRangeFunc();
				Range.Min = Value;
				PlotColumnInfo->SetCustomYRangeFunc(Range, CommitType);
			}
		}
	}

	void FSoundPlotsWidgetView::SetPlotColumnCustomYRangeMax(const FName& ColumnName, float Value, TOptional<ETextCommit::Type> CommitType)
	{
		if (const FSoundPlotsWidgetView::FPlotColumnInfo* PlotColumnInfo = ColumnInfo.Find(ColumnName))
		{
			if (PlotColumnInfo->SetCustomYRangeFunc.IsSet() && PlotColumnInfo->GetCustomYRangeFunc.IsSet())
			{
				FFloatInterval Range = PlotColumnInfo->GetCustomYRangeFunc();
				Range.Max = Value;
				PlotColumnInfo->SetCustomYRangeFunc(Range, CommitType);
			}
		}
	}

	void FSoundPlotsWidgetView::SetTimestampFromCurveView(const double Timestamp)
	{
		// Timestamp from SAudioCurve is a relative one - we need to transform it back to the actual timestamp
#if WITH_EDITOR
		const double AbsoluteTimestamp = Timestamp + BeginTimestamp;
		const TRange<double> AbsolutePlottingRange = TRange<double>(CurrentRange.GetLowerBoundValue() + BeginTimestamp, CurrentRange.GetUpperBoundValue() + BeginTimestamp);
#else
		const double AbsoluteTimestamp = Timestamp;
		const TRange<double> AbsolutePlottingRange = CurrentRange;
#endif // WITH_EDITOR

		FAudioInsightsModule& AudioInsightsModule = UE::Audio::Insights::FAudioInsightsModule::GetChecked();
		AudioInsightsModule.GetTimingViewExtender().PauseTimeMarker(AbsoluteTimestamp, ESystemControllingTimeMarker::PlotsWidget, AbsolutePlottingRange);
	}

	TRange<double> FSoundPlotsWidgetView::GetViewRange()
	{
		const FAudioInsightsModule& AudioInsightsModule = UE::Audio::Insights::FAudioInsightsModule::GetChecked();
		TOptional<ESystemControllingTimeMarker> ControllingSystem = AudioInsightsModule.GetTimingViewExtender().TryGetSystemControllingTimeMarker();
		if (ControllingSystem.IsSet() && ControllingSystem.GetValue() == ESystemControllingTimeMarker::PlotsWidget)
		{
			return CurrentRange;
		}

		if (BeginTimestamp == TNumericLimits<double>::Max())
		{
			return TRange<double>(0, FAudioInsightsTimingViewExtender::MaxPlottingHistorySeconds);
		}

		double RangeUpperBound = 0.0;

#if WITH_EDITOR
		const bool bAnyMessageReceived = CurrentTimestamp != TNumericLimits<double>::Lowest();
		if (!bAnyMessageReceived && !bIsTraceActive)
		{
			return CurrentRange;
		}

		const double CurrentTime = FPlatformTime::Seconds() - GStartTime;

		double TimestampsDiff = 0.0;

		if (GameState != EGameState::Paused && GameState != EGameState::ControlledByTimemarker && bAnyMessageReceived)
		{
			const double RelativeCurrentTime = CurrentTime - BeginTimestamp;
			TimestampsDiff = RelativeCurrentTime - (CurrentTimestamp - BeginTimestamp);
		}

		const double FinalCurrentTime = bAnyMessageReceived ? CurrentTimestamp : CurrentTime;

		constexpr double RangeAlignmentOffset = 0.2;
		RangeUpperBound = FinalCurrentTime - BeginTimestamp + TimestampsDiff - RangeAlignmentOffset;
#else
		if (GameState == EGameState::ControlledByTimemarker || !IAudioInsightsModule::IsLiveSession())
		{
			CurrentRange = TRange<double>(CurrentTimestamp - FAudioInsightsTimingViewExtender::MaxPlottingHistorySeconds, CurrentTimestamp);
			return CurrentRange;
		}
		else
		{
			const double CurrentTime = FPlatformTime::Seconds();
			const double DeltaTime = CurrentTime - PreviousTime;

			PreviousTime = CurrentTime;

			const double TraceCurrentDurationSeconds = FAudioInsightsModule::GetChecked().GetTimingViewExtender().GetCurrentTraceDurationSeconds();

			CurrentRangeUpperBound = FMath::FInterpTo(CurrentRangeUpperBound, TraceCurrentDurationSeconds, DeltaTime, 1.0);

			constexpr double RangeAlignmentOffset = 0.9;
			RangeUpperBound = CurrentRangeUpperBound + RangeAlignmentOffset;
		}
#endif // WITH_EDITOR

		CurrentRange = TRange<double>(RangeUpperBound - FAudioInsightsTimingViewExtender::MaxPlottingHistorySeconds, RangeUpperBound);
		return CurrentRange;
	}

	bool FSoundPlotsWidgetView::GetIsPlotEnabledForEntry(const IDashboardDataTreeViewEntry& InEntry) const
	{
		return IsPlotEnabledForEntryFunc.IsSet() && IsPlotEnabledForEntryFunc(InEntry);
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
