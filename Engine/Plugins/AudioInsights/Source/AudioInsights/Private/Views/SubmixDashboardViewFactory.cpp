// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/SubmixDashboardViewFactory.h"

#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTimingViewExtender.h"
#include "IAudioInsightsModule.h"
#include "Internationalization/Text.h"
#include "Providers/AudioMeterProvider.h"
#include "Providers/SubmixTraceProvider.h"
#include "Widgets/Input/SCheckBox.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#else
#include "AudioInsightsComponent.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace FSubmixDashboardViewFactoryPrivate
	{
		const FSubmixDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FSubmixDashboardEntry&>(InData);
		};

		const FSlateColor SubmixMeterTextColor(FColor(30, 240, 90));
	}

	FSubmixDashboardViewFactory::FSubmixDashboardViewFactory()
	{
		IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();

		SubmixProvider = MakeShared<FSubmixTraceProvider>();

		AudioInsightsTraceModule.AddTraceProvider(SubmixProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			SubmixProvider
		};

		SortByColumn = "Name";
		SortMode     = EColumnSortMode::Ascending;

		SubmixProvider->OnSubmixAdded.BindRaw(this, &FSubmixDashboardViewFactory::HandleOnSubmixAdded);
		SubmixProvider->OnSubmixRemoved.BindRaw(this, &FSubmixDashboardViewFactory::HandleOnSubmixRemoved);
		SubmixProvider->OnSubmixLoaded.BindRaw(this, &FSubmixDashboardViewFactory::HandleOnSubmixLoaded);
		SubmixProvider->OnSubmixListUpdated.BindRaw(this, &FSubmixDashboardViewFactory::RequestListRefresh);
		SubmixProvider->OnTimeMarkerUpdated.BindRaw(this, &FSubmixDashboardViewFactory::HandleOnTimeMarkerUpdated);
	}

	FSubmixDashboardViewFactory::~FSubmixDashboardViewFactory()
	{
		SubmixProvider->OnSubmixAdded.Unbind();
		SubmixProvider->OnSubmixRemoved.Unbind();
		SubmixProvider->OnSubmixLoaded.Unbind();
		SubmixProvider->OnSubmixListUpdated.Unbind();
		SubmixProvider->OnTimeMarkerUpdated.Unbind();
	}

	FName FSubmixDashboardViewFactory::GetName() const
	{
		return "Submix";
	}

	FText FSubmixDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_Submix_DisplayName", "Submixes");
	}
	
	FSlateIcon FSubmixDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Submix");
	}

	EDefaultDashboardTabStack FSubmixDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Analysis;
	}

	TSharedRef<SWidget> FSubmixDashboardViewFactory::GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& InColumnName)
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		if (InColumnName == "Active")
		{
			static const FLinearColor DarkGreen(0.027f, 0.541f, 0.22f);
			static const float Radius = 4.0f;
			static const FVector2f Size(7.0f, 7.0f);
			static const FSlateRoundedBoxBrush GreenRoundedBrush(DarkGreen, Radius, Size);

			return SNew(SBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Padding(6.0f)
				.Visibility_Lambda([InRowData]()
				{ 
					const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(InRowData.Get());
					return SubmixDashboardEntry.bHasActivity ? EVisibility::Visible : EVisibility::Hidden;
				})
				[
					SNew(SImage)
					.Image(&GreenRoundedBrush)
				];
		}
		else if (InColumnName == "Name")
		{
			const FColumnData& ColumnData = GetColumns()[InColumnName];
			const FText ValueText = ColumnData.GetDisplayValue(InRowData.Get());

			if (ValueText.IsEmpty())
			{
				return SNullWidget::NullWidget;
			}

			const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*InRowData);

			if (ShouldUpdateAudioMeter(SubmixDashboardEntry) && !SubmixDashboardEntry.AudioMeterInfo->EnvelopeValues.IsEmpty())
			{
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName());
			}

			return SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsEnabled_Lambda([this, InRowData]()
					{
#if WITH_EDITOR
						return SubmixProvider && SubmixProvider->GetMessageCacheAndProcessingStatus() == ECacheAndProcess::Latest;
#else
						return IAudioInsightsModule::IsLiveSession();
#endif // WITH_EDITOR
					})
					.IsChecked_Lambda([this, InRowData]()
					{
						const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*InRowData);

						if (CheckedSubmixes.Contains(SubmixDashboardEntry.SubmixId))
						{
							return ECheckBoxState::Checked;
						}

#if !WITH_EDITOR
						if (!IAudioInsightsModule::IsLiveSession())
						{
							return SubmixDashboardEntry.bEnvelopeFollowerEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
#endif // !WITH_EDITOR

						return ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this, InRowData](ECheckBoxState NewState)
					{
						const bool bIsChecked = NewState == ECheckBoxState::Checked;

						// When the clicked row is part of the current multi-selection, the new state propagates to
						// every selected row (bulk-enable after Shift+Click). Otherwise it only affects the clicked row.
						TArray<TSharedPtr<IDashboardDataViewEntry>> EntriesToToggle;

						if (FilteredEntriesListView.IsValid())
						{
							const TArray<TSharedPtr<IDashboardDataViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

							if (SelectedItems.Contains(InRowData.ToSharedPtr()))
							{
								EntriesToToggle = SelectedItems;
							}
						}

						// Clicked row is not part of the current selection: toggle only that one row.
						if (EntriesToToggle.IsEmpty())
						{
							EntriesToToggle.Emplace(InRowData.ToSharedPtr());
						}

						for (const TSharedPtr<IDashboardDataViewEntry>& Entry : EntriesToToggle)
						{
							if (Entry.IsValid())
							{
								const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);
								SetSubmixEnvelopeFollowerEnabled(SubmixDashboardEntry, bIsChecked);
							}
						}
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(5.0f)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(ValueText)
					.ToolTipText(ValueText)
					.OnDoubleClicked_Lambda([this, InRowData](const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
					{
#if WITH_EDITOR
						if (GEditor)
						{
							const TSharedPtr<IObjectDashboardEntry> ObjectData = StaticCastSharedPtr<IObjectDashboardEntry>(InRowData.ToSharedPtr());
							if (ObjectData.IsValid())
							{
								const TObjectPtr<UObject> Object = ObjectData->GetObject();
								if (Object && Object->IsAsset())
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
									return FReply::Handled();
								}
							}
						}
#endif // WITH_EDITOR

						return FReply::Unhandled();
					})
				];
		}

		return SNullWidget::NullWidget;
	}

	void FSubmixDashboardViewFactory::ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason)
	{
		const FString FilterString = GetSearchFilterText().ToString();
		
		FTraceTableDashboardViewFactory::FilterEntries<FSubmixTraceProvider>([&FilterString](const IDashboardDataViewEntry& Entry)
		{
			const FSubmixDashboardEntry& SubmixEntry = static_cast<const FSubmixDashboardEntry&>(Entry);
			
			return SubmixEntry.GetDisplayName().ToString().Contains(FilterString);
		});

		if (bShouldRebuildAudioMeters)
		{
			RebuildAudioMeters();
			bShouldRebuildAudioMeters = false;
		}
		else if (bListRefreshed)
		{
			UpdateAudioMeters();
			bListRefreshed = false;
		}
		else if (bShouldReactivateAudioMeters)
		{
			ReactivateAudioMeters();
			bShouldReactivateAudioMeters = false;
		}
	}

	TSharedRef<SWidget> FSubmixDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			FAudioInsightsModule& AudioInsightsModule = FAudioInsightsModule::GetChecked();
			AudioInsightsModule.GetTimingViewExtender().OnTimeControlMethodReset.AddSP(this, &FSubmixDashboardViewFactory::HandleOnTimeControlMethodReset);

			DashboardWidget = FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs);
		}
#if WITH_EDITOR
		else
		{
			if (SubmixProvider.IsValid())
			{
				SubmixProvider->RequestEntriesUpdate();
			}
		}
#endif // WITH_EDITOR

		// Update audio meters (if any has been checked previously)
		UpdateAudioMeters();

		return DashboardWidget->AsShared();
	}

	void FSubmixDashboardViewFactory::SendSubmixEnvelopeFollowerCVar(const bool bEnableEnvelopeFollower, const FSubmixDashboardEntry& SubmixDashboardEntry)
	{
#if !WITH_EDITOR
		if (SubmixDashboardEntry.IsMainSubmix() && !bEnableEnvelopeFollower)
		{
			// Don't disable envelope follower on main submix.
			return;
		}
#endif // !WITH_EDITOR

		const FString InSubmixName = SubmixDashboardEntry.GetDisplayName().ToString();
#if WITH_EDITOR
		const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
#else
		const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
#endif // WITH_EDITOR

		const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();

		const FString SubmixCommandStr = FString::Printf(TEXT("au.%sSubmixEnvelopeFollower_AD%d %s"), bEnableEnvelopeFollower ? TEXT("Start") : TEXT("Stop"), AudioDeviceId, *InSubmixName);

		IAudioInsightsTraceModule& AudioInsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();
		AudioInsightsTraceModule.ExecuteConsoleCommand(SubmixCommandStr);
	}

	void FSubmixDashboardViewFactory::SetSubmixEnvelopeFollowerEnabled(const FSubmixDashboardEntry& InSubmixDashboardEntry, const bool bInEnabled)
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		SendSubmixEnvelopeFollowerCVar(bInEnabled, InSubmixDashboardEntry);

		if (bInEnabled)
		{
			CheckedSubmixes.Emplace(InSubmixDashboardEntry.SubmixId);
			FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(InSubmixDashboardEntry.SubmixId, InSubmixDashboardEntry.AudioMeterInfo, InSubmixDashboardEntry.GetDisplayName(), SubmixMeterTextColor);
		}
		else
		{
			CheckedSubmixes.Remove(InSubmixDashboardEntry.SubmixId);
			FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(InSubmixDashboardEntry.SubmixId);
		}
	}

	const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& FSubmixDashboardViewFactory::GetColumns() const
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		auto CreateColumnData = []()
		{
			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					"Active",
					{
						LOCTEXT("SubmixDashboard_ActiveDisplayName", "Active"),
						[](const IDashboardDataViewEntry& InData) { return FText::GetEmpty(); },
						nullptr		/* GetIconName */,
						false,		/* bDefaultHidden */
						0.08f,		/* FillWidth */
						EHorizontalAlignment::HAlign_Center
					}
				},
				{
					"Name",
					{
						LOCTEXT("SubmixDashboard_NameColumnDisplayName", "Name"),
						[](const IDashboardDataViewEntry& InData) { return CastEntry(InData).GetDisplayName(); },
						nullptr		/* GetIconName */,
						false,		/* bDefaultHidden */
						0.92f,		/* FillWidth */
						EHorizontalAlignment::HAlign_Left
					}
				}
			};
		};
		
		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		
		return ColumnData;
	}

	void FSubmixDashboardViewFactory::SortTable()
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		if (SortByColumn == "Active")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FSubmixDashboardEntry& AData = CastEntry(*A.Get());
					const FSubmixDashboardEntry& BData = CastEntry(*B.Get());

					return !AData.bHasActivity && BData.bHasActivity;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FSubmixDashboardEntry& AData = CastEntry(*A.Get());
					const FSubmixDashboardEntry& BData = CastEntry(*B.Get());

					return !BData.bHasActivity && AData.bHasActivity;
				});
			}
		}
		else if (SortByColumn == "Name")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FSubmixDashboardEntry& AData = CastEntry(*A.Get());
					const FSubmixDashboardEntry& BData = CastEntry(*B.Get());

					return AData.GetDisplayName().CompareToCaseIgnored(BData.GetDisplayName()) < 0;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FSubmixDashboardEntry& AData = CastEntry(*A.Get());
					const FSubmixDashboardEntry& BData = CastEntry(*B.Get());

					return BData.GetDisplayName().CompareToCaseIgnored(AData.GetDisplayName()) < 0;
				});
			}
		}
	}

	void FSubmixDashboardViewFactory::RebuildAudioMeters() const
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

			if (ShouldUpdateAudioMeter(SubmixDashboardEntry))
			{
				FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName(), SubmixMeterTextColor);

				if (!SubmixDashboardEntry.AudioMeterInfo->EnvelopeValues.IsEmpty())
				{
					FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName());
				}
			}
			else
			{
				FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(SubmixDashboardEntry.SubmixId);
			}
		}
	}

	bool FSubmixDashboardViewFactory::ShouldUpdateAudioMeter(const FSubmixDashboardEntry& InSubmixDashboardEntry) const
	{
#if WITH_EDITOR
		if (!SubmixProvider.IsValid())
		{
			return false;
		}
#endif // WITH_EDITOR

		if (CheckedSubmixes.Contains(InSubmixDashboardEntry.SubmixId))
		{
			return true;
		}

#if !WITH_EDITOR
		if (!IAudioInsightsModule::IsLiveSession())
		{
			return InSubmixDashboardEntry.bEnvelopeFollowerEnabled;
		}
#endif // !WITH_EDITOR

		return false;
	}

	void FSubmixDashboardViewFactory::UpdateAudioMeters() const
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		for (const TSharedPtr<IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

			if (CheckedSubmixes.Find(SubmixDashboardEntry.SubmixId) != nullptr)
			{
				SendSubmixEnvelopeFollowerCVar(true, SubmixDashboardEntry);
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName());
			}
		}
	}

	void FSubmixDashboardViewFactory::HandleOnSubmixAdded(const uint32 InSubmixId)
	{
		RequestListRefresh();
	}

	void FSubmixDashboardViewFactory::HandleOnSubmixRemoved(const uint32 InSubmixId)
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		const TSharedPtr<IDashboardDataViewEntry>* FoundEntry = DataViewEntries.FindByPredicate([&InSubmixId](const TSharedPtr<IDashboardDataViewEntry>& Entry)
		{
			const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

			return SubmixDashboardEntry.SubmixId == InSubmixId;
		});

		if (FoundEntry)
		{
			const TSharedPtr<IDashboardDataViewEntry> Entry   = *FoundEntry;
			const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

			SendSubmixEnvelopeFollowerCVar(false, SubmixDashboardEntry);
		}

		FAudioMeterProvider::OnRemoveAudioMeter.ExecuteIfBound(InSubmixId);
		RequestListRefresh();
	}

	void FSubmixDashboardViewFactory::ReactivateAudioMeters()
	{
		using namespace FSubmixDashboardViewFactoryPrivate;

		for (auto It = PendingToReactivateSubmixIds.CreateIterator(); It; ++It)
		{
			const TSharedPtr<IDashboardDataViewEntry>* FoundEntry = DataViewEntries.FindByPredicate([&InSubmixId = *It](const TSharedPtr<IDashboardDataViewEntry>& Entry)
				{
					const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

					return SubmixDashboardEntry.SubmixId == InSubmixId;
				});

			if (FoundEntry)
			{
				const TSharedPtr<IDashboardDataViewEntry> Entry = *FoundEntry;
				const FSubmixDashboardEntry& SubmixDashboardEntry = CastEntry(*Entry);

				SendSubmixEnvelopeFollowerCVar(true, SubmixDashboardEntry);
				FAudioMeterProvider::OnAddAudioMeter.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName(), SubmixMeterTextColor);
				FAudioMeterProvider::OnUpdateAudioMeterInfo.ExecuteIfBound(SubmixDashboardEntry.SubmixId, SubmixDashboardEntry.AudioMeterInfo, SubmixDashboardEntry.GetDisplayName());

				It.RemoveCurrent();
			}
		}
	}

	void FSubmixDashboardViewFactory::HandleOnSubmixLoaded(const uint32 InSubmixId)
	{
		if (CheckedSubmixes.Contains(InSubmixId))
		{
			PendingToReactivateSubmixIds.Emplace(InSubmixId);
			bShouldReactivateAudioMeters = true;
		}
	}

	void FSubmixDashboardViewFactory::HandleOnTimeMarkerUpdated()
	{
		bShouldRebuildAudioMeters = true;
	}

	void FSubmixDashboardViewFactory::HandleOnTimeControlMethodReset()
	{
		bShouldRebuildAudioMeters = true;
	}

	void FSubmixDashboardViewFactory::RequestListRefresh()
	{
		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->RequestListRefresh();
		}

		bListRefreshed = true;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
