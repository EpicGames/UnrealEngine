// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/TableDashboardViewFactory.h"

#include "Algo/Transform.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceProviderBase.h"
#include "Containers/Array.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"

#if WITH_EDITOR
#include "AudioDeviceManager.h"
#include "AudioInsightsDashboardAssetCommands.h"
#include "AudioInsightsDetailsSelectionManager.h"
#include "AudioInsightsModule.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	void FTraceTableDashboardViewFactory::SRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<IDashboardDataViewEntry> InData, TSharedRef<FTraceTableDashboardViewFactory> InFactory)
	{
		Data = InData;
		Factory = InFactory;

		FSuperRowType::FArguments Args = InArgs._Style != nullptr ? FSuperRowType::FArguments().Style(InArgs._Style) : FSuperRowType::FArguments();

		SMultiColumnTableRow<TSharedPtr<IDashboardDataViewEntry>>::Construct(Args, InOwnerTable);
	}

	TSharedRef<SWidget> FTraceTableDashboardViewFactory::SRowWidget::GenerateWidgetForColumn(const FName& Column)
	{
		return Factory->GenerateWidgetForColumn(Data->AsShared(), Column);
	}

	FTraceTableDashboardViewFactory::FTraceTableDashboardViewFactory()
		: FTraceDashboardViewFactoryBase()
	{
	}

	TSharedRef<SWidget> FTraceTableDashboardViewFactory::GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column)
	{
		const FColumnData& ColumnData = GetColumns()[Column];
		const FText ValueText = ColumnData.GetDisplayValue.IsSet() ? ColumnData.GetDisplayValue(InRowData.Get()) : FText::GetEmpty();
		const FName IconName = ColumnData.GetIconName.IsSet() ? ColumnData.GetIconName(InRowData.Get()) : NAME_None;

		if (ValueText.IsEmpty() && IconName == NAME_None)
		{
			return SNullWidget::NullWidget;
		}

		return SNew(SHorizontalBox)

		// Icon
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 2.0f)
		.HAlign(ColumnData.Alignment)
		.AutoWidth()
		[
			ColumnData.GetIconName.IsSet()

			? SNew(SImage)
			.Image_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return FSlateStyle::Get().GetBrush(ColumnData.GetIconName(InRowData.Get()));
			})
			.ColorAndOpacity_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return ColumnData.GetIconColor.IsSet() ? ColumnData.GetIconColor(InRowData.Get()) : FLinearColor::White;
			})
			.ToolTipText_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return ColumnData.GetIconTooltip.IsSet() ? ColumnData.GetIconTooltip(InRowData.Get()) : FText::GetEmpty();
			})
			.Visibility_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return ColumnData.GetIconName(InRowData.Get()) == NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
			})

			: SNullWidget::NullWidget
		]

		+ SHorizontalBox::Slot()
		.Padding(IconName != NAME_None ? 10.0f : 0.0f, 2.0f)
		.FillWidth(0.9f)
		[
			ValueText.IsEmpty()

			? SNew(STextBlock)
			.Text_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				const FText ValueText = ColumnData.GetDisplayValue(InRowData.Get());

				return ValueText;
			})
			.ToolTipText_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return ColumnData.GetDisplayValue.IsSet() ? ColumnData.GetDisplayValue(InRowData.Get()) : FText::GetEmpty();
			})

			: SNullWidget::NullWidget
		];
	}

	FReply FTraceTableDashboardViewFactory::OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const
	{
		return FReply::Unhandled();
	}

	EColumnSortMode::Type FTraceTableDashboardViewFactory::GetColumnSortMode(const FName InColumnId) const
	{
		return (SortByColumn == InColumnId) ? SortMode : EColumnSortMode::None;
	}

	void FTraceTableDashboardViewFactory::RequestSort()
	{
		SortTable();

		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->RequestListRefresh();
		}
	}

	void FTraceTableDashboardViewFactory::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
	{
		// Sorting can be disabled on specified columns
		if (!IsColumnSortable(InColumnId))
		{
			return;
		}

		SortByColumn = InColumnId;
		SortMode = InSortMode;
		RequestSort();
	}

	TSharedRef<SWidget> FTraceTableDashboardViewFactory::CreateHorizontalScrollBox()
	{
		if (!FilteredEntriesListView.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		return SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
			
				+ SScrollBox::Slot()
				.Padding(0)
				.FillSize(1.0f)
				.HAlign(HAlign_Fill)
				[
					FilteredEntriesListView->AsShared()
				];
	}

	TSharedRef<SWidget> FTraceTableDashboardViewFactory::CreateSettingsButtonWidget()
	{
		// Override OnGetSettingsMenuContent in derived class to enable the settings menu widget
		if (OnGetSettingsMenuContent() == SNullWidget::NullWidget)
		{
			return SNullWidget::NullWidget;
		}

		return SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &FTraceTableDashboardViewFactory::OnGetSettingsMenuContent)
			.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
			.HasDownArrow(false)
			.ToolTipText(OnGetSettingsMenuToolTip())
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Settings"))
				]
			];
	}

	TSharedPtr<SWidget> FTraceTableDashboardViewFactory::OnConstructContextMenu()
	{
		return SNullWidget::NullWidget;
	}

	void FTraceTableDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		// To be optionally implemented by derived classes
	}

	FSlateColor FTraceTableDashboardViewFactory::GetRowColor(const TSharedPtr<IDashboardDataViewEntry>& InRowDataPtr)
	{
		return FSlateColor(FColor(255, 255, 255));
	}

	TSharedRef<SHeaderRow> FTraceTableDashboardViewFactory::MakeHeaderRowWidget()
	{
		TArray<FName> DefaultHiddenColumns;
		Algo::TransformIf(GetColumns(), DefaultHiddenColumns,
			[](const TPair<FName, FColumnData>& ColumnInfo) { return ColumnInfo.Value.bDefaultHidden; },
			[](const TPair<FName, FColumnData>& ColumnInfo) { return ColumnInfo.Key; });

		SAssignNew(HeaderRowWidget, SHeaderRow)
		.CanSelectGeneratedColumn(true) // Allows for showing/hiding columns
		.OnHiddenColumnsListChanged(this, &FTraceTableDashboardViewFactory::OnHiddenColumnsListChanged);

		// This only works if header row columns are added with slots and not programmatically
		// check in SHeaderRow::Construct: for ( FColumn* const Column : InArgs.Slots ) for more info
		// A potential alternative would be to delegate to the derived classes the SHeaderRow creation with slots
		//.HiddenColumnsList(DefaultHiddenColumns);

		for (const auto& [ColumnName, ColumnData] : GetColumns())
		{
			const SHeaderRow::FColumn::FArguments ColumnArgs = SHeaderRow::Column(ColumnName)
				.DefaultLabel(ColumnData.DisplayName)
				.HAlignCell(ColumnData.Alignment)
				.FillWidth(ColumnData.FillWidth)
				.SortMode(this, &FTraceTableDashboardViewFactory::GetColumnSortMode, ColumnName)
				.OnSort(this, &FTraceTableDashboardViewFactory::OnColumnSortModeChanged)
				.HeaderContent()
				[
					SNew(SHorizontalBox)

					// Icon (optional)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(ColumnData.HeaderRowIconName != NAME_None ? 4.0f : 0.0f, 3.0f)
					[
						ColumnData.HeaderRowIconName != NAME_None
							? SNew(SImage).Image(FSlateStyle::Get().GetBrush(ColumnData.HeaderRowIconName)).ToolTipText(ColumnData.HeaderRowTooltip)
							: SNullWidget::NullWidget
					]
					// Text (optional)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 3.0f, 0.0f, 3.0f)
					[
						ColumnData.bShowDisplayName && !ColumnData.DisplayName.IsEmpty()
							? SNew(STextBlock).Text(ColumnData.DisplayName).ToolTipText(ColumnData.HeaderRowTooltip)
							: SNullWidget::NullWidget
					]
				];

			// .HiddenColumnsList workaround:
			// simulate what SHeaderRow::AddColumn( const FColumn::FArguments& NewColumnArgs ) does but allowing us to modify the bIsVisible property
			// Memory handling (delete) is done by TIndirectArray<FColumn> Columns; defined in SHeaderRow
			SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(ColumnArgs);
			NewColumn->bIsVisible = !DefaultHiddenColumns.Contains(ColumnName);
			HeaderRowWidget->AddColumn(*NewColumn);
		}

		return HeaderRowWidget.ToSharedRef();
	}

	TSharedRef<SWidget> FTraceTableDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			FilterBarButton = CreateFilterBarButtonWidget();

			FilteredEntriesListView = SNew(SScrollableListView<TSharedPtr<IDashboardDataViewEntry>>)
			.ListItemsSource(&DataViewEntries)
			.OnContextMenuOpening(this, &FTraceTableDashboardViewFactory::OnConstructContextMenu)
			.OnSelectionChanged(this, &FTraceTableDashboardViewFactory::OnSelectionChanged)
			.OnListViewScrolled(this, &FTraceTableDashboardViewFactory::OnListViewScrolled)
			.OnFinishedScrolling(this, &FTraceTableDashboardViewFactory::OnFinishedScrolling)
			.ClearSelectionOnClick(ClearSelectionOnClick())
			.OnGenerateRow_Lambda([this](TSharedPtr<IDashboardDataViewEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(SRowWidget, OwnerTable, Item, AsShared())
					.Style(GetRowStyle());
			})
			.HeaderRow
			(
				MakeHeaderRowWidget()
			)
			.SelectionMode(ESelectionMode::Multi)
			.OnKeyDownHandler_Lambda([this](const FGeometry& Geometry, const FKeyEvent& KeyEvent)
			{
				return OnDataRowKeyInput(Geometry, KeyEvent);
			});

			DashboardWidget = SNew(SVerticalBox)
			.Clipping(EWidgetClipping::ClipToBounds)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2)
			[
				SNew(SHorizontalBox)

				// Optional : Filters menu
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FilterBarButton.IsValid() ? 3.0f : 0.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SVerticalBox)
						
					+ SVerticalBox::Slot()
					.MaxHeight(30.0f)
					[
						FilterBarButton.IsValid() ? FilterBarButton.ToSharedRef() : SNullWidget::NullWidget
					]
				]

				// Search box
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.MaxHeight(FSlateStyle::Get().GetSearchBoxMaxHeight())
					.Padding(0.0f, 4.0f, 0.0f, 4.0f)
					[
						SAssignNew(SearchBoxWidget, SSearchBox)
						.SelectAllTextWhenFocused(true)
						.HintText(LOCTEXT("TableDashboardView_SearchBoxHintText", "Filter"))
						.MinDesiredWidth(100)
						.OnTextChanged(this, &FTraceTableDashboardViewFactory::SetSearchBoxFilterText)
					]
				]

				// Optional : Active filters area
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreateFilterBarWidget()
				]

				// Empty Spacing
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)

				// Optional : Settings button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					CreateSettingsButtonWidget()
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0, 2)
			[
				EnableHorizontalScrollBox() ? CreateHorizontalScrollBox() : FilteredEntriesListView->AsShared()
			];
		}

		return DashboardWidget->AsShared();
	}

	void FTraceTableDashboardViewFactory::SetSearchBoxFilterText(const FText& NewText)
	{
		SearchBoxFilterText = NewText;
		UpdateFilterReason = EProcessReason::FilterUpdated;
	}


	void FTraceTableDashboardViewFactory::RefreshFilteredEntriesListView()
	{
		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->RequestListRefresh();
		}
	}

#if WITH_EDITOR
	void FTraceTableDashboardViewFactory::UpdateDebugDraw(const float DeltaTime)
	{
		if (IsDebugDrawEnabled())
		{
			if (FilteredEntriesListView.IsValid())
			{
				TArray<TSharedPtr<IDashboardDataViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

				if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
				{
					AudioDeviceManager->IterateOverAllDevices([this, &SelectedItems, DeltaTime](::Audio::FDeviceId DeviceId, FAudioDevice* Device)
					{
						DebugDraw(DeltaTime, SelectedItems, DeviceId);
					});
				}
			}
		}
	}
#endif // WITH_EDITOR

	const FText& FTraceTableDashboardViewFactory::GetSearchFilterText() const
	{
		return SearchBoxFilterText;
	}

	FText FSoundAssetDashboardEntry::GetDisplayName() const
	{
		return FText::FromString(FSoftObjectPath(Name).GetAssetName());
	}

	const UObject* FSoundAssetDashboardEntry::GetObject() const
	{
		return FSoftObjectPath(Name).ResolveObject();
	}

	UObject* FSoundAssetDashboardEntry::GetObject()
	{
		return FSoftObjectPath(Name).ResolveObject();
	}

	bool FSoundAssetDashboardEntry::IsValid() const
	{
		return PlayOrder != INDEX_NONE;
	}

	TSharedRef<SWidget> FTraceObjectTableDashboardViewFactory::GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column)
	{
		const FColumnData& ColumnData = GetColumns()[Column];
		const FText ValueText = ColumnData.GetDisplayValue.IsSet() ? ColumnData.GetDisplayValue(InRowData.Get()) : FText::GetEmpty();
		const FName IconName = ColumnData.GetIconName.IsSet() ? ColumnData.GetIconName(InRowData.Get()) : NAME_None;

		if (ValueText.IsEmpty() && IconName == NAME_None)
		{
			return SNullWidget::NullWidget;
		}

		return SNew(SHorizontalBox)

			// Icon
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f)
			.HAlign(ColumnData.Alignment)
			.AutoWidth()
			[
				ColumnData.GetIconName.IsSet()

				? SNew(SImage)
				.Image_Lambda([this, InRowData, Column]()
				{
					const FColumnData& ColumnData = GetColumns()[Column];
					return FSlateStyle::Get().GetBrush(ColumnData.GetIconName(InRowData.Get()));
				})
				.ColorAndOpacity_Lambda([this, InRowData, Column]()
				{
					const FColumnData& ColumnData = GetColumns()[Column];
					return ColumnData.GetIconColor.IsSet() ? ColumnData.GetIconColor(InRowData.Get()) : FLinearColor::White;
				})
				.ToolTipText_Lambda([this, InRowData, Column]()
				{
					const FColumnData& ColumnData = GetColumns()[Column];
					return ColumnData.GetIconTooltip.IsSet() ? ColumnData.GetIconTooltip(InRowData.Get()) : FText::GetEmpty();
				})
				.Visibility_Lambda([this, InRowData, Column]()
				{
					const FColumnData& ColumnData = GetColumns()[Column];
					return ColumnData.GetIconName(InRowData.Get()) == NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
				})

				: SNullWidget::NullWidget
		]

		+ SHorizontalBox::Slot()
		.Padding(IconName != NAME_None ? 10.0f : 0.0f, 2.0f)
		.FillWidth(0.9f)
		[
			SNew(STextBlock)
			.Text_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				const FText ValueText = ColumnData.GetDisplayValue(InRowData.Get());

				return ValueText;
			})
			.ToolTipText_Lambda([this, InRowData, Column]()
			{
				const FColumnData& ColumnData = GetColumns()[Column];
				return ColumnData.GetDisplayValue.IsSet() ? ColumnData.GetDisplayValue(InRowData.Get()) : FText::GetEmpty();
			})
			.ColorAndOpacity_Lambda([this, InRowData]()
			{
				return GetRowColor(InRowData.ToSharedPtr());
			})
			.OnDoubleClicked_Lambda([this, InRowData](const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
			{
#if WITH_EDITOR
				if (GEditor)
				{
					const TSharedRef<IObjectDashboardEntry> ObjectData = StaticCastSharedRef<IObjectDashboardEntry>(InRowData);
					const TObjectPtr<UObject> Object = ObjectData->GetObject();

					if (Object && Object->IsAsset())
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
						return FReply::Handled();
					}
				}
#endif // WITH_EDITOR
				return FReply::Unhandled();
			})
		];
	}

	FReply FTraceObjectTableDashboardViewFactory::OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const
	{
#if WITH_EDITOR
		if (GEditor && FilteredEntriesListView.IsValid())
		{
			if (KeyEvent.GetKey() == EKeys::Enter)
			{
				TArray<TSharedPtr<IDashboardDataViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

				for (const TSharedPtr<IDashboardDataViewEntry>& SelectedItem : SelectedItems)
				{
					if (SelectedItem.IsValid())
					{
						IObjectDashboardEntry& RowData = *StaticCastSharedPtr<IObjectDashboardEntry>(SelectedItem).Get();
						if (UObject* Object = RowData.GetObject())
						{
							if (Object && Object->IsAsset())
							{
								GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
							}
						}
					}
				}

				return FReply::Handled();
			}
		}
#endif // WITH_EDITOR
		return FReply::Unhandled();
	}

	TSharedRef<SWidget> FTraceObjectTableDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			DashboardWidget = SNew(SVerticalBox)
#if WITH_EDITOR
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					MakeAssetMenuBar()
				]
			]
#endif // WITH_EDITOR
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs)
			];

			if (FilteredEntriesListView.IsValid())
			{
				FilteredEntriesListView->SetSelectionMode(ESelectionMode::Multi);
			}
		}

		return DashboardWidget->AsShared();
	}

#if WITH_EDITOR
	TSharedRef<SWidget> FTraceObjectTableDashboardViewFactory::MakeAssetMenuBar() const
	{
		const FDashboardAssetCommands& Commands = FDashboardAssetCommands::Get();
		TSharedPtr<FUICommandList> ToolkitCommands = MakeShared<FUICommandList>();
		ToolkitCommands->MapAction(Commands.GetOpenCommand(), FExecuteAction::CreateLambda([this]() { OpenAsset(); }));
		ToolkitCommands->MapAction(Commands.GetBrowserSyncCommand(), FExecuteAction::CreateLambda([this]() { BrowseToAsset(); }));

		FToolBarBuilder ToolbarBuilder(ToolkitCommands, FMultiBoxCustomization::None);
		Commands.AddAssetCommands(ToolbarBuilder);

		return ToolbarBuilder.MakeWidget();
	}

	TArray<UObject*> FTraceObjectTableDashboardViewFactory::GetSelectedEditableAssets() const
	{
		TArray<UObject*> Objects;

		if (!FilteredEntriesListView.IsValid())
		{
			return Objects;
		}

		const TArray<TSharedPtr<IDashboardDataViewEntry>> Items = FilteredEntriesListView->GetSelectedItems();
		Algo::TransformIf(Items, Objects,
			[](const TSharedPtr<IDashboardDataViewEntry>& Item)
			{
				if (Item.IsValid())
				{
					IObjectDashboardEntry& RowData = *StaticCastSharedPtr<IObjectDashboardEntry>(Item).Get();
					if (UObject* Object = RowData.GetObject())
					{
						return Object && Object->IsAsset();
					}
				}

				return false;
			},
			[](const TSharedPtr<IDashboardDataViewEntry>& Item)
			{
				IObjectDashboardEntry& RowData = *StaticCastSharedPtr<IObjectDashboardEntry>(Item).Get();
				return RowData.GetObject();
			}
		);

		return Objects;
	}

	bool FTraceObjectTableDashboardViewFactory::OpenAsset() const
	{
		if (GEditor && FilteredEntriesListView.IsValid())
		{
			TArray<UObject*> Objects = GetSelectedEditableAssets();
			if (UAssetEditorSubsystem* AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				return AssetSubsystem->OpenEditorForAssets(Objects);
			}
		}

		return false;
	}

	bool FTraceObjectTableDashboardViewFactory::BrowseToAsset() const
	{
		if (GEditor)
		{
			TArray<UObject*> EditableAssets = GetSelectedEditableAssets();
			GEditor->SyncBrowserToObjects(EditableAssets);
			return true;
		}

		return false;
	}
#endif // WITH_EDITOR

	void FTraceObjectTableDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
#if WITH_EDITOR
		// Notify the details panel that a new asset has been selected
		FAudioInsightsDetailsSelectionManager& SelectionManager = IAudioInsightsModule::GetChecked().GetDetailsSelectionManager();
		TObjectPtr<UObject> SelectedAssetPtr = nullptr;

		const TArray<UObject*> Assets = GetSelectedEditableAssets();
		if (Assets.Num() > 0)
		{
			SelectedAssetPtr = Assets[0];
		}
		else if (SelectedItem.IsValid())
		{
			// If we could not find a loaded asset for this object, try to load one using the object path
			IObjectDashboardEntry& RowData = *StaticCastSharedPtr<IObjectDashboardEntry>(SelectedItem).Get();
			const FString ObjectPath = RowData.GetObjectPath();
			if (!ObjectPath.IsEmpty())
			{
				const TObjectPtr<UObject> LoadedObject = FSoftObjectPath(ObjectPath).TryLoad();
				if (LoadedObject && LoadedObject->IsAsset())
				{
					SelectedAssetPtr = LoadedObject;
				}
			}
		}

		if (SelectedAssetPtr)
		{
			SelectionManager.SetSelectedAsset(SelectedAssetPtr);
		}
		else if (SelectInfo != ESelectInfo::Type::Direct)
		{
			SelectionManager.ClearSelection();
		}
#endif // WITH_EDITOR
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE