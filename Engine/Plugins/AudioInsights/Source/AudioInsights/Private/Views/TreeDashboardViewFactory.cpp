// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/TreeDashboardViewFactory.h"

#include "Algo/Transform.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceProviderBase.h"
#include "Containers/Array.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Layout/SScrollBox.h"

#if WITH_EDITOR
#include "AudioDeviceManager.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	// SRowWidget
	void FTraceTreeDashboardViewFactory::SRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<IDashboardDataTreeViewEntry> InData, TSharedRef<FTraceTreeDashboardViewFactory> InFactory)
	{
		Data = InData;
		Factory = InFactory;

		RenderOpacity = InArgs._RenderOpacity;

		FSuperRowType::FArguments Args = FSuperRowType::FArguments().Style(&FSlateStyle::Get().GetWidgetStyle<FTableRowStyle>("TreeDashboard.TableViewRow"));

		SMultiColumnTableRow<TSharedPtr<IDashboardDataTreeViewEntry>>::Construct(Args, InOwnerTable);
	}

	TSharedRef<SWidget> FTraceTreeDashboardViewFactory::SRowWidget::GenerateWidgetForColumn(const FName& Column)
	{
		return Factory->GenerateWidgetForColumn(StaticCastSharedRef<SRowWidget>(AsShared()), Data->AsShared(), Column);
	}

	void FTraceTreeDashboardViewFactory::SRowWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSoundDashboardViewFactory::TickRow)

		SetRenderOpacity(RenderOpacity.Get());

		SMultiColumnTableRow<TSharedPtr<IDashboardDataTreeViewEntry>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	// FTraceTreeDashboardViewFactory
	TSharedPtr<SWidget> FTraceTreeDashboardViewFactory::GetFilterBarButtonWidget()
	{
		return nullptr;
	}

	FTraceTreeDashboardViewFactory::FTraceTreeDashboardViewFactory()
		: FTraceDashboardViewFactoryBase()
	{
	}

	TSharedPtr<SWidget> FTraceTreeDashboardViewFactory::GetFilterBarWidget()
	{
		return nullptr;
	}

	TSharedRef<SWidget> FTraceTreeDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			const TSharedPtr<SWidget> FilterBar = GetFilterBarWidget();
			const TSharedPtr<SWidget> FilterBarButton = GetFilterBarButtonWidget();

			SAssignNew(DashboardWidget, SVerticalBox)
			.Clipping(EWidgetClipping::ClipToBounds)
			// Filter / Search area
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 3.0f)
			[
				SNew(SHorizontalBox)
				// Filter selector
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FilterBarButton ? 3.0f : 0.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.MaxHeight(30.0f)
					[
						FilterBarButton ? FilterBarButton.ToSharedRef() : SNullWidget::NullWidget
					]
				]
				// Search box
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.MaxHeight(FSlateStyle::Get().GetSearchBoxMaxHeight())
					.Padding(0.0f, 4.0f, 0.0f, 6.0f)
					[
						SAssignNew(SearchBoxWidget, SSearchBox)
						.SelectAllTextWhenFocused(true)
						.HintText(LOCTEXT("TreeDashboardView_SearchBoxHintText", "Search"))
						.MinDesiredWidth(100)
						.OnTextChanged(this, &FTraceTreeDashboardViewFactory::SetSearchBoxFilterText)
					]
				]
				// Active filters area
				+ SHorizontalBox::Slot()
				[
					FilterBar ? FilterBar.ToSharedRef() : SNullWidget::NullWidget
				]
			]
			// TreeView
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				+ SScrollBox::Slot()
				.FillSize(1.0f)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(FilteredEntriesListView, STreeView<TSharedPtr<IDashboardDataTreeViewEntry>>)
					.HeaderRow(MakeHeaderRowWidget())
					.TreeItemsSource(&GetTreeItemsSource())
					.OnGenerateRow(this, &FTraceTreeDashboardViewFactory::OnGenerateRow)
					.OnContextMenuOpening(this, &FTraceTreeDashboardViewFactory::OnConstructContextMenu)
					.SelectionMode(ESelectionMode::Multi)
					.OnSelectionChanged(this, &FTraceTreeDashboardViewFactory::OnSelectionChanged)
					.OnKeyDownHandler(this, &FTraceTreeDashboardViewFactory::OnDataRowKeyInput)
					.OnGetChildren_Lambda([this](TSharedPtr<IDashboardDataTreeViewEntry> InParent, TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& OutChildren)
					{
						if (InParent.IsValid() && !InParent->Children.IsEmpty())
						{
							OutChildren = InParent->Children;

							if (FilteredEntriesListView.IsValid() && !InParent->HasSetInitExpansion())
							{
								FilteredEntriesListView->SetItemExpansion(InParent, ShouldAutoExpand(InParent));
								InParent->ResetHasSetInitExpansion();
							}
						}
					})
					.OnSetExpansionRecursive(this, &FTraceTreeDashboardViewFactory::HandleRecursiveExpansion)
					.OnExpansionChanged_Lambda([](TSharedPtr<IDashboardDataTreeViewEntry> Item, bool bIsExpanded)
					{
						Item->bIsExpanded = bIsExpanded;
					})
				]
			];
		}

		return DashboardWidget->AsShared();
	}
	
	TSharedRef<SHeaderRow> FTraceTreeDashboardViewFactory::MakeHeaderRowWidget()
	{
		TArray<FName> DefaultHiddenColumns;

		Algo::TransformIf(GetHeaderRowColumns(), DefaultHiddenColumns,
			[](const TPair<FName, FHeaderRowColumnData>& ColumnInfo) { return ColumnInfo.Value.bDefaultHidden; },
			[](const TPair<FName, FHeaderRowColumnData>& ColumnInfo) { return ColumnInfo.Key; });

		SAssignNew(HeaderRowWidget, SHeaderRow)
		.CanSelectGeneratedColumn(true) // Allows for showing/hiding columns
		.OnHiddenColumnsListChanged(this, &FTraceTreeDashboardViewFactory::OnHiddenColumnsListChanged);

		// This only works if header row columns are added with slots and not programmatically
		// check in SHeaderRow::Construct: for ( FColumn* const Column : InArgs.Slots ) for more info
		// A potential alternative would be to delegate to the derived classes the SHeaderRow creation with slots
		//.HiddenColumnsList(DefaultHiddenColumns);

		for (const auto& [ColumnName, ColumnData] : GetHeaderRowColumns())
		{
			SHeaderRow::FColumn::FArguments ColumnArgs = SHeaderRow::Column(ColumnName)
				.DefaultLabel(ColumnData.DisplayName)
				.HAlignCell(ColumnData.Alignment)
				.SortMode(this, &FTraceTreeDashboardViewFactory::GetColumnSortMode, ColumnName)
				.OnSort(this, &FTraceTreeDashboardViewFactory::OnColumnSortModeChanged)
				.HeaderContent()
				[
					SNew(SHorizontalBox)
					// Icon (optional)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(ColumnData.IconName != NAME_None ? 4.0f : 0.0f, 3.0f)
					[
						ColumnData.IconName != NAME_None
							? SNew(SImage).Image(FSlateStyle::Get().GetBrush(ColumnData.IconName)).ToolTipText(ColumnData.TooltipText)
							: SNullWidget::NullWidget
					]
					// Text (optional)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 3.0f, 0.0f, 3.0f)
					[
						ColumnData.bShowDisplayName 
							? SNew(STextBlock).Text(ColumnData.DisplayName).ToolTipText(ColumnData.TooltipText)
							: SNullWidget::NullWidget
					]
				];

			if (ColumnData.FixedWidth.IsSet())
			{
				ColumnArgs.FixedWidth(ColumnData.FixedWidth.GetValue());
			}
			else
			{
				ColumnArgs.FillWidth(ColumnData.FillWidth);
			}

			// .HiddenColumnsList workaround:
			// simulate what SHeaderRow::AddColumn( const FColumn::FArguments& NewColumnArgs ) does but allowing us to modify the bIsVisible property
			// Memory handling (delete) is done by TIndirectArray<FColumn> Columns; defined in SHeaderRow
			SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(ColumnArgs);
			NewColumn->bIsVisible = !DefaultHiddenColumns.Contains(ColumnName);
			HeaderRowWidget->AddColumn(*NewColumn);
		}

		return HeaderRowWidget.ToSharedRef();
	}

	TSharedRef<ITableRow> FTraceTreeDashboardViewFactory::OnGenerateRow(TSharedPtr<IDashboardDataTreeViewEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SRowWidget, OwnerTable, Item, AsShared());
	}

	TSharedPtr<SWidget> FTraceTreeDashboardViewFactory::OnConstructContextMenu()
	{
		// To be optionally implemented by derived classes
		return SNullWidget::NullWidget;
	}

	void FTraceTreeDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataTreeViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		// To be optionally implemented by derived classes
	}

	FReply FTraceTreeDashboardViewFactory::OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const
	{
		// To be optionally implemented by derived classes
		return FReply::Unhandled();
	}

	const FText& FTraceTreeDashboardViewFactory::GetSearchFilterText() const
	{
		return SearchBoxFilterText;
	}

	void FTraceTreeDashboardViewFactory::SetSearchBoxFilterText(const FText& NewText)
	{
		SearchBoxFilterText = NewText;
		UpdateFilterReason = EProcessReason::FilterUpdated;
	}

	void FTraceTreeDashboardViewFactory::RefreshFilteredEntriesListView()
	{
		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->RequestTreeRefresh();
		}
	}

	EColumnSortMode::Type FTraceTreeDashboardViewFactory::GetColumnSortMode(const FName InColumnId) const
	{
		return SortByColumn == InColumnId ? SortMode : EColumnSortMode::None;
	}

	void FTraceTreeDashboardViewFactory::RequestSort()
	{
		SortTable();
		RefreshFilteredEntriesListView();
	}

	void FTraceTreeDashboardViewFactory::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
	{
		// Sorting can be disabled on specified columns
		if (!IsColumnSortable(InColumnId))
		{
			return;
		}

		SortByColumn = InColumnId;
		SortMode = InSortMode;

		RequestSort();
	}

	void FTraceTreeDashboardViewFactory::HandleRecursiveExpansion(TSharedPtr<IDashboardDataTreeViewEntry> Item, bool bIsItemExpanded)
	{
		for (TSharedPtr<IDashboardDataTreeViewEntry> ChildItem : Item->Children)
		{
			HandleRecursiveExpansion(ChildItem, bIsItemExpanded);
		}

		FilteredEntriesListView->SetItemExpansion(Item, bIsItemExpanded);
	}

	void FTraceTreeDashboardViewFactory::TransformDataRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& OriginalEntry, TFunctionRef<bool(IDashboardDataTreeViewEntry&)> FilterPredicate, const int32 DepthToDisplayFrom, const bool bParentPassedFilter /*= true*/)
	{
		if (!OriginalEntry.IsValid())
		{
			return;
		}

		const bool bPassesFilter = DepthToDisplayFrom >= 0 ? FilterPredicate(*OriginalEntry) : bParentPassedFilter;
		const bool bIsAtValidTreeDepth = DepthToDisplayFrom == 0 || OriginalEntry->Children.IsEmpty();

		if (bIsAtValidTreeDepth && bPassesFilter)
		{
			DataViewEntries.Add(OriginalEntry);
		}
		else
		{
			for (const TSharedPtr<IDashboardDataTreeViewEntry>& Child : OriginalEntry->Children)
			{
				TransformDataRecursive(Child, FilterPredicate, DepthToDisplayFrom - 1, bPassesFilter);
			}
		}
	}

#if WITH_EDITOR
	void FTraceTreeDashboardViewFactory::UpdateDebugDraw(const float DeltaTime)
	{
		if (IsDebugDrawEnabled())
		{
			if (FilteredEntriesListView.IsValid())
			{
				TArray<TSharedPtr<IDashboardDataTreeViewEntry>> SelectedItems = FilteredEntriesListView->GetSelectedItems();

				if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
				{
					AudioDeviceManager->IterateOverAllDevices([this, &SelectedItems, DeltaTime](::Audio::FDeviceId DeviceId, FAudioDevice* Device)
					{
						DebugDraw(DeltaTime, SelectedItems, DeviceId);
					});
				}
			}
		}
	}
#endif // WITH_EDITOR
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE