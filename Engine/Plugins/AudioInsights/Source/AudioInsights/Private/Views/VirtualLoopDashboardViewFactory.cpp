// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/VirtualLoopDashboardViewFactory.h"

#include "AudioDefines.h"
#include "AudioInsightsModule.h"
#include "AudioInsightsStyle.h"
#include "Internationalization/Text.h"
#include "Providers/VirtualLoopTraceProvider.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#include "Views/AssetEditorContextMenuHelper.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioInsights"


namespace UE::Audio::Insights
{
	namespace VirtualLoopPrivate
	{
		const FVirtualLoopDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FVirtualLoopDashboardEntry&>(InData);
		};
	} // namespace VirtualLoopPrivate

	FVirtualLoopDashboardViewFactory::FVirtualLoopDashboardViewFactory()
	{
		FTraceModule& AudioInsightsTraceModule = static_cast<FTraceModule&>(FAudioInsightsModule::GetChecked().GetTraceModule());

		const TSharedPtr<FVirtualLoopTraceProvider> VirtualLoopTraceProvider = MakeShared<FVirtualLoopTraceProvider>();

		AudioInsightsTraceModule.AddTraceProvider(VirtualLoopTraceProvider);

		Providers = TArray<TSharedPtr<FTraceProviderBase>>
		{
			VirtualLoopTraceProvider
		};
	}

	FName FVirtualLoopDashboardViewFactory::GetName() const
	{
		return "VirtualLoops";
	}

	FText FVirtualLoopDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_VirtualLoops_DisplayName", "Virtual Loops");
	}

	TSharedRef<SWidget> FVirtualLoopDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> InOwnerTab, const FSpawnTabArgs& InSpawnTabArgs)
	{
		if (!DashboardWidget.IsValid())
		{
			DashboardWidget = FTraceTableDashboardViewFactory::MakeWidget(InOwnerTab, InSpawnTabArgs);

			if (FilteredEntriesListView.IsValid())
			{
				FilteredEntriesListView->SetSelectionMode(ESelectionMode::Multi);
			}

		}

		return DashboardWidget->AsShared();
	}

	void FVirtualLoopDashboardViewFactory::ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason)
	{
		const FString FilterString = GetSearchFilterText().ToString();
		FTraceTableDashboardViewFactory::FilterEntries<FVirtualLoopTraceProvider>([&FilterString](const IDashboardDataViewEntry& Entry)
		{
			const FVirtualLoopDashboardEntry& VirtualLoopEntry = static_cast<const FVirtualLoopDashboardEntry&>(Entry);

			return VirtualLoopEntry.GetDisplayName().ToString().Contains(FilterString);
		});
	}

	FSlateIcon FVirtualLoopDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.VirtualLoop");
	}

	EDefaultDashboardTabStack FVirtualLoopDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Analysis;
	}

	const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& FVirtualLoopDashboardViewFactory::GetColumns() const
	{
		auto CreateColumnData = []()
		{
			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					"PlayOrder",
					{
						LOCTEXT("VirtualLoop_PlayOrderColumnDisplayName", "Play Order"),
						[](const IDashboardDataViewEntry& InData) { return FText::AsNumber(static_cast<const FVirtualLoopDashboardEntry&>(InData).PlayOrder); },
						nullptr /*GetIconName*/,
						true /* bDefaultHidden */,
						0.1f /* FillWidth */
					}
				},
				{
					"Name",
					{
						LOCTEXT("VirtualLoop_NameColumnDisplayName", "Name"),
						[](const IDashboardDataViewEntry& InData) { return VirtualLoopPrivate::CastEntry(InData).GetDisplayName(); },
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.6f  /* FillWidth */
					}
				},
				{
					"TimeVirtualized",
					{
						LOCTEXT("VirtualLoop_VirtualizedTimeColumnDisplayName", "Time (Virtualized)"),
						[](const IDashboardDataViewEntry& InData) { return FSlateStyle::Get().FormatSecondsAsTime(VirtualLoopPrivate::CastEntry(InData).TimeVirtualized); },
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					"PlaybackTime",
					{
						LOCTEXT("VirtualLoop_TotalTimeColumnDisplayName", "Time (Total)"),
						[](const IDashboardDataViewEntry& InData) { return FSlateStyle::Get().FormatSecondsAsTime(VirtualLoopPrivate::CastEntry(InData).PlaybackTime); },
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.12f /* FillWidth */
					}
				},
				{
					"UpdateInterval",
					{
						LOCTEXT("VirtualLoop_UpdateIntervalColumnDisplayName", "Update Interval"),
						[](const IDashboardDataViewEntry& InData) { return FSlateStyle::Get().FormatSecondsAsTime(VirtualLoopPrivate::CastEntry(InData).UpdateInterval); },
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.13f /* FillWidth */
					}
				}
			};
		};
		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		return ColumnData;
	}

	void FVirtualLoopDashboardViewFactory::SortTable()
	{
		if (SortByColumn == "PlayOrder")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return AData.PlayOrder < BData.PlayOrder;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return BData.PlayOrder < AData.PlayOrder;
				});
			}
		}
		else if (SortByColumn == "Name")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return AData.GetDisplayName().CompareToCaseIgnored(BData.GetDisplayName()) < 0;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return BData.GetDisplayName().CompareToCaseIgnored(AData.GetDisplayName()) < 0;
				});
			}
		}
		else if (SortByColumn == "TimeVirtualized")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return AData.TimeVirtualized < BData.TimeVirtualized;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return BData.TimeVirtualized < AData.TimeVirtualized;
				});
			}
		}
		else if (SortByColumn == "PlaybackTime")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return AData.PlaybackTime < BData.PlaybackTime;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return BData.PlaybackTime < AData.PlaybackTime;
				});
			}
		}
		else if (SortByColumn == "UpdateInterval")
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return AData.UpdateInterval < BData.UpdateInterval;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FVirtualLoopDashboardEntry& AData = VirtualLoopPrivate::CastEntry(*A.Get());
					const FVirtualLoopDashboardEntry& BData = VirtualLoopPrivate::CastEntry(*B.Get());

					return BData.UpdateInterval < AData.UpdateInterval;
				});
			}
		}
	}

#if WITH_EDITOR
	FReply FVirtualLoopDashboardViewFactory::OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const
	{
		if (AssetContextMenuHelper.ProcessCommandBindings(KeyEvent))
		{
			return FReply::Handled();
		}

		return FTraceObjectTableDashboardViewFactory::OnDataRowKeyInput(Geometry, KeyEvent);
	}

	TSharedPtr<SWidget> FVirtualLoopDashboardViewFactory::OnConstructContextMenu()
	{
		return AssetContextMenuHelper.ContructContextMenuOptions();
	}

	void FVirtualLoopDashboardViewFactory::OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		FTraceObjectTableDashboardViewFactory::OnSelectionChanged(SelectedItem, SelectInfo);

		if (SelectedItem.IsValid())
		{
			TSharedPtr<IObjectDashboardEntry> ObjectEntry = StaticCastSharedPtr<IObjectDashboardEntry>(SelectedItem);
			AssetContextMenuHelper.SetAssetEntry(ObjectEntry);
		}
		else
		{
			AssetContextMenuHelper.ResetAssetEntry();
		}
	}

	bool FVirtualLoopDashboardViewFactory::IsDebugDrawEnabled() const
	{
#if ENABLE_AUDIO_DEBUG
		if (FAudioDeviceManager* Manager = FAudioDeviceManager::Get())
		{
			return !(Manager->IsVisualizeDebug3dEnabled() && !::Audio::FAudioDebugger::IsVirtualLoopVisualizeEnabled());
		}
#endif // ENABLE_AUDIO_DEBUG

		return false;
	}

	void FVirtualLoopDashboardViewFactory::DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const
	{
		OnDebugDrawEntries.Broadcast(InElapsed, InSelectedItems, InAudioDeviceId);
	}
#endif // WITH_EDITOR
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
