// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace UE::Audio::Insights
{
	inline const ANSICHAR* AudioLoggerName = "Audio";
	inline const ANSICHAR* TimestampFieldName = "Timestamp";
}
