// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Templates/SharedPointer.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights { class FDashboardAssetCommands; }

UE_DECLARE_TCOMMANDS(UE::Audio::Insights::FDashboardAssetCommands, UE_API)


namespace UE::Audio::Insights
{
	class FDashboardAssetCommands : public TCommands<FDashboardAssetCommands>
	{
	public:
		UE_API FDashboardAssetCommands();

		UE_API virtual void RegisterCommands() override;

		UE_API virtual void AddAssetCommands(FToolBarBuilder& OutToolbarBuilder) const;

		UE_API virtual TSharedPtr<const FUICommandInfo> GetBrowserSyncCommand() const;
		UE_API virtual TSharedPtr<const FUICommandInfo> GetOpenCommand() const;
		UE_API virtual TSharedPtr<const FUICommandInfo> GetSaveCommand() const;
		UE_API virtual TSharedPtr<const FUICommandInfo> GetSaveAllCommand() const;
		UE_API virtual TSharedPtr<const FUICommandInfo> GetTraceBookmarkCommand() const;

	private:
		/** Selects the sound in the content browser. */
		TSharedPtr<FUICommandInfo> BrowserSync;

		/** Opens the sound asset in respective editor(s). */
		TSharedPtr<FUICommandInfo> Open;

		/** Saves the selected asset. */
		TSharedPtr<FUICommandInfo> Save;

		/** Saves all dirty assets. */
		TSharedPtr<FUICommandInfo> SaveAll;

		/** Places a bookmark inside a trace file. */
		TSharedPtr<FUICommandInfo> CreateTraceBookmark;
	};
} // namespace UE::Audio::Insights

#undef UE_API