// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsToolbarWidgets.h"
#include "Containers/Map.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsDashboardFactory.h"
#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/Docking/SDockTab.h"

namespace UE::Audio::Insights
{
	class FDashboardFactory : public IDashboardFactory, public TSharedFromThis<FDashboardFactory>
	{
	public:
		FDashboardFactory() = default;
		virtual ~FDashboardFactory() = default;

		TSharedRef<SDockTab> MakeDockTabWidget(const FSpawnTabArgs& Args);

		AUDIOINSIGHTS_API static TSharedRef<SWidget> CreateDocumentationButtonWidget();

		virtual void RegisterViewFactory(TSharedRef<IDashboardViewFactory> InFactory) override;
		virtual void UnregisterViewFactory(FName InDashboardName) override;
		virtual ::Audio::FDeviceId GetDeviceId() const override;

	private:
		TSharedRef<SWidget> MakeMenuBarWidget();
		TSharedRef<SWidget> MakeMainToolbarWidget();
		TSharedRef<SWidget> MakeCacheSnapshotButtonWidget();
		TSharedRef<SWidget> MakeBookmarkButtonWidget();

		TSharedPtr<FTabManager::FLayout> GetDefaultTabLayout();

		void RegisterTabSpawners();
		void UnregisterTabSpawners();

#if !WITH_EDITOR
		TSharedRef<SWidget> MakeEnableTracesOverlay();
		TSharedRef<SWidget> MakeEnableTracesButton();
		FReply ToggleAutoEnableAudioTraces();
#endif
		void CreateTraceBookmark();

		TSharedPtr<FToolbarWidgets> ToolbarWidgets;

		TSharedPtr<FTabManager> DashboardTabManager;
		TSharedPtr<FWorkspaceItem> DashboardWorkspace;
		TSharedPtr<FTabManager::FLayout> TabLayout;

		static constexpr ::Audio::FDeviceId ActiveDeviceId = 1; // The default audio device id in standalone

		int32 BookmarkIndex = 0;

		TMap<FName, TSharedPtr<IDashboardViewFactory>> DashboardViewFactories;
	};
} // namespace UE::Audio::Insights
