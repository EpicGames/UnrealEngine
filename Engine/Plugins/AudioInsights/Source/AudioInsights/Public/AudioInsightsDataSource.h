// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

namespace UE::Audio::Insights
{
	using FDataPoint = TPair<double, float>; // (Timestamp, Value)

	class IDashboardDataViewEntry : public TSharedFromThis<IDashboardDataViewEntry>
	{
	public:
		virtual ~IDashboardDataViewEntry() = default;

		virtual bool IsValid() const = 0;
	};
	
	template <typename T>
	const T& CastEntry(const IDashboardDataViewEntry& InData)
	{
		return static_cast<const T&>(InData);
	}

	class IDashboardDataTreeViewEntry : public TSharedFromThis<IDashboardDataTreeViewEntry>
	{
	public:
		virtual ~IDashboardDataTreeViewEntry() = default;

		virtual bool IsValid() const = 0;
		virtual uint64 GetEntryID() const = 0;
		virtual const FText& GetDisplayName() const = 0;
		virtual const FLinearColor& GetEntryColor() const = 0;
		virtual void SetEntryColor(const FLinearColor& Color) = 0;
		virtual bool HasSetInitExpansion() const = 0;
		virtual void ResetHasSetInitExpansion() = 0;

		TArray<TSharedPtr<IDashboardDataTreeViewEntry>> Children;
		bool bIsExpanded = false;
	};
} // namespace UE::Audio::Insights
