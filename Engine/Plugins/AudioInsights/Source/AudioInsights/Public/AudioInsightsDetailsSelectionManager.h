// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "Delegates/Delegate.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FAudioInsightsDetailsSelectionManager
	{
	public:
		UE_API void SetSelectedAsset(TObjectPtr<UObject> InAsset);
		UE_API void ClearSelection();

		UE_API TWeakObjectPtr<UObject> GetSelectedAsset() const;

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetSelectionChanged, TObjectPtr<UObject> /*SelectedAsset*/);
		FOnAssetSelectionChanged OnAssetSelectionChanged;

	private:
		TWeakObjectPtr<UObject> SelectedAsset;
	};
} // namespace UE::Audio::Insights

#undef UE_API

#endif // WITH_EDITOR
