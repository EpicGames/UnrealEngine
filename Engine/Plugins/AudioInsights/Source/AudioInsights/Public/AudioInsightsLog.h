// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"


AUDIOINSIGHTS_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioInsights, Log, All);
