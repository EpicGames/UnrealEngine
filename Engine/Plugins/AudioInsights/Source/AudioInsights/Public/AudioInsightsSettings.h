// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "LoudnessMeterSettings.h"
#include "Settings/CacheSettings.h"
#include "Settings/OutputMeterDashboardSettings.h"

#include "AudioInsightsSettings.generated.h"

#define UE_API AUDIOINSIGHTS_API

UCLASS(MinimalAPI, config = AudioInsights)
class UAudioInsightsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Settings for the Cache */
	UPROPERTY(EditAnywhere, config, Category = Cache, meta = (ShowOnlyInnerProperties))
	FCacheSettings CacheSettings;

	/** Settings for the Output Meters (only visible via settings cog)*/
	UPROPERTY(config)
	FLoudnessMeterSettings OutputMeterSettings;

	/** Settings for the Output Meter dashboard tab (not visible to the user) */
	UPROPERTY(config)
	FOutputMeterDashboardSettings OutputMeterDashboardSettings;

	static FString GetAudioInsightsConfigFilename();

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void SaveCacheSettings();

private:
	FName GetContainerName() const override { return TEXT("Editor"); }
	FName GetCategoryName()  const override { return TEXT("Plugins"); }
	FName GetSectionName()   const override { return TEXT("Audio Insights"); }

#if WITH_EDITOR
	FText GetSectionText() const override;
#endif

	void SavePropertyToConfigFile(const FProperty* const PropertyToSave);
};

#undef UE_API
