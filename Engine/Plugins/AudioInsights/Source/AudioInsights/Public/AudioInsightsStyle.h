// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	class FSlateStyle final : public FSlateStyleSet
	{
	public:
		AUDIOINSIGHTS_API static FSlateStyle& Get();

		static FName GetStyleName()
		{
			const FLazyName StyleName = "AudioInsights";
			return StyleName.Resolve();
		}

		const FNumberFormattingOptions* GetAmpFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 3;
			FloatFormat.MinimumFractionalDigits = 1;
			FloatFormat.MaximumFractionalDigits = 1;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetLinearAmplitudeFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 3;
			FloatFormat.MinimumFractionalDigits = 5;
			FloatFormat.MaximumFractionalDigits = 5;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetLinearVolumeFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 3;
			FloatFormat.MinimumFractionalDigits = 3;
			FloatFormat.MaximumFractionalDigits = 3;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetDefaultFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 4;
			FloatFormat.MaximumFractionalDigits = 4;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetDistanceFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 0;
			FloatFormat.MaximumFractionalDigits = 0;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetFreqFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 5;
			FloatFormat.MinimumFractionalDigits = 0;
			FloatFormat.MaximumFractionalDigits = 2;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetPitchFloatFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 3;
			FloatFormat.MinimumFractionalDigits = 3;
			FloatFormat.MaximumFractionalDigits = 3;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetTimeFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 3;
			FloatFormat.MaximumFractionalDigits = 3;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetShortTimeFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 1;
			FloatFormat.MaximumFractionalDigits = 1;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetRelativeRenderCostFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MaximumIntegralDigits = 3;
			FloatFormat.MinimumFractionalDigits = 2;
			FloatFormat.MaximumFractionalDigits = 2;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetMemoryFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 1;
			FloatFormat.MaximumFractionalDigits = 1;
			return &FloatFormat;
		};

		const FNumberFormattingOptions* GetSendLevelFormat() const
		{
			static FNumberFormattingOptions FloatFormat;
			FloatFormat.MinimumIntegralDigits = 1;
			FloatFormat.MinimumFractionalDigits = 1;
			FloatFormat.MaximumFractionalDigits = 2;
			return &FloatFormat;
		};

		FText FormatSecondsAsTime(float InTimeSec) const
		{
			return FText::Format(LOCTEXT("TimeInSecondsFormat", "{0}s"), FText::AsNumber(InTimeSec, GetTimeFormat()));
		}

		FText FormatTimestamp(const double InSeconds) const;

		FText FormatMillisecondsAsTime(float InTimeMS) const
		{
			return FText::Format(LOCTEXT("TimeInMillisecondsFormat", "{0}ms"), FText::AsNumber(InTimeMS, GetTimeFormat()));
		}

		FText FormatMemoryAsMegabytes(const float InMemoryMB) const
		{
			return FText::Format(LOCTEXT("MemoryInMegabytesFormat", "{0}mb"), FText::AsNumber(InMemoryMB, GetMemoryFormat()));
		}

		float GetSearchBoxMaxHeight() const
		{
			return 22.0f;
		}

		FSlateIcon CreateIcon(FName InName)  const
		{
			return { GetStyleName(), InName };
		}

		const FSlateBrush& GetBrushEnsured(FName InName)  const
		{
			const ISlateStyle* AudioInsightsStyle = FSlateStyleRegistry::FindSlateStyle(GetStyleName());
			if (ensureMsgf(AudioInsightsStyle, TEXT("Missing slate style '%s'"), *GetStyleName().ToString()))
			{
				const FSlateBrush* Brush = AudioInsightsStyle->GetBrush(InName);
				if (ensureMsgf(Brush, TEXT("Missing brush '%s'"), *InName.ToString()))
				{
					return *Brush;
				}
			}

			if (const FSlateBrush* NoBrush = FAppStyle::GetBrush("NoBrush"))
			{
				return *NoBrush;
			}

			return *DefaultBrush;
		}

		TSharedRef<SBox> CreateButtonContentWidget(const FName& IconName = FName(), const FText& Label = FText::GetEmpty(), const FName& TextStyle = TEXT("ButtonText"), const float HeightOverride = 16.0f) const;
		TSharedRef<SBox> CreateToggleButtonContent(const FName& IconName, const FName& TextStyle, TFunction<FSlateColor()> ColorFunction, TFunction<FText()> TextFunction, const float HeightOverride = 16.0f) const;

		FSlateStyle()
			: FSlateStyleSet(GetStyleName())
		{
			SetParentStyleName(FAppStyle::GetAppStyleSetName());

			const FString PluginsDir = FPaths::EnginePluginsDir();

			SetContentRoot(PluginsDir / TEXT("AudioInsights/Content"));
			SetCoreContentRoot(PluginsDir / TEXT("Slate"));

			/* Audio Insights */
			// Colors
			Set("AudioInsights.Analyzers.BackgroundColor", FLinearColor(0.0075f, 0.0075f, 0.0075f, 1.0f));

			// Icons
			const FVector2D Icon12(12.0f, 12.0f);
			const FVector2D Icon16(16.0f, 16.0f);
			const FVector2D Icon20(20.0f, 20.0f);
			const FVector2D Icon24(24.0f, 24.0f);
			const FVector2D Icon64(64.0f, 64.0f);

			Set("AudioInsights.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_insights_icon"), Icon16));
			Set("AudioInsights.Icon.ClearLog", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_clear_log"), Icon20));
			Set("AudioInsights.Icon.ContentBrowser", new IMAGE_BRUSH_SVG(TEXT("Icons/content_browser"), Icon20));
			Set("AudioInsights.Icon.CheckBoxCheck", new IMAGE_BRUSH_SVG(TEXT("Icons/checkboxcheck"), Icon16));
			Set("AudioInsights.Icon.CheckBoxIndeterminate", new IMAGE_BRUSH_SVG(TEXT("Icons/checkboxindeterminate"), Icon16));
			Set("AudioInsights.Icon.Dashboard", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_dashboard"), Icon16));
			Set("AudioInsights.Icon.Details", new IMAGE_BRUSH_SVG(TEXT("Icons/details"), Icon16));
			Set("AudioInsights.Icon.Documentation", new IMAGE_BRUSH_SVG(TEXT("Icons/documentation"), Icon16));
			Set("AudioInsights.Icon.Event", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_event"), Icon16));
			Set("AudioInsights.Icon.ExpandArrowDown", new IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12));
			Set("AudioInsights.Icon.ExpandArrowUp", new IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_up"), Icon12));
			Set("AudioInsights.Icon.Log", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_log"), Icon16));
			Set("AudioInsights.Icon.OnlyTraceAudioChannels", new IMAGE_BRUSH_SVG(TEXT("Icons/only_trace_audio_channels"), Icon16));
			Set("AudioInsights.Icon.Open", new IMAGE_BRUSH_SVG(TEXT("Icons/open"), Icon20));
			Set("AudioInsights.Icon.Pause", new IMAGE_BRUSH_SVG(TEXT("Icons/pause"), Icon20));
			Set("AudioInsights.Icon.Record.Active", new IMAGE_BRUSH_SVG(TEXT("Icons/record_active"), Icon20));
			Set("AudioInsights.Icon.Record.Inactive", new IMAGE_BRUSH_SVG(TEXT("Icons/record_inactive"), Icon20));
			Set("AudioInsights.Icon.Settings", new IMAGE_BRUSH_SVG(TEXT("Icons/settings"), Icon20));
			Set("AudioInsights.Icon.Sources", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_sources"), Icon16));
			Set("AudioInsights.Icon.Sources.Plots", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_sources_plots"), Icon20));
			Set("AudioInsights.Icon.Start.Active", new IMAGE_BRUSH_SVG(TEXT("Icons/start_active"), Icon20));
			Set("AudioInsights.Icon.Start.Inactive", new IMAGE_BRUSH_SVG(TEXT("Icons/start_inactive"), Icon20));
			Set("AudioInsights.Icon.Stop.Active", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_active"), Icon20));
			Set("AudioInsights.Icon.Stop.Inactive", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_inactive"), Icon20));
			Set("AudioInsights.Icon.Submix", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_submix"), Icon16));
			Set("AudioInsights.Icon.Trace", new IMAGE_BRUSH_SVG(TEXT("Icons/trace"), Icon16));
			Set("AudioInsights.Icon.Trace.CacheSnapshot", new IMAGE_BRUSH_SVG(TEXT("Icons/cache_snapshot"), Icon16));
			Set("AudioInsights.Icon.Trace.Bookmark", new IMAGE_BRUSH_SVG(TEXT("Icons/bookmark"), Icon16));
			Set("AudioInsights.Icon.VirtualLoop", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_virtualloop"), Icon16));
			Set("AudioInsights.Icon.Viewport", new IMAGE_BRUSH_SVG(TEXT("Icons/viewport"), Icon16));
			Set("AudioInsights.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/audio_insights"), Icon16));


			/* Tree Dashboard */
			// Table style
			Set("TreeDashboard.TableViewRow", FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
				.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
				.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background))
				.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
				.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
			);

			/* Cache Control */
			// Icons
			Set("AudioInsights.Icon.CacheControls.CachePause", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CachePause"), Icon20));
			Set("AudioInsights.Icon.CacheControls.CacheResume", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CacheResume"), Icon20));
			Set("AudioInsights.Icon.CacheControls.CacheStop", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CacheStop"), Icon20));
			Set("AudioInsights.Icon.CacheControls.CacheNudgeBackward", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CacheNudgeBackward"), Icon20));
			Set("AudioInsights.Icon.CacheControls.CacheNudgeForward", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CacheNudgeForward"), Icon20));
			Set("AudioInsights.Icon.CacheControls.CacheFollowPlayhead", new IMAGE_BRUSH_SVG(TEXT("Icons/CacheControls/CacheFollowPlayhead"), Icon20));

			/* Sound Dashboard */
			// Icons
			Set("AudioInsights.Icon.SoundDashboard.Browse",           new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/browse"),           Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Edit",             new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/edit"),             Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Filter",           new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/filter"),           Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Info",             new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/info"),             Icon16));
			Set("AudioInsights.Icon.SoundDashboard.MetaSound",        new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/metasound"),        Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Mute",             new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/mute"),             Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Pin",              new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/pin"),              Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Plots",			  new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/plots"),			 Icon16));
			Set("AudioInsights.Icon.SoundDashboard.ProceduralSource", new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/proceduralsource"), Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Reset",            new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/reset"),            Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Solo",             new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/solo"),             Icon16));
			Set("AudioInsights.Icon.SoundDashboard.SoundCue",         new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/soundcue"),         Icon16));
			Set("AudioInsights.Icon.SoundDashboard.SoundWave",        new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/soundwave"),        Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Tab",              new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/tab"),              Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Transparent",      new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/transparent"),      Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Visible",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/visible"),			 Icon16));
			Set("AudioInsights.Icon.SoundDashboard.Invisible",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SoundDashboard/invisible"),		 Icon16));

			/* Event Log */
			Set("AudioInsights.Icon.EventLog.MarkedForDelete",		  new IMAGE_BRUSH_SVG(TEXT("Icons/EventLog/cache_marked_for_delete"), Icon16));

			/* Signal Flow */
			Set("AudioInsights.Icon.SignalFlow.AlignHorizontalBottom", new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_horizontal_bottom"), Icon16));
			Set("AudioInsights.Icon.SignalFlow.AlignHorizontalCenter", new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_horizontal_center"), Icon16));
			Set("AudioInsights.Icon.SignalFlow.AlignHorizontalTop",	   new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_horizontal_top"),	 Icon16));
			Set("AudioInsights.Icon.SignalFlow.AlignVerticalCenter",   new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_vertical_center"),	 Icon16));
			Set("AudioInsights.Icon.SignalFlow.AlignVerticalLeft",	   new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_vertical_left"),	 Icon16));
			Set("AudioInsights.Icon.SignalFlow.AlignVerticalRight",	   new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/align_vertical_right"),	 Icon16));

			Set("AudioInsights.Icon.SignalFlow.CenterView",			  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/center_view"),				 Icon16));
			Set("AudioInsights.Icon.SignalFlow.HighlightPath",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/highlight_path"),			 Icon16));
			Set("AudioInsights.Icon.SignalFlow.HorizontalFlow",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/horizontal_flow"),			 Icon16));
			Set("AudioInsights.Icon.SignalFlow.NodeDetails",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/node_details"),			 Icon16));
			Set("AudioInsights.Icon.SignalFlow.Settings",			  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/settings"),				 Icon16));
			
			Set("AudioInsights.Icon.SignalFlow.Pill",				  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/pill"),					 Icon16));
			Set("AudioInsights.Icon.SignalFlow.Connected",			  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/pin_trigger_connected"),	 Icon16));
			Set("AudioInsights.Icon.SignalFlow.Disconnected",		  new IMAGE_BRUSH_SVG(TEXT("Icons/SignalFlow/pin_trigger_disconnected"), Icon16));

			/* Modulation */
			// Icons
			Set("AudioInsights.Icon.Modulation.ControlBus", new IMAGE_BRUSH(TEXT("Icons/Modulation/SoundControlBus"), Icon16));
			Set("AudioInsights.Icon.Modulation.ControlBusMix", new IMAGE_BRUSH(TEXT("Icons/Modulation/SoundControlBusMix"), Icon16));
			Set("AudioInsights.Icon.Modulation.Generator", new IMAGE_BRUSH(TEXT("Icons/Modulation/SoundModulationGenerator"), Icon16));
			Set("AudioInsights.Icon.Modulation.Parameter", new IMAGE_BRUSH(TEXT("Icons/Modulation/SoundModulationParameter"), Icon16));
			Set("AudioInsights.Icon.Modulation.ParameterPatch", new IMAGE_BRUSH(TEXT("Icons/Modulation/SoundModulationPatch"), Icon16));
			
			// Category colors
			Set("SoundDashboard.MetaSoundColor", FLinearColor(0.008f, 0.76f, 0.078f));
			Set("SoundDashboard.SoundCueColor", FLinearColor(0.022f, 0.49f, 0.98f));
			Set("SoundDashboard.ProceduralSourceColor", FLinearColor(0.98f, 0.32f, 0.006f));
			Set("SoundDashboard.SoundWaveColor", FLinearColor(0.12f, 0.093f, 0.64f));
			Set("SoundDashboard.SoundCueTemplateColor", FLinearColor(0.98f, 0.01f, 0.01f));
			Set("SoundDashboard.PinnedColor", FLinearColor(0.9f, 0.9f, 0.9f));
			Set("SoundDashboard.HiddenColor", FLinearColor(0.4f, 0.4f, 0.4f));

			// Mute/Solo buttons style
			const FSlateRoundedBoxBrush RoundedWhiteAlphaBrush(FLinearColor(1.0f, 1.0f, 1.0f, 0.1f), 5.0f);

			Set("SoundDashboard.MuteSoloButton", FCheckBoxStyle(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("TransparentCheckBox"))
				.SetPadding(FMargin(5.0f, 2.0f, 5.0f, 2.0f))
				.SetCheckedHoveredImage(RoundedWhiteAlphaBrush)
				.SetUncheckedHoveredImage(RoundedWhiteAlphaBrush)
			);

			Set("AudioInsights.LeftExpandArrowToggle", FCheckBoxStyle(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("TransparentCheckBox"))
				.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
				.SetCheckedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::Foreground))
				.SetCheckedHoveredImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::ForegroundHover))
				.SetCheckedPressedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::ForegroundHover))
				.SetUncheckedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_left"), Icon12, FStyleColors::Foreground))
				.SetUncheckedHoveredImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_left"), Icon12, FStyleColors::ForegroundHover))
				.SetUncheckedPressedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_left"), Icon12, FStyleColors::ForegroundHover))
			);

			Set("AudioInsights.RightExpandArrowToggle", FCheckBoxStyle(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("TransparentCheckBox"))
				.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
				.SetCheckedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::Foreground))
				.SetCheckedHoveredImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::ForegroundHover))
				.SetCheckedPressedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_down"), Icon12, FStyleColors::ForegroundHover))
				.SetUncheckedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_right"), Icon12, FStyleColors::Foreground))
				.SetUncheckedHoveredImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_right"), Icon12, FStyleColors::ForegroundHover))
				.SetUncheckedPressedImage(IMAGE_BRUSH_SVG(TEXT("Icons/expand_arrow_right"), Icon12, FStyleColors::ForegroundHover))
			);

			FSlateStyleRegistry::RegisterSlateStyle(*this);
		}

		~FSlateStyle()
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*this);
		}
	};
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
