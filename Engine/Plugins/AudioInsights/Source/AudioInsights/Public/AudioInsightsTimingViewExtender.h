// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Insights/ITimingViewExtender.h"
#include "Math/Range.h"
#include "Misc/Optional.h"

#define UE_API AUDIOINSIGHTS_API

namespace Insights
{
	class ITimingViewSession;
	enum class ETimeChangedFlags : int32;
}

class IAnalysisSession;

namespace UE::Audio::Insights
{
	class FAudioInsightsCacheManager;
	class FTraceModule;

	// How the cache manage handles new incoming messages
	// Cache - will save the incoming message inside the cache to be retrievable later
	// Process - sends the message to the Audio Insights providers to be processed and displayed in the UI
	enum class ECacheAndProcess
	{
		Latest = 0,				// Cache and process all new messages
		CacheLatestNoProcess,	// Cache new messages but do not process them
		None					// Do not cache or process any new messages
	};

	enum class ESystemControllingTimeMarker
	{
		EventLog = 0,
		PlotsWidget,
		SignalFlow,
		External
	};

	class FAudioInsightsTimingViewExtender final : public UE::Insights::Timing::ITimingViewExtender
	{
	public:

		// AudioInsights API:
		ECacheAndProcess GetMessageCacheAndProcessingStatus() const { return CacheAndProcessMessageStatus; }
		TOptional<ESystemControllingTimeMarker> TryGetSystemControllingTimeMarker() const { return SystemControllingTime; }

		UE_API double GetCurrentTraceDurationSeconds() const;

		UE_API void PauseTimeMarker(const double InTimestamp, const ESystemControllingTimeMarker InControllingSystem, TOptional<TRange<double>> InPlottingRange = TOptional<TRange<double>>());
		UE_API void ResumeTimeMarker();

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnTimingViewTimeMarkerChanged, double /*TimeMarker*/);
		FOnTimingViewTimeMarkerChanged OnTimingViewTimeMarkerChanged;

		DECLARE_MULTICAST_DELEGATE(FOnTimeControlMethodReset);
		FOnTimeControlMethodReset OnTimeControlMethodReset;

		// Move to AudioInsightsConstants
		static constexpr double MaxPlottingHistorySeconds = 5.0;
		static constexpr double PlottingMarginSeconds = 0.2;

		// AudioInsights Internal:
		void StopProcessingNewMessages();
		void StopCachingAndProcessingNewMessages();
		TRange<double> GetPlottingRange() const;

		void BindToTraceModule(FTraceModule& InTraceModule);
		void UnbindFromTraceModule(FTraceModule& InTraceModule);

		void BindToCacheManager(FAudioInsightsCacheManager& InCacheManager);
		void UnbindFromCacheManager(FAudioInsightsCacheManager& InCacheManager);

	private:
		// Insights::ITimingViewExtender interface
		virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;

		void OnTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);
		void OnAnalysisStarting(const double Timestamp);

		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
		UE::Insights::Timing::ITimingViewSession* TimingView = nullptr;

		void SetPausedTimeMarker(const double InTimeMarker);
		void ApplyStopCacheWhenPausedBehaviour();
		void OnCacheChunkOverwritten(const double NewCacheStartTimestamp);

		ECacheAndProcess CacheAndProcessMessageStatus = ECacheAndProcess::Latest;
		TOptional<ESystemControllingTimeMarker> SystemControllingTime;
		TRange<double> PlottingRange { 0.0, MaxPlottingHistorySeconds };
		double TraceDurationSeconds = 0.0;
		double PausedTimeMarker = 0.0;
		bool bUserInputDetected = false;

		FDelegateHandle OnAnalysisStartingHandle;
		FDelegateHandle OnCacheChunkOverwrittenHandle;
	};
} // namespace UE::Audio::Insights

#undef UE_API
