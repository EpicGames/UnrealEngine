// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"

#define UE_API AUDIOINSIGHTS_API

class SWidget;

namespace UE::Audio::Insights
{
	class UE_API FToolbarWidgets final
	{
	public:
		FToolbarWidgets();
		~FToolbarWidgets();

		TSharedRef<SWidget> MakeCacheLabelWidget();
		TSharedRef<SWidget> MakeCachePauseButtonWidget();
		TSharedRef<SWidget> MakeCacheStopButtonWidget();
		TSharedRef<SWidget> MakeCacheResumeButtonWidget();
		TSharedRef<SWidget> MakeCacheFollowButtonWidget();
		TSharedRef<SWidget> MakeCacheCurrentTimestampWidget();
		TSharedRef<SWidget> MakeCacheBeginTimestampWidget();
		TSharedRef<SWidget> MakeCacheEndTimestampWidget();
		TSharedRef<SWidget> MakeCacheSizeAndDurationWidget();
		TSharedRef<SWidget> MakeCacheNudgeBackButtonWidget();
		TSharedRef<SWidget> MakeCacheTimelineRulerWidget();
		TSharedRef<SWidget> MakeCacheNudgeForwardButtonWidget();
		TSharedRef<SWidget> MakeCacheSettingsButtonWidget();

	private:
		void InitializeDelegates();
		void DeinitializeDelegates();

		TSharedRef<SWidget> MakeCacheSettingsMenuContent();

		double GetCurrentRelativeTime() const;
		void OnAnalysisStarting(const double Timestamp);
		void OnTimingViewTimeMarkerChanged(double InTimeMarker);
		void OnTimeControlMethodReset();

		bool bIsPaused = false;
		bool bTimelineAutoScroll = true;
		bool bTimelineViewRangeSetByUser = false;
		double BeginTimestamp = 0.0;
		double PausedTimeMarker = 0.0;
		TRange<double> TimelineViewRange = TRange<double>(0.0, 10.0);

		FDelegateHandle OnAnalysisStartingHandle;
		FDelegateHandle OnTimingViewTimeMarkerChangedHandle;
		FDelegateHandle OnTimeControlMethodResetHandle;
	};
} // namespace UE::Audio::Insights

#undef UE_API
