// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "IAudioInsightsTraceModule.h"
#include "ITraceController.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "Templates/SharedPointer.h"
#include "Trace/StoreClient.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	enum class ETraceMode : uint8
	{
		Monitoring = 0,
		Recording,
		None
	};

	class FRewindDebuggerAudioInsightsRuntime : public IRewindDebuggerRuntimeExtension
	{
	public:
		virtual void RecordingStarted() override;
	};

	class FTraceModule : public IAudioInsightsTraceModule
	{
	public:
		UE_API FTraceModule();
		virtual ~FTraceModule();

		//~ Begin TraceServices::IModule interface
		UE_API virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
		UE_API virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
		UE_API virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
		UE_API virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
		virtual const TCHAR* GetCommandLineArgument() override { return TEXT("audiotrace"); }
		//~ End TraceServices::IModule interface

		template <typename TraceProviderType>
		TSharedPtr<TraceProviderType> FindAudioTraceProvider() const
		{
			return StaticCastSharedPtr<TraceProviderType>(TraceProviders.FindRef(TraceProviderType::GetName_Static()));
		}

		UE_API virtual void AddTraceProvider(TSharedPtr<FTraceProviderBase> TraceProvider) override;

		UE_API virtual void StartTraceAnalysis(const bool bInOnlyTraceAudioChannels, const ETraceMode InTraceMode) override;
		UE_API virtual bool IsTraceAnalysisActive() const override;
		UE_API virtual void StopTraceAnalysis() override;
		UE_API virtual void OnOnlyTraceAudioChannelsStateChanged(const bool bOnlyTraceAudioChannels) override;

		UE_API virtual bool AudioChannelsCanBeManuallyEnabled() const override;

		UE_API virtual void ExecuteConsoleCommand(const FString& InCommandStr) const override;

		UE_API virtual void SaveCacheSnapshot() override;

		UE_API virtual ETraceMode GetTraceMode() const override;

#if !WITH_EDITOR
		UE_API virtual void InitializeSessionInfo(const TraceServices::FSessionInfo& SessionInfo) override;
		UE_API virtual void RequestChannelUpdate() override;
		UE_API virtual void ResetTicker() override;
		UE_API virtual bool TraceControllerIsAvailable() const override;
#else
		virtual bool IsAudioInsightsTrace() const override { return bIsAudioInsightsTrace; }
#endif // !WITH_EDITOR

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnAnalysisStarting, const double /*Timestamp*/);
		FOnAnalysisStarting OnAnalysisStarting;

	private:
		static UE_API const FName GetName();

		static UE_API void DisableAllTraceChannels();
		UE_API bool EnableAudioInsightsTraceChannels();

		UE_API void DisableAudioInsightsTraceChannels() const;

		UE_API void CacheCurrentlyEnabledTraceChannels();
		UE_API void RestoreCachedChannels() const;

#if !WITH_EDITOR
		UE_API bool IsSessionInfoAvailable() const;
		UE_API bool GetAudioTracesAreEnabled() const;
		UE_API void SendDiscoveryRequestToTraceController() const;
		UE_API void OnSessionsUpdated();

		UE_API bool Tick(float DeltaTime);
#else
		UE_API void OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination);
#endif // !WITH_EDITOR

		TMap<FName, TSharedPtr<FTraceProviderBase>> TraceProviders;
		TArray<FString> ChannelsToRestore;
		FRewindDebuggerAudioInsightsRuntime RewindDebugger;

		ETraceMode CurrentTraceMode = ETraceMode::None;

		bool bTraceAnalysisHasStarted = false;
		bool bAudioTraceChannelsEnabled = false;
		bool bIsAudioInsightsTrace = false;

#if !WITH_EDITOR
		FGuid InstanceID;

		FTickerDelegate OnTick;
		FTSTicker::FDelegateHandle OnTickHandle;

		const TArray<FString> AudioChannels;
		const TArray<FString> EmptyArray;
#endif // !WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API
