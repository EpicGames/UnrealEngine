// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <atomic>
#include "AudioDefines.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Cache/IAudioCachedMessage.h"
#include "Containers/Ticker.h"
#include "IAudioInsightsModule.h"
#include "IAudioInsightsTraceModule.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "AudioDeviceManager.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	enum class ECacheAndProcess;

	class FTraceProviderBase : public TraceServices::IProvider, public TraceServices::IEditableProvider
	{
	public:
		FTraceProviderBase() = delete;
		AUDIOINSIGHTS_API explicit FTraceProviderBase(FName InName);

		AUDIOINSIGHTS_API virtual ~FTraceProviderBase();

		virtual Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) = 0;
		AUDIOINSIGHTS_API FName GetName() const;

		virtual void Reset()
		{
			LastUpdateId = 0;
			LastMessageId = 0;
		}

		virtual void OnTraceChannelsEnabled()
		{
			LastUpdateId = 0;
			LastMessageId = 0;
		}

		virtual bool ProcessMessages()
		{
			LastUpdateId = LastMessageId;
			return true;
		}

		virtual bool ProcessManuallyUpdatedEntries()
		{ 
			return false;
		};

		uint64 GetLastUpdateId() const
		{
			return LastUpdateId;
		}

		bool IsUpdated() const
		{
			return GetLastMessageId() == LastUpdateId;
		}

		bool ShouldForceUpdate() const
		{
			return bForceUpdate;
		}

		void ResetShouldForceUpdate()
		{
			bForceUpdate = false;
		}

#if !WITH_EDITOR
		virtual void InitSessionCachedMessages(TraceServices::IAnalysisSession& InSession) {}
#endif // !WITH_EDITOR

		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) { ++LastMessageId; };
		virtual void OnTimeControlMethodReset() { };

		AUDIOINSIGHTS_API ECacheAndProcess GetMessageCacheAndProcessingStatus() const;
		AUDIOINSIGHTS_API void ResetMessageProcessType();

	protected:
		class FTraceAnalyzerBase : public Trace::IAnalyzer
		{
		public:
			AUDIOINSIGHTS_API FTraceAnalyzerBase(TSharedRef<FTraceProviderBase> InProvider);
			virtual ~FTraceAnalyzerBase() = default;

			AUDIOINSIGHTS_API virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;

		protected:
			AUDIOINSIGHTS_API virtual bool OnEventSuccess(uint16 RouteId, EStyle Style, const FOnEventContext& Context);
			AUDIOINSIGHTS_API virtual bool OnEventFailure(uint16 RouteId, EStyle Style, const FOnEventContext& Context);

			// Override in derived to handle incoming trace messages
			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) = 0;

			AUDIOINSIGHTS_API bool ShouldIgnoreNewEvents() const;
			AUDIOINSIGHTS_API virtual bool ShouldProcessNewEvents() const;

			template <typename TProviderType>
			TProviderType& GetProvider()
			{
				return *StaticCastSharedRef<TProviderType>(Provider);
			}

			template <typename TProviderType>
			const TProviderType& GetProvider() const
			{
				return *StaticCastSharedRef<TProviderType>(Provider);
			}

			// Will cache a message, and add it to the provided OutMessageQueue for processing
			template <TIsCacheableMessage T>
			void CacheMessage(const FOnEventContext& Context, TAnalyzerMessageQueue<T>& OutMessageQueue)
			{
				TSharedPtr<T> Message = MakeShared<T>(Context);
				AddMessageToCache(MoveTemp(Message), OutMessageQueue);
			}


			// If processing is enabled, will add a message to the provided OutMessageQueue for processing, but will not cache it.
			template <TIsCacheableMessage T>
			void AddMessageToQueue(const FOnEventContext& Context, TAnalyzerMessageQueue<T>& OutMessageQueue)
			{
				if (ShouldProcessNewEvents())
				{
					OutMessageQueue.Enqueue(T(Context));
				}
			}

		private:
			AUDIOINSIGHTS_API virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

			bool HandleChannelAnnounceEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context);
			bool HandleChannelToggleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context);

			template <TIsCacheableMessage T>
			void AddMessageToCache(TSharedPtr<T>&& Message, TAnalyzerMessageQueue<T>& OutMessageQueue)
			{
				CacheManager.AddMessageToCache(Message);

				if (ShouldProcessNewEvents())
				{
					T MessageCopy = *Message;
					OutMessageQueue.Enqueue(MoveTemp(MessageCopy));
				}
			}

			TSharedRef<FTraceProviderBase> Provider;

			enum : uint16
			{
				RouteID_ChannelAnnounce = TNumericLimits<uint16>::Max() - 1u,
				RouteID_ChannelToggle = TNumericLimits<uint16>::Max()
			};

			FAudioInsightsCacheManager& CacheManager;

			TOptional<uint32> AudioChannelID;
			TOptional<uint32> AudioMixerChannelID;

			bool bAudioChannelIsEnabled = false;
			bool bAudioMixerChannelIsEnabled = false;
		};

		uint64 GetLastMessageId() const
		{
			return LastMessageId;
		}

		AUDIOINSIGHTS_API virtual bool IsMessageProcessingPaused() const;

		uint64 LastUpdateId = 0;
		bool bForceUpdate = false;

	private:
		std::atomic<uint64> LastMessageId { 0 };
		FName Name;

		friend class FTraceAnalyzerBase;
	};

	template<typename EntryKey, typename EntryType /* = IDashboardDataViewEntry */>
	class TDeviceDataMapTraceProvider : public FTraceProviderBase
	{
	public:
		using KeyType = EntryKey;
		using ValueType = EntryType;

		using FDeviceData = TSortedMap<EntryKey, EntryType>;
		using FEntryPair = TPair<EntryKey, EntryType>;

		TDeviceDataMapTraceProvider(FName InName)
			: FTraceProviderBase(InName)
#if WITH_EDITOR
			, bIsTraceActive(FTraceAuxiliary::IsConnected())
#endif // WITH_EDITOR
		{
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(*InName.ToString(), 0.0f, [this](float DeltaTime)
			{
				if (LastUpdateId != GetLastMessageId() && !IsMessageProcessingPaused())
				{
					ProcessMessages();
				}
				LastUpdateId = GetLastMessageId();

				const bool bShouldForceUpdate = ProcessManuallyUpdatedEntries();
				if (bShouldForceUpdate)
				{
					bForceUpdate = true;
				}

				return true;
			});

#if WITH_EDITOR
			RegisterDelegates();
#endif // WITH_EDITOR
		}

		virtual ~TDeviceDataMapTraceProvider()
		{
#if WITH_EDITOR
			UnregisterDelegates();
#endif // WITH_EDITOR

			FTSTicker::RemoveTicker(TickerHandle);
		}

		const TMap<::Audio::FDeviceId, FDeviceData>& GetDeviceDataMap() const
		{
			return DeviceDataMap;
		}

		::Audio::FDeviceId GetFilteredAudioDeviceID() const
		{
#if WITH_EDITOR
			const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
			return AudioInsightsModule.GetDeviceId();
#else
			return GetAudioDeviceIdFromDeviceDataMap();
#endif // WITH_EDITOR
		}

		const FDeviceData* FindFilteredDeviceData() const
		{
			const ::Audio::FDeviceId AudioDeviceId = GetFilteredAudioDeviceID();
			return DeviceDataMap.Find(AudioDeviceId);
		}

		virtual void ClearActiveAudioDeviceEntries()
		{
#if WITH_EDITOR
			const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
			const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();
#else
			const ::Audio::FDeviceId AudioDeviceId = GetAudioDeviceIdFromDeviceDataMap();
#endif // WITH_EDITOR

			DeviceDataMap.Remove(AudioDeviceId);
		}

	protected:
		using Super = TDeviceDataMapTraceProvider<EntryKey, EntryType>;

		TMap<::Audio::FDeviceId, FDeviceData> DeviceDataMap;

		virtual void Reset() override
		{
			DeviceDataMap.Empty();
			FTraceProviderBase::Reset();
		}

		virtual void OnTraceChannelsEnabled() override
		{
			DeviceDataMap.Empty();
			FTraceProviderBase::OnTraceChannelsEnabled();
		}

		template <typename TMsgType>
		void ProcessMessageQueue(
			TAnalyzerMessageQueue<TMsgType>& InQueue,
			TFunctionRef<EntryType*(const TMsgType&)> GetEntry,
			TFunctionRef<void(const TMsgType&, EntryType*)> ProcessEntry)
		{
#if WITH_EDITOR
			if (bIsTraceActive)
#endif // WITH_EDITOR
			{
				TArray<TMsgType> Messages = InQueue.DequeueAll();
				for (const TMsgType& Msg : Messages)
				{
					EntryType* Entry = GetEntry(Msg);
					ProcessEntry(Msg, Entry);
				};
			}
		}

		EntryType* FindDeviceEntry(::Audio::FDeviceId InDeviceId, const EntryKey& InKey)
		{
			if (FDeviceData* Entry = DeviceDataMap.Find(InDeviceId))
			{
				return Entry->Find(InKey);
			}

			return nullptr;
		}

		const EntryType* FindDeviceEntry(::Audio::FDeviceId InDeviceId, const EntryKey& InKey) const
		{
			if (const FDeviceData* Entry = DeviceDataMap.Find(InDeviceId))
			{
				return Entry->Find(InKey);
			}

			return nullptr;
		}

		FDeviceData* FindFilteredDeviceData()
		{
#if WITH_EDITOR
			const IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetEditorChecked();
			const ::Audio::FDeviceId AudioDeviceId = AudioInsightsModule.GetDeviceId();
#else
			const ::Audio::FDeviceId AudioDeviceId = GetAudioDeviceIdFromDeviceDataMap();
#endif // WITH_EDITOR

			return DeviceDataMap.Find(AudioDeviceId);
		}

		EntryType& FindOrAddDeviceEntry(::Audio::FDeviceId InDeviceId, const EntryKey& InKey)
		{
			ensure(InDeviceId != INDEX_NONE);
			FDeviceData& DeviceData = DeviceDataMap.FindOrAdd(InDeviceId);
			return DeviceData.FindOrAdd(InKey);
		}

		bool RemoveDeviceEntry(::Audio::FDeviceId InDeviceId, const EntryKey& InKey)
		{
			if (FDeviceData* DeviceData = DeviceDataMap.Find(InDeviceId))
			{
				if (DeviceData->Remove(InKey) > 0)
				{
					if (DeviceData->IsEmpty())
					{
						DeviceDataMap.Remove(InDeviceId);
					}

					return true;
				}
			}

			return false;
		}

		void UpdateDeviceEntry(::Audio::FDeviceId InDeviceId, const EntryKey& InKey, TFunctionRef<void(EntryType&)> InEntryMutator)
		{
			EntryType& Entry = FindOrAddDeviceEntry(InDeviceId, InKey);
			InEntryMutator(Entry);
		}

#if !WITH_EDITOR
	const uint32 GetAudioDeviceIdFromDeviceDataMap() const
	{
		// In Audio Insights for Unreal Insights we don't have access to the Audio Device Manager, 
		// we need to extract the AudioDeviceId from the already recorded trace info stored in DeviceDataMap
		uint32 AudioDeviceId = 1; // 1 is the default audio device id

		for (const auto& [DeviceId, DeviceData] : DeviceDataMap)
		{
			if (DeviceId > AudioDeviceId)
			{
				AudioDeviceId = DeviceId;
			}
		}

		return AudioDeviceId;
	}
#endif // !WITH_EDITOR

	private:
#if WITH_EDITOR
		void RegisterDelegates()
		{
			FTraceAuxiliary::OnTraceStarted.AddRaw(this, &TDeviceDataMapTraceProvider::OnTraceStarted);
			FTraceAuxiliary::OnTraceStopped.AddRaw(this, &TDeviceDataMapTraceProvider::OnTraceStopped);
			FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddRaw(this, &TDeviceDataMapTraceProvider::OnAudioDeviceCreated);
		}

		void UnregisterDelegates()
		{
			FAudioDeviceManagerDelegates::OnAudioDeviceCreated.RemoveAll(this);
			FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
			FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
		}

		void OnTraceStarted(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
		{
			bIsTraceActive = true;
		}

		void OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
		{
			bIsTraceActive = false;
		}

		void OnAudioDeviceCreated(::Audio::FDeviceId InDeviceId)
		{
			// Check if old audio device data needs to be removed from DeviceDataMap
			if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				for (auto It = DeviceDataMap.CreateIterator(); It; ++It)
				{
					if (!DeviceManager->IsValidAudioDevice(It.Key()))
					{
						It.RemoveCurrent();
					}
				}
			}
		}
#endif // WITH_EDITOR

#if WITH_EDITOR
		bool bIsTraceActive = false;
#endif // WITH_EDITOR

		FTSTicker::FDelegateHandle TickerHandle;
	};
} // namespace UE::Audio::Insights
