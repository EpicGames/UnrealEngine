// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsStyle.h"
#include "DSP/Dsp.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Audio::Insights
{
	class AudioInsightsUtils
	{
	public:
		static FText ConvertToDecibelsText(const float InAmplitude)
		{
			if (FMath::IsNearlyEqual(InAmplitude, 1.0f))
			{
				return FText::AsNumber(0.0f, FSlateStyle::Get().GetAmpFloatFormat());
			}
			else if (FMath::IsNearlyZero(InAmplitude))
			{
				static const FText MinusInifinityDbText = INVTEXT("-∞");
				return MinusInifinityDbText;
			}

			return FText::AsNumber(::Audio::ConvertToDecibels(InAmplitude), FSlateStyle::Get().GetAmpFloatFormat());
		}

		static FString ResolveObjectDisplayName(const FString& InObjectPath)
		{
			const FSoftObjectPath SoftObjectPath = FSoftObjectPath(InObjectPath);

			if (SoftObjectPath.IsAsset())
			{
				return SoftObjectPath.GetAssetName();
			}
			else if (SoftObjectPath.IsSubobject())
			{
				FString Left;
				FString Right;
				if (SoftObjectPath.GetSubPathString().Split(".", &Left, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					return Right;
				}
			}

			return InObjectPath;
		}
	};
} // namespace UE::Audio::Insights
