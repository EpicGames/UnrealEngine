// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Cache/IAudioCachedMessage.h"
#include "Common/PagedArray.h"
#include "TraceServices/Containers/SlabAllocator.h"
#include "TraceServices/Model/AnalysisSession.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FAudioCachedMessageChunk
	{
	public:
		using TCachePagedArray = TraceServices::TPagedArray<TSharedPtr<class IAudioCachedMessage>>;

		FAudioCachedMessageChunk(TraceServices::FSlabAllocator& InAllocator, const uint32 InChunkSize)
			: Allocator(InAllocator)
			, MaxChunkSize(InChunkSize)
		{ }

		virtual ~FAudioCachedMessageChunk() = default;

		UE_API void AddMessageToChunk(const TSharedPtr<class IAudioCachedMessage> Message);
		UE_API void ClearChunk();

		UE_API uint32 GetCurrentChunkSize() const;
		UE_API bool IsChunkFull() const;
		UE_API bool HasAnyData() const;
		
		UE_API const TMap<FName, TSharedPtr<TCachePagedArray>>& GetAllChunkMessages() const;

		UE_API bool TimestampIsInChunkRange(const double Timestamp) const;
		UE_API double GetChunkTimeRangeStart() const;
		UE_API double GetChunkTimeRangeEnd() const;

		// Finds the closest message to a specific timestamp and where predicate returns true.
		// Will iterate backwards from param Timestamp until the first message of type MessageID is found
		// @param Predicate: A callable that returns a BooleanTestable value.
		//					 E.g. [](const T& InMessage){ return InMessage.GetID() == 10; }
		template<TIsCacheableMessage T, typename Predicate>
		const T* FindClosestMessageByPredicate(const FName& MessageID, const double Timestamp, Predicate Pred) const
		{
			const TSharedPtr<TCachePagedArray>* MessageCache = Messages.Find(MessageID);
			if (MessageCache == nullptr || !MessageCache->IsValid() || (*MessageCache)->Num() == 0u)
			{
				return nullptr;
			}

			const TCachePagedArray& MessagesArray = *(MessageCache->Get());
			const int32 ClosestMessageToTimeStampIndex = TraceServices::PagedArrayAlgo::BinarySearchClosestBy(MessagesArray, Timestamp,
			[](const TSharedPtr<class IAudioCachedMessage>& Msg) 
			{
				return Msg.IsValid() ? Msg->Timestamp : INVALID_TIMESTAMP;
			});

			// Iterate backwards from TimeMarker until we find a valid message and where the predicate returns true
			for (auto MessageItr = MessagesArray.GetIteratorFromItem(ClosestMessageToTimeStampIndex); MessageItr; --MessageItr)
			{
				const T* Message = static_cast<const T*>((*MessageItr).Get());

				if (Message && Pred(*Message))
				{
					return Message;
				}
			}

			return nullptr;
		}

		// Finds the closest message to a specific timestamp
		// Will iterate backwards from param Timestamp until the first message of type MessageID is found
		// Param ID can be overriden to filter the results further
		template<TIsCacheableMessage T>
		T* FindClosestMessage(const FName& MessageID, const double Timestamp, const uint64 ID = INDEX_NONE) const
		{
			return FindClosestMessageByPredicate<T>(MessageID,
				Timestamp,
				[ID](const T& InMessage)
				{
					return (ID == INDEX_NONE) || (InMessage.GetID() == ID);
				});
		}

	private:
		static constexpr double INVALID_TIMESTAMP = -1.0;

		TMap<FName, TSharedPtr<TCachePagedArray>> Messages;

		double RangeStartTimestamp = INVALID_TIMESTAMP;
		double RangeEndTimestamp = INVALID_TIMESTAMP;

		TraceServices::FSlabAllocator& Allocator;

		uint32 CurrentChunkSize = 0u;
		const uint32 MaxChunkSize = 0u;
	};
} // namespace UE::Audio::Insights

#undef UE_API
