// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "Cache/AudioCachedMessageChunk.h"
#include "Cache/IAudioCachedMessage.h"
#include "Containers/Ticker.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Misc/Optional.h"
#include "TraceServices/Containers/SlabAllocator.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	// Helper class that manages the cache container
	// The cache may or may not be limited in size based on the bLimitCacheSize flag
	// - In Editor, the cache size is always limited, behaving like a circular buffer
	// - In Standalone, when connected to a live session (either via direct or live trace), the cache size is limited, behaving like a circular buffer
	// - In Standalone, when loading in a .utrace file, the cache will grow to hold all of the trace file data in memory
	class FAudioInsightsCache
	{
	public:
		FAudioInsightsCache();
		virtual ~FAudioInsightsCache();

		FAudioCachedMessageChunk& operator[](uint32 Index)
		{
			return *Cache[Index].Get();
		}

		const FAudioCachedMessageChunk& operator[](uint32 Index) const
		{
			return *Cache[Index].Get();
		}

		void ResetCache(TraceServices::FSlabAllocator& Allocator, const uint32 InNumChunks);
		void ReserveNewChunks(TraceServices::FSlabAllocator& Allocator, const uint32 NumToReserve);

		UE_API uint32 Capacity() const;
		UE_API uint32 GetNextIndex(uint32 ReadIndex) const;
		UE_API uint32 GetPreviousIndex(uint32 ReadIndex) const;

	private:
#if !WITH_EDITOR
		void InitCacheIsLimited();
		void OnSessionChangedEvent();
#endif // !WITH_EDITOR

		TArray<TUniquePtr<FAudioCachedMessageChunk>> Cache;
		bool bLimitCacheSize = false;
	};

	class FAudioInsightsCacheManager
	{
	public:
		UE_API FAudioInsightsCacheManager();
		UE_API virtual ~FAudioInsightsCacheManager();

		UE_API void AddMessageToCache(TSharedPtr<IAudioCachedMessage> Message);
		UE_API void ClearCache();

		UE_API float GetCacheDuration() const;
		UE_API double GetCacheStartTimeStamp() const;
		UE_API double GetCacheEndTimeStamp() const;

		// Returns the total used memory size of the cache in bytes
		UE_API uint32 GetUsedCacheSize() const;

		// Returns the maximum memory size of the cache in bytes
		UE_API uint32 GetMaxCacheSize() const;

		// Returns the maximum number of chunks inside the cache
		UE_API uint32 GetNumChunks() const;

		// Returns the number of chunks with data present inside the cache
		UE_API uint32 GetNumUsedChunks() const;

		// Tries to find the number of chunks from the start of the cache for a specific timestamp, where lower indexed chunks contain older messages
		// If the timestamp does not exist within the current cache, the returned result will be unset
		UE_API TOptional<uint32> TryGetNumChunksFromStartForTimestamp(const double Timestamp) const;

		// Returns a chunk by index relative to the start of the cache
		// e.g. NumChunksFromStart = 0u will return the chunk with the earliest messages in the cache
		UE_API const FAudioCachedMessageChunk* const GetChunk(const uint32 NumChunksFromStart) const;

		// Returns true if the cache is full and the timestamp falls within the oldest chunk (next to be overwritten)
		UE_API bool IsTimestampMarkedForDeletion(const double Timestamp) const;

		// Returns one representative message per FName, collected as messages flow into the cache.
		// Used by the cache trace writer to discover and declare events without iterating all chunks.
		const TMap<FName, TSharedPtr<const IAudioCachedMessage>>& GetMessageTypeRegistry() const { return MessageTypeRegistry; }

		// Find the closest message to a timestamp in the cache for a specific message type and where predicate returns true.
		// Will look backwards from the chosen timestamp until a message is found
		// @param Predicate: A callable that returns a BooleanTestable value.
		//					 E.g. [](const T& InMessage){ return InMessage.GetID() == 10; }
		// @param BackOutTimestamp: When iterating backwards, if a chunk is detected with an end timestamp less than
		//							the backout timestamp, the search is canceled and this returns nullptr
		template<TIsCacheableMessage T, typename Predicate>
		const T* FindClosestMessageByPredicate(const FName& MessageID, const double Timestamp, Predicate Pred, const double BackOutTimestamp = -1.0) const
		{
			uint32 ReadIndex = INDEX_NONE;
			for (uint32 Index = 0u; Index < Cache.Capacity(); ++Index)
			{
				const FAudioCachedMessageChunk& Chunk = Cache[Index];

				if (Chunk.TimestampIsInChunkRange(Timestamp))
				{
					ReadIndex = Index;
					break;
				}
			}

			if (ReadIndex == INDEX_NONE)
			{
				// No chunks contain data from the current timestamp - return null
				return nullptr;
			}

			// Iterate backwards until we find a matching message
			for (uint32 Index = ReadIndex; ; )
			{
				const FAudioCachedMessageChunk& Chunk = Cache[Index];
				if (Chunk.GetChunkTimeRangeEnd() <= BackOutTimestamp)
				{
					break;
				}

				if (const T* FoundMessage = Chunk.FindClosestMessageByPredicate<T>(MessageID, Timestamp, Pred))
				{
					return FoundMessage;
				}
				else if (Index == StartIndex)
				{
					break;
				}
				else
				{
					Index = Cache.GetPreviousIndex(Index);
					if (Index == INDEX_NONE)
					{
						break;
					}
				}
			}

			return nullptr;
		}

		// Find the closest message to a timestamp in the cache for a specific message type
		// Will look backwards from the chosen timestamp until a message is found
		// @param ID: When overloaded, the search algorithm will only return messages that have a matching ID.
		//			  If left at INDEX_NONE, the search algorithm will return the closest message irrespective of ID
		// @param BackOutTimestamp: When iterating backwards, if a chunk is detected with an end timestamp less than
		//							the backout timestamp, the search is canceled and this returns nullptr
		template<TIsCacheableMessage T>
		const T* FindClosestMessage(const FName& MessageID, const double Timestamp, const uint64 ID = INDEX_NONE, const double BackOutTimestamp = -1.0) const
		{
			return FindClosestMessageByPredicate<T>(MessageID,
				Timestamp,
				[ID](const T& InMessage)
				{
					return (ID == INDEX_NONE) || (InMessage.GetID() == ID);
				},
				BackOutTimestamp);
		}

		// Iterates the cache over a chosen timestamp range for a specific message ID
		// Will stop iterating if the end of the cache is reached
		// @param ID: When overloaded, the search algorithm will only return messages that have a matching ID.
		//			  If left at INDEX_NONE, the search algorithm will return all matching messages irrespective of ID
		template<TIsCacheableMessage T>
		void IterateOverRange(const FName& MessageID, const double Start, const double End, TFunctionRef<void(const T& Message)> OnMessageFunc, const uint64 ID = INDEX_NONE) const
		{
			uint32 ReadIndex = StartIndex;
			double LatestChunkStartTimestamp = -1.0;
			for (uint32 Index = 0u; Index < Cache.Capacity(); ++Index)
			{
				const FAudioCachedMessageChunk& Chunk = Cache[ReadIndex];

				const double ChunkStartTime = Chunk.GetChunkTimeRangeStart();
				if (ChunkStartTime > End)
				{
					break;
				}

				if (Chunk.GetChunkTimeRangeEnd() < Start)
				{
					if (ChunkStartTime < LatestChunkStartTimestamp)
					{
						break;
					}
					else
					{
						ReadIndex = Cache.GetNextIndex(ReadIndex);
						continue;
					}
				}

				LatestChunkStartTimestamp = ChunkStartTime;

				const TSharedPtr<TraceServices::TPagedArray<TSharedPtr<IAudioCachedMessage>>>* FoundMessages = Chunk.GetAllChunkMessages().Find(MessageID);
				if (FoundMessages == nullptr || !FoundMessages->IsValid())
				{
					ReadIndex = Cache.GetNextIndex(ReadIndex);
					continue;
				}

				bool bReachedEndOfRange = false;
				for (const TSharedPtr<IAudioCachedMessage>& Message : **FoundMessages)
				{
					if (!Message.IsValid() || Message->Timestamp < Start)
					{
						continue;
					}

					if (Message->Timestamp > End)
					{
						bReachedEndOfRange = true;
						break;
					}

					if (ID == INDEX_NONE || Message->GetID() == ID)
					{
						OnMessageFunc(static_cast<const T&>(*Message));
					}
				}

				if (bReachedEndOfRange)
				{
					break;
				}
				else
				{
					ReadIndex = Cache.GetNextIndex(ReadIndex);
				}
			}
		}

		// Iterates over the cache from the start to a chosen timestamp for a specific message ID
		// Will stop iterating if the end of the cache is reached
		template<TIsCacheableMessage T>
		void IterateTo(const FName& MessageID, const double End, TFunctionRef<void(const T& Message)> OnMessageFunc, const uint64 ID = INDEX_NONE) const
		{
			constexpr double Start = 0.0;
			IterateOverRange<T>(MessageID, Start, End, OnMessageFunc, ID);
		}

		// Iterates over the cache from a chosen timestamp to the end of the cache for a specific message ID
		template<TIsCacheableMessage T>
		void IterateFrom(const FName& MessageID, const double Start, TFunctionRef<void(const T& Message)> OnMessageFunc, const uint64 ID = INDEX_NONE) const
		{
			IterateOverRange<T>(MessageID, Start, GetCacheEndTimeStamp(), OnMessageFunc, ID);
		}

		// Iterates over the entire cache for a specific message ID
		template<TIsCacheableMessage T>
		void IterateOverAll(const FName& MessageID, TFunctionRef<void(const T& Message)> OnMessageFunc, const uint64 ID = INDEX_NONE) const
		{
			constexpr double Start = 0.0;
			IterateOverRange<T>(MessageID, Start, GetCacheEndTimeStamp(), OnMessageFunc, ID);
		}

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnChunkOverwritten, double /*NewStartTimestamp*/);
		FOnChunkOverwritten OnChunkOverwritten;

	private:
		void CreateCache();
		void OnCacheSizeSettingsChanged();

		void ProcessPendingMessages(float DeltaTime);

		TraceServices::FSlabAllocator Allocator;

		FAudioInsightsCache Cache;
		TAnalyzerMessageQueue<TSharedPtr<IAudioCachedMessage>> NewMessageQueue;
		TMap<FName, TSharedPtr<const IAudioCachedMessage>> MessageTypeRegistry;

		// Number of chunks in the cache. Each chunk is 1 MB, so NumChunks == CacheSizeMB.
		uint32 NumChunks = 0u;

		uint32 WriteIndex = 0u;
		uint32 StartIndex = 0u;

		FTSTicker::FDelegateHandle TickerHandle;
		FDelegateHandle CacheSettingsChangedHandle;
	};
} // namespace UE::Audio::Insights

#undef UE_API