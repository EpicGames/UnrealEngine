// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "Templates/SharedPointer.h"
#include "Trace/Analyzer.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace UE::Audio::Insights
{
	class IAudioCachedMessage
	{
	public:
		virtual ~IAudioCachedMessage() = default;

		virtual uint32 GetSizeOf() const = 0;
		virtual uint64 GetID() const = 0;
		virtual const FName GetMessageName() const = 0;

		// Override to return a declare/write handler pair so this message type is included
		// when exporting the Audio Insights cache to a custom .utrace file.
		// See existing implementations in AudioInsights/Private/Messages/ for examples.
		virtual FCacheWriteHandler GetCacheWriteHandler() const { return {}; }

		double Timestamp = 0.0;
	};
} // namespace UE::Audio::Insights

// Cacheable messages must inherit from IAudioCachedMessage and have a constructor that takes in a FOnEventContext argument
template<typename T>
concept TIsCacheableMessage = std::is_base_of_v<UE::Audio::Insights::IAudioCachedMessage, T> 
							  && requires (const UE::Trace::IAnalyzer::FOnEventContext& Context) { T(Context); };