// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "UObject/NameTypes.h"
#include "Views/DashboardViewFactory.h"


namespace UE::Audio::Insights
{
	class IDashboardFactory
	{
	public:
		virtual ~IDashboardFactory() = default;

		virtual void RegisterViewFactory(TSharedRef<IDashboardViewFactory> InFactory) = 0;
		virtual void UnregisterViewFactory(FName InDashboardName) = 0;
		virtual ::Audio::FDeviceId GetDeviceId() const = 0;
	};
} // namespace UE::Audio::Insights
