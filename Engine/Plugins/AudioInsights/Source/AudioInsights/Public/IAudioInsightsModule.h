// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsModuleInterface.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API AUDIOINSIGHTS_API

class IAudioInsightsTraceModule;

namespace UE::Audio::Insights
{
	class IDashboardViewFactory;
	class FAudioInsightsCacheManager;
	class FAudioInsightsDetailsSelectionManager;
	class FAudioInsightsTimingViewExtender;
	class FTraceProviderBase;
} // namespace UE::Audio::Insights

class IAudioInsightsModule : public IAudioInsightsModuleInterface
{
public:
	virtual void RegisterDashboardViewFactory(TSharedRef<UE::Audio::Insights::IDashboardViewFactory> InDashboardFactory) = 0;
	virtual void UnregisterDashboardViewFactory(FName InName) = 0;

	virtual ::Audio::FDeviceId GetDeviceId() const = 0;

	UE_API virtual IAudioInsightsTraceModule& GetTraceModule() override;

	UE_API virtual UE::Audio::Insights::FAudioInsightsCacheManager& GetCacheManager() override;

	UE_API virtual UE::Audio::Insights::FAudioInsightsTimingViewExtender& GetTimingViewExtender() override;

#if WITH_EDITOR
	UE_API virtual UE::Audio::Insights::FAudioInsightsDetailsSelectionManager& GetDetailsSelectionManager() override;
#endif // WITH_EDITOR

	virtual TSharedRef<SDockTab> CreateDashboardTabWidget(const FSpawnTabArgs& Args) = 0;

	UE_API static bool IsModuleLoaded();

	static UE_API bool IsLiveSession();

	static UE_API IAudioInsightsModule& GetChecked();

#ifdef WITH_EDITOR
	static UE_API IAudioInsightsModule& GetEditorChecked();
#endif

};

#undef UE_API
