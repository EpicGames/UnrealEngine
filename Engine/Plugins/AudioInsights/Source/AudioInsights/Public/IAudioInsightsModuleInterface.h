// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsDetailsSelectionManager.h"
#include "AudioInsightsTimingViewExtender.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IAudioInsightsTraceModule;

namespace UE::Audio::Insights
{
	class IDashboardViewFactory;
	class FTraceProviderBase;
} // namespace UE::Audio::Insights

class IAudioInsightsModuleInterface : public IModuleInterface
{
public:
	virtual void RegisterDashboardViewFactory(TSharedRef<UE::Audio::Insights::IDashboardViewFactory> InDashboardFactory) = 0;
	virtual void UnregisterDashboardViewFactory(FName InName) = 0;

	// Allows external plug-ins to batch-define events and categories for the Audio Insights Event Log.
	// Best called from StartupModule. Categories and events are merged across callers; duplicates are skipped.
	// Event names should match the strings you pass to Audio::Trace::EventLog::SendEvent.
	virtual void RegisterEventLogCategories(const TMap<FString, TSet<FString>>& InCategoriesToEvents) = 0;

	// Allows external plug-ins to provide localized display names for their custom event types and categories.
	// Maps event/category ID strings to localized FText. Call after RegisterEventLogCategories.
	virtual void RegisterEventLogDisplayNames(const TMap<FString, FText>& InDisplayNames) = 0;

	virtual ::Audio::FDeviceId GetDeviceId() const = 0;

	virtual IAudioInsightsTraceModule& GetTraceModule() = 0;
	virtual UE::Audio::Insights::FAudioInsightsCacheManager& GetCacheManager() = 0;
	virtual UE::Audio::Insights::FAudioInsightsTimingViewExtender& GetTimingViewExtender() = 0;

#if WITH_EDITOR
	virtual UE::Audio::Insights::FAudioInsightsDetailsSelectionManager& GetDetailsSelectionManager() = 0;
#endif // WITH_EDITOR
};
