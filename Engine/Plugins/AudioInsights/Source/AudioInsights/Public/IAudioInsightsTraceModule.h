// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "AudioInsightsTraceProviderBase.h"
#include "TraceServices/ModuleService.h"

namespace UE::Audio::Insights
{
	class FTraceProviderBase;
	enum class ETraceMode : uint8;
} // namespace UE::Audio::Insights

namespace TraceServices
{
	struct FSessionInfo;
} // namespace TraceServices

class IAudioInsightsTraceModule : public TraceServices::IModule
{
public:
	virtual void AddTraceProvider(TSharedPtr<UE::Audio::Insights::FTraceProviderBase> TraceProvider) = 0;

	virtual void StartTraceAnalysis(const bool bInOnlyTraceAudioChannels, const UE::Audio::Insights::ETraceMode InTraceMode) = 0;
	virtual bool IsTraceAnalysisActive() const = 0;
	virtual void StopTraceAnalysis() = 0;
	
	virtual void OnOnlyTraceAudioChannelsStateChanged(const bool bOnlyTraceAudioChannels) = 0;

	virtual bool AudioChannelsCanBeManuallyEnabled() const = 0;

	virtual void ExecuteConsoleCommand(const FString& InCommandStr) const = 0;

	virtual void SaveCacheSnapshot() = 0;

	virtual UE::Audio::Insights::ETraceMode GetTraceMode() const = 0;

#if !WITH_EDITOR
	virtual void InitializeSessionInfo(const TraceServices::FSessionInfo& SessionInfo) = 0;
	virtual void RequestChannelUpdate() = 0;
	virtual void ResetTicker() = 0;
	virtual bool TraceControllerIsAvailable() const = 0;
#else
	virtual bool IsAudioInsightsTrace() const = 0;
#endif // !WITH_EDITOR
};
