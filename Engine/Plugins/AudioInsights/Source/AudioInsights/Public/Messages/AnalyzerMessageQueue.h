// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/MpscQueue.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopeLock.h"

namespace UE::Audio::Insights
{
	template <typename T>
	struct TAnalyzerMessageQueue
	{
		static constexpr double MaxHistoryLimitSec = 5.0;

		explicit TAnalyzerMessageQueue(const double InHistoryLimitSec = 5.0)
			: HistoryLimitSec(FMath::Clamp(InHistoryLimitSec, UE_DOUBLE_KINDA_SMALL_NUMBER, MaxHistoryLimitSec))
		{
		}

		virtual ~TAnalyzerMessageQueue() = default;

		bool IsEmpty() const
		{
			return Data.IsEmpty();
		}

		TArray<T> DequeueAll()
		{
			TArray<T> Output;

			{
				FScopeLock Lock(&AccessCritSec);

				if (!Data.IsEmpty())
				{
					TimestampedMessage TimestampedMessage;
					while (Data.Dequeue(TimestampedMessage))
					{
						Output.Add(MoveTemp(TimestampedMessage.Message));
					}
				}
			}

			return Output;
		}

		void Enqueue(T&& InMessage)
		{
			TimestampedMessage NewMessage(MoveTemp(InMessage));

			{
				// Clear queue if queue time limit hit.
				// Have to lock as queues do not support
				// removal from producer thread. Should rarely
				// be expensive, as that would hit only under
				// conditions where consuming thread is stalled.
				FScopeLock Lock(&AccessCritSec);

				if (!Data.IsEmpty())
				{
					double FirstTimeStamp = Data.Peek()->TimestampWhenCreated;
					while ((NewMessage.TimestampWhenCreated - FirstTimeStamp) > HistoryLimitSec)
					{
						if (!Data.Dequeue().IsSet() || Data.IsEmpty())
						{
							break;
						}
						FirstTimeStamp = Data.Peek()->TimestampWhenCreated;
					}
				}
			}

			Data.Enqueue(MoveTemp(NewMessage));
		}

	protected:
		struct TimestampedMessage
		{
			TimestampedMessage() = default;

			TimestampedMessage(T&& InMessage)
				: Message(MoveTemp(InMessage))
				, TimestampWhenCreated(FPlatformTime::Seconds())
			{
			}

			T Message;
			double TimestampWhenCreated = 0.0;
		};


		double HistoryLimitSec = 0.0;
		TMpscQueue<TimestampedMessage> Data;
		mutable FCriticalSection AccessCritSec;
	};
} // namespace UE::Audio::Insights
