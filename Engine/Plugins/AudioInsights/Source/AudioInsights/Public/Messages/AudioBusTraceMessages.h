// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Cache/IAudioCachedMessage.h"
#include "Providers/AudioMeterProvider.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Trace/Analyzer.h"
#include "Views/TableDashboardViewFactory.h"

#if !WITH_EDITOR
#include "Common/PagedArray.h"
#include "TraceServices/Model/AnalysisSession.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace AudioBusMessageNames
	{
		extern const FName Start;
		extern const FName HasActivity;
		extern const FName EnvelopeFollowerEnabled;
		extern const FName EnvelopeValues;
		extern const FName Stop;
	};

	enum class EAudioBusType : uint8
	{
		AssetBased,
		CodeGenerated,
		None
	};

	struct FAudioBusMessageBase : public IAudioCachedMessage
	{
		FAudioBusMessageBase() = default;
		FAudioBusMessageBase(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return AudioBusId; }

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 AudioBusId = INDEX_NONE;
	};

	struct FAudioBusStartMessage : public FAudioBusMessageBase
	{
		FAudioBusStartMessage() = default;
		FAudioBusStartMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return AudioBusMessageNames::Start; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Name;
		int32 NumChannels = 0;
		EAudioBusType AudioBusType = EAudioBusType::None;
	};

	struct FAudioBusHasActivityMessage : public FAudioBusMessageBase
	{
		FAudioBusHasActivityMessage() = default;
		FAudioBusHasActivityMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FAudioBusHasActivityMessage); };
		virtual const FName GetMessageName() const override { return AudioBusMessageNames::HasActivity; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		bool bHasActivity = false;
	};

	struct FAudioBusEnvelopeFollowerEnabledMessage : public FAudioBusMessageBase
	{
		FAudioBusEnvelopeFollowerEnabledMessage() = default;
		FAudioBusEnvelopeFollowerEnabledMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FAudioBusEnvelopeFollowerEnabledMessage); };
		virtual const FName GetMessageName() const override { return AudioBusMessageNames::EnvelopeFollowerEnabled; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		bool bEnvelopeFollowerEnabled = false;
	};

	struct FAudioBusEnvelopeValuesMessage : public FAudioBusMessageBase
	{
		FAudioBusEnvelopeValuesMessage() = default;
		FAudioBusEnvelopeValuesMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return AudioBusMessageNames::EnvelopeValues; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		TArray<float> EnvelopeValues;
	};

	struct FAudioBusStopMessage : public FAudioBusMessageBase
	{
		FAudioBusStopMessage() = default;
		FAudioBusStopMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FAudioBusStopMessage); };
		virtual const FName GetMessageName() const override { return AudioBusMessageNames::Stop; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	class FAudioBusMessages
	{
		TAnalyzerMessageQueue<FAudioBusStartMessage> StartMessages;
		TAnalyzerMessageQueue<FAudioBusHasActivityMessage> HasActivityMessages;
		TAnalyzerMessageQueue<FAudioBusEnvelopeFollowerEnabledMessage> EnvelopeFollowerEnabledMessages;
		TAnalyzerMessageQueue<FAudioBusEnvelopeValuesMessage> EnvelopeValuesMessages;
		TAnalyzerMessageQueue<FAudioBusStopMessage> StopMessages;

		friend class FAudioBusTraceProvider;
	};

	struct FAudioBusDashboardEntry : public IObjectDashboardEntry
	{
		FAudioBusDashboardEntry();
		FAudioBusDashboardEntry(FAudioBusDashboardEntry& Other); // // Copy constructor - make sure we take a deep copy of FAudioMeterInfo

		virtual ~FAudioBusDashboardEntry() = default;

		FAudioBusDashboardEntry& operator=(const FAudioBusDashboardEntry& Other);

		virtual FText GetDisplayName() const override { return FText::FromString(AudioBusType == EAudioBusType::AssetBased ? FSoftObjectPath(Name).GetAssetName() : Name); }
		virtual FString GetObjectPath() const override { return Name; }
		virtual const UObject* GetObject() const override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual UObject* GetObject() override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual bool IsValid() const override { return AudioBusId != static_cast<uint32>(INDEX_NONE); }

		uint32 AudioBusId = INDEX_NONE;
		double Timestamp = 0.0;

		TSharedRef<FAudioMeterInfo> AudioMeterInfo;

		FString Name;
		bool bHasActivity = false;
		bool bEnvelopeFollowerEnabled = false;
		EAudioBusType AudioBusType = EAudioBusType::None;
	};
} // namespace UE::Audio::Insights
