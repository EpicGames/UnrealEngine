// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Cache/IAudioCachedMessage.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "SoundTraceMessages.h"
#include "Trace/Analyzer.h"
#include "Views/TableDashboardViewFactory.h"

namespace UE::Audio::Insights
{
	enum class EAudioEventLogSoundCategory : uint8
	{
		None,
		MetaSound,
		SoundCue,
		ProceduralSource,
		SoundWave,
		SoundCueTemplate
	};

	enum class EAudioEventCacheState : uint8
	{
		Cached,
		NextToBeDeleted
	};

	struct FAudioEventLogMessage : public IAudioCachedMessage
	{
		FAudioEventLogMessage() = default;
		FAudioEventLogMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual uint64 GetID() const override { return PlayOrder; }
		virtual const FName GetMessageName() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 MessageID = INDEX_NONE;
		uint32 PlayOrder = INDEX_NONE;

		FString EventName;
		FString AssetPath;
		FString ActorLabel;
		FString ActorIconName;
		FString AudioComponentName;
		FString CategoryName;

		EAudioEventLogSoundCategory CategoryType;
	};

	class FAudioEventLogDashboardEntry : public FSoundAssetDashboardEntry
	{
	public:
		FAudioEventLogDashboardEntry() = default;
		virtual ~FAudioEventLogDashboardEntry() = default;

		virtual bool IsValid() const override
		{
			return MessageID != static_cast<uint32>(INDEX_NONE);
		}

		uint32 MessageID = INDEX_NONE;

		FString EventName;
		FString ActorLabel;
		FName ActorIconName;
		FString AudioComponentName;
		FString CategoryName;

		EAudioEventLogSoundCategory Category = EAudioEventLogSoundCategory::None;
		EAudioEventCacheState CachedState = EAudioEventCacheState::Cached;
		bool bCacheStatusIsDirty = false;
	};

	class FAudioEventLogMessages
	{
		TAnalyzerMessageQueue<FAudioEventLogMessage> AudioEventLogMessages;

		friend class FAudioEventLogTraceProvider;
	};
} // namespace UE::Audio::Insights
