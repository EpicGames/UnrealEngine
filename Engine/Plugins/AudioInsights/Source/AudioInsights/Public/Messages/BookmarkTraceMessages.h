// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Cache/IAudioCachedMessage.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	namespace BookmarkMessageNames
	{
		extern UE_API const FName Bookmark;
	};

	struct FBookmarkTraceMessage : public IAudioCachedMessage
	{
		UE_API FBookmarkTraceMessage(double InTimestamp, const FString& InLabel);

		virtual uint64 GetID() const override { return 0; }
		virtual const FName GetMessageName() const override { return BookmarkMessageNames::Bookmark; }
		virtual UE_API uint32 GetSizeOf() const override;
		virtual UE_API FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Label;
	};
} // namespace UE::Audio::Insights

#undef UE_API
