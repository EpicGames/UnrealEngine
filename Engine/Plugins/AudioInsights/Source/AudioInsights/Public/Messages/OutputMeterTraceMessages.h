// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsDataSource.h"
#include "Cache/IAudioCachedMessage.h"
#include "Messages/AnalyzerMessageQueue.h"

namespace UE::Audio::Insights
{
	namespace OutputMeterMessageNames
	{
		extern const FName MainSubmixLoaded;
		extern const FName MainSubmixAlivePing;
		extern const FName TruePeak;
		extern const FName LKFSValues;
	};

	struct FOutputMeterMessageBase : public IAudioCachedMessage
	{
		FOutputMeterMessageBase() = default;
		FOutputMeterMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return SubmixId; }

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 SubmixId = INDEX_NONE;
	};

	struct FOutputMeterMainSubmixLoadedMessage : public FOutputMeterMessageBase
	{
		FOutputMeterMainSubmixLoadedMessage() = default;
		FOutputMeterMainSubmixLoadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return OutputMeterMessageNames::MainSubmixLoaded; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FOutputMeterMainSubmixAlivePingMessage : public FOutputMeterMainSubmixLoadedMessage
	{
		FOutputMeterMainSubmixAlivePingMessage() = default;
		FOutputMeterMainSubmixAlivePingMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return OutputMeterMessageNames::MainSubmixAlivePing; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FOutputMeterTruePeakMessage : public FOutputMeterMessageBase
	{
		FOutputMeterTruePeakMessage() = default;
		FOutputMeterTruePeakMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return OutputMeterMessageNames::TruePeak; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		float TruePeakMaxValueDb = MIN_VOLUME_DECIBELS;
	};

	struct FOutputMeterLKFSValuesMessage : public FOutputMeterMessageBase
	{
		FOutputMeterLKFSValuesMessage() = default;
		FOutputMeterLKFSValuesMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return OutputMeterMessageNames::LKFSValues; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		float LongTermLoudness  = MIN_VOLUME_DECIBELS;
		float ShortTermLoudness = MIN_VOLUME_DECIBELS;
		float MomentaryLoudness = MIN_VOLUME_DECIBELS;
		float LoudnessRangeLowerBound = 0.0f;
		float LoudnessRangeUpperBound = 0.0f;
	};

	class FOutputMeterDashboardEntry : public IDashboardDataViewEntry
	{
	public:
		FOutputMeterDashboardEntry() = default;
		virtual ~FOutputMeterDashboardEntry() = default;

		virtual bool IsValid() const override
		{
			return SubmixId != static_cast<uint32>(INDEX_NONE);
		}

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		double Timestamp = 0.0;
		uint32 SubmixId = static_cast<uint32>(INDEX_NONE);

		float TruePeakMaxValueDb = MIN_VOLUME_DECIBELS;

		float LongTermLoudness  = MIN_VOLUME_DECIBELS;
		float ShortTermLoudness = MIN_VOLUME_DECIBELS;
		float MomentaryLoudness = MIN_VOLUME_DECIBELS;
		float LoudnessRangeLowerBound = 0.0f;
		float LoudnessRangeUpperBound = 0.0f;
	};

	class FOutputMeterMessages
	{
		TAnalyzerMessageQueue<FOutputMeterMainSubmixLoadedMessage> MainSubmixLoadedMessages;
		TAnalyzerMessageQueue<FOutputMeterMainSubmixAlivePingMessage> MainSubmixAlivePingMessages;
		TAnalyzerMessageQueue<FOutputMeterTruePeakMessage> TruePeakMessages;
		TAnalyzerMessageQueue<FOutputMeterLKFSValuesMessage> LKFSValuesMessages;

		friend class FOutputMeterTraceProvider;
	};
} // namespace UE::Audio::Insights
