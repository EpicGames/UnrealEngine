// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

namespace UE::Audio::Insights
{
	// Which render stage does this node belong to.
	// This enum is in the order the render stages occur - from root to audio device
	enum class ESignalFlowEntryType : int32
	{
		OwnerObject = 0,
		SoundSource,
		AudioBus,
		Submix,
		AudioDevice,

		///
		InvalidLow = TNumericLimits<int32>::Lowest(),
		InvalidMax = TNumericLimits<int32>::Max()
	};

	enum class ESignalFlowNodeDetailParam : uint8
	{
		Amplitude = 0u,
		Volume,
		Pitch,
		LPFFreq,
		HPFFreq,
		Priority,
		Distance,
		Attenuation,
		RelativeRenderCost,
		AudioComponentName,
		SendOutputVolume,

		///
		MAX
	};

	struct FSignalFlowEntryKey
	{
		FSignalFlowEntryKey() = default;
		FSignalFlowEntryKey(const ::Audio::FDeviceId InDeviceID, const ESignalFlowEntryType InEntryType, const uint32 InEntryID)
			: DeviceID(InDeviceID)
			, EntryType(InEntryType)
			, EntryID(InEntryID)
		{
		}

		friend uint32 GetTypeHash(const FSignalFlowEntryKey& Key)
		{
			return HashCombine(GetTypeHash(Key.DeviceID), GetTypeHash(static_cast<int32>(Key.EntryType)), GetTypeHash(Key.EntryID));
		}

		bool operator==(const FSignalFlowEntryKey& InOther) const
		{
			return DeviceID == InOther.DeviceID && EntryType == InOther.EntryType && EntryID == InOther.EntryID;
		}

		bool operator<(const FSignalFlowEntryKey& InOther) const
		{
			if (DeviceID != InOther.DeviceID)
			{
				return DeviceID < InOther.DeviceID;
			}

			if (EntryType != InOther.EntryType)
			{
				return static_cast<int32>(EntryType) < static_cast<int32>(InOther.EntryType);
			}

			return EntryID < InOther.EntryID;
		}

		::Audio::FDeviceId DeviceID = INDEX_NONE;
		ESignalFlowEntryType EntryType = ESignalFlowEntryType::SoundSource;
		uint32 EntryID = INDEX_NONE;
	};
} // namespace UE::Audio::Insights
