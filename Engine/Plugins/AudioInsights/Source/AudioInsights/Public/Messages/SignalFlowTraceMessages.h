// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Cache/IAudioCachedMessage.h"
#include "Containers/CircularBuffer.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Messages/AudioBusTraceMessages.h"
#include "Messages/SignalFlowEntryKey.h"
#include "Messages/SoundTraceMessages.h"
#include "Messages/SubmixTraceMessages.h"
#include "Trace/Analyzer.h"
#include "Views/TableDashboardViewFactory.h"

namespace UE::Audio::Insights
{
	namespace SignalFlowMessageNames
	{
		extern const FName SoundWaveToBusSend;
		extern const FName SoundWaveToSubmixSend;
		extern const FName SubmixToSubmixSend;
		extern const FName BusPatchWriterConnected;
		extern const FName BusPatchReaderConnected;
	};

	struct FSignalFlowMessageBase : public IAudioCachedMessage
	{
		FSignalFlowMessageBase() = default;
		FSignalFlowMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		::Audio::FDeviceId DeviceId = INDEX_NONE;
	};

	struct FSignalFlowSendMessageBase : public FSignalFlowMessageBase
	{
		FSignalFlowSendMessageBase() = default;
		FSignalFlowSendMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;

		struct FSendInfoPair
		{
			uint32 ReceiverID = INDEX_NONE;
			float SendLevel = 0.0f;
		};

		uint32 SenderID = INDEX_NONE;
		TArray<FSendInfoPair> SendInfos;
	};

	struct FSoundToBusSendMessage : public FSignalFlowSendMessageBase
	{
		FSoundToBusSendMessage() = default;
		FSoundToBusSendMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSignalFlowSendMessageBase(InContext)
		{
		}

		virtual uint64 GetID() const override { return GetTypeHash(FSignalFlowEntryKey{ DeviceId, ESignalFlowEntryType::SoundSource, SenderID }); }
		virtual const FName GetMessageName() const override { return SignalFlowMessageNames::SoundWaveToBusSend; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FBusPatchWriterConnectedMessage : public FSignalFlowSendMessageBase
	{
		FBusPatchWriterConnectedMessage() = default;
		FBusPatchWriterConnectedMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSignalFlowSendMessageBase(InContext)
		{
		}

		virtual uint64 GetID() const override { return GetTypeHash(FSignalFlowEntryKey{ DeviceId, ESignalFlowEntryType::SoundSource, SenderID }); }
		virtual const FName GetMessageName() const override { return SignalFlowMessageNames::BusPatchWriterConnected; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FBusPatchReaderConnectedMessage : public FSignalFlowSendMessageBase
	{
		FBusPatchReaderConnectedMessage() = default;
		FBusPatchReaderConnectedMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSignalFlowSendMessageBase(InContext)
		{
		}

		virtual uint64 GetID() const override { return GetTypeHash(FSignalFlowEntryKey{ DeviceId, ESignalFlowEntryType::AudioBus, SenderID }); }
		virtual const FName GetMessageName() const override { return SignalFlowMessageNames::BusPatchReaderConnected; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FSoundToSubmixSendMessage : public FSignalFlowSendMessageBase
	{
		FSoundToSubmixSendMessage() = default;
		FSoundToSubmixSendMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSignalFlowSendMessageBase(InContext)
		{
		}

		virtual uint64 GetID() const override { return GetTypeHash(FSignalFlowEntryKey{ DeviceId, ESignalFlowEntryType::SoundSource, SenderID }); }
		virtual const FName GetMessageName() const override { return SignalFlowMessageNames::SoundWaveToSubmixSend; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FSubmixToSubmixSendMessage : public FSignalFlowMessageBase
	{
		FSubmixToSubmixSendMessage() = default;
		FSubmixToSubmixSendMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return GetTypeHash(FSignalFlowEntryKey{ DeviceId, ESignalFlowEntryType::Submix, SenderID }); }
		virtual const FName GetMessageName() const override { return SignalFlowMessageNames::SubmixToSubmixSend; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		uint32 SenderID = INDEX_NONE;
		uint32 ReceiverID = INDEX_NONE;
		float SendLevel = 0.0f;
		bool bIsFinalStage = false;
	};

	class FSignalFlowMessages
	{
		TAnalyzerMessageQueue<FSoundWaveStartMessage> SoundWaveStartMessages;
		TAnalyzerMessageQueue<FSoundWaveIsAlivePingMessage> SoundWaveIsAlivePingMessages;
		TAnalyzerMessageQueue<FSoundWaveStopMessage> SoundWaveStopMessages;
		TAnalyzerMessageQueue<FSoundStopMessage> SoundStopMessages;
		TAnalyzerMessageQueue<FAudioBusStartMessage> BusStartMessages;
		TAnalyzerMessageQueue<FAudioBusStartMessage> BusIsAlivePingMessages;
		TAnalyzerMessageQueue<FAudioBusStopMessage> BusStopMessages;
		TAnalyzerMessageQueue<FSubmixLoadedMessage> SubmixLoadedMessages;
		TAnalyzerMessageQueue<FSubmixAlivePingMessage> SubmixAliveMessages;
		TAnalyzerMessageQueue<FSubmixUnloadedMessage> SubmixUnloadedMessages;

		TAnalyzerMessageQueue<FSoundToBusSendMessage> SoundToBusSendMessages;
		TAnalyzerMessageQueue<FBusPatchWriterConnectedMessage> BusPatchWriterConnectedMessages;
		TAnalyzerMessageQueue<FBusPatchReaderConnectedMessage> BusPatchReaderConnectedMessages;
		TAnalyzerMessageQueue<FSoundToSubmixSendMessage> SoundToSubmixSendMessages;
		TAnalyzerMessageQueue<FSubmixToSubmixSendMessage> SubmixToSubmixSendMessages;

		TAnalyzerMessageQueue<FSoundEnvelopeMessage> SoundEnvelopeMessages;
		TAnalyzerMessageQueue<FSoundVolumeMessage> SoundVolumeMessages;
		TAnalyzerMessageQueue<FSoundPitchMessage> SoundPitchMessages;
		TAnalyzerMessageQueue<FSoundLPFFreqMessage> SoundLPFFreqMessages;
		TAnalyzerMessageQueue<FSoundHPFFreqMessage> SoundHPFFreqMessages;
		TAnalyzerMessageQueue<FSoundPriorityMessage> SoundPriorityMessages;
		TAnalyzerMessageQueue<FSoundDistanceMessage> SoundDistanceMessages;
		TAnalyzerMessageQueue<FSoundDistanceAttenuationMessage> SoundAttenuationMessages;
		TAnalyzerMessageQueue<FSoundRelativeRenderCostMessage> SoundRelativeRenderCostMessages;

		TAnalyzerMessageQueue<FAudioBusEnvelopeValuesMessage> AudioBusEnvelopeMessages;
		TAnalyzerMessageQueue<FSubmixEnvelopeValuesMessage> SubmixEnvelopeMessages;

		friend class FSignalFlowTraceProvider;
	};

	struct FSignalFlowOutputData
	{
		TOptional<float> SendLevel;
	};

	struct FSignalFlowDashboardEntry : public IObjectDashboardEntry
	{
		FSignalFlowDashboardEntry();
		virtual ~FSignalFlowDashboardEntry() = default;

		virtual FText GetDisplayName() const override 
		{ 
			const FString DisplayName = FSoftObjectPath(Name).GetAssetName();
			return DisplayName.IsEmpty() ? FText::FromString(Name) : FText::FromString(DisplayName);
		}

		virtual FString GetObjectPath() const override { return Name; }
		virtual const UObject* GetObject() const override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual UObject* GetObject() override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual bool IsValid() const override { return SignalFlowNodeID != static_cast<uint32>(INDEX_NONE); }

		FSignalFlowEntryKey GetSignalFlowEntryKey() const { return { DeviceId, EntryType, SignalFlowNodeID }; }

		void WriteNewAmplitudeValue(const float Value);
		void ResetDataBuffers();
		bool IsEntryAmplitudeWindowSilent() const;

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		ESignalFlowEntryType EntryType = ESignalFlowEntryType::SoundSource;
		uint32 SignalFlowNodeID = INDEX_NONE;
		double Timestamp = 0.0;

		FString Name;
		FName IconName = NAME_None;

		TSet<FSignalFlowEntryKey> Inputs;
		TSortedMap<FSignalFlowEntryKey, FSignalFlowOutputData> Outputs;

		TOptional<FSignalFlowEntryKey> LinkedSourceBus;
		TSet<FSignalFlowEntryKey> LinkedSoundSources;

		TSet<FSignalFlowEntryKey> LinkedBusPatchInputs;   // Entries that send bus patch connections to this entry
		TSet<FSignalFlowEntryKey> LinkedBusPatchOutputs;   // Entries that this entry sends bus patch connections to

		TCircularBuffer<float> AmplitudeDataRange;
		int32 AmplitudeWriteIndex = 0;
		uint32 NumConsecutiveSilentAmplitudeValues = 0u;

		TOptional<float> Amplitude;
		TOptional<float> Volume;
		TOptional<float> Pitch;
		TOptional<float> LPFFreq;
		TOptional<float> HPFFreq;
		TOptional<float> Priority;
		TOptional<float> Distance;
		TOptional<float> Attenuation;
		TOptional<float> RelativeRenderCost;

		TOptional<FString> AudioComponentName;
	};
} // namespace UE::Audio::Insights
