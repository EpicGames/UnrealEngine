// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Cache/IAudioCachedMessage.h"
#include "DSP/Dsp.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Trace/Analyzer.h"
#include "UObject/SoftObjectPath.h"
#include "Views/TreeDashboardViewFactory.h"

#if !WITH_EDITOR
#include "Common/PagedArray.h"
#include "TraceServices/Model/AnalysisSession.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	static constexpr double INVALID_TIMEOUT = -1.0;

	enum class ESoundDashboardEntryType : uint8
	{
		None,
		MetaSound,
		SoundCue,
		ProceduralSource,
		SoundWave,
		SoundCueTemplate,
		Pinned
	};

	namespace SoundClassNames
	{
		const FString MetaSoundSource = TEXT("MetaSoundSource");
		const FString SoundWaveProcedural = TEXT("SoundWaveProcedural");
		const FString SoundCue = TEXT("SoundCue");
		const FString SoundWave = TEXT("SoundWave");
		const FString SoundCueTemplate = TEXT("SoundCueTemplate");
	}

	namespace SoundMessageNames
	{
		extern const FName SoundStart;
		extern const FName SoundIsAlivePing;
		extern const FName SoundWaveStart;
		extern const FName SoundWaveIsAlivePing;
		extern const FName SoundStop;
		extern const FName SoundWaveStop;
		
		extern const FName PriorityParam;
		extern const FName DistanceParam;
		extern const FName DistanceAttenuationParam;
		extern const FName HPFFreqParam;
		extern const FName LPFFreqParam;
		extern const FName EnvelopeParam;
		extern const FName PitchParam;
		extern const FName VolumeParam;
		extern const FName RelativeRenderCostParam;
	};

	namespace SoundMessageUtils
	{
		uint64 GeneratePlayOrderUniqueID(const uint32 ActiveSoundPlayOrder, const uint32 WaveInstancePlayOrder);
		bool IsPriorityValueSetToMax(const float PriorityValue);
	};

	// Trace messages
	struct FSoundMessageBase : public IAudioCachedMessage
	{
		FSoundMessageBase() = default;
		FSoundMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		::Audio::FDeviceId DeviceId = INDEX_NONE;
	};

	struct FSoundParameterMessage : public FSoundMessageBase
	{
		FSoundParameterMessage() = default;

		FSoundParameterMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundMessageBase(InContext)
		{
			WaveInstancePlayOrder = InContext.EventData.GetValue<uint32>("PlayOrder");
			ActiveSoundPlayOrder = InContext.EventData.GetValue<uint32>("ActiveSoundPlayOrder");
		}

		virtual uint64 GetID() const override { return SoundMessageUtils::GeneratePlayOrderUniqueID(ActiveSoundPlayOrder, WaveInstancePlayOrder); }

		uint32 WaveInstancePlayOrder = INDEX_NONE;
		uint32 ActiveSoundPlayOrder = INDEX_NONE;
	};

	struct FSoundStartMessage : public FSoundMessageBase
	{
		FSoundStartMessage() = default;
		FSoundStartMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundStart; }
		virtual uint64 GetID() const override { return ActiveSoundPlayOrder; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Name;
		ESoundDashboardEntryType EntryType;
		FString ActorLabel;
		FString AudioComponentName;
		uint32 ActiveSoundPlayOrder = INDEX_NONE;
	};

	struct FSoundIsAlivePingMessage : public FSoundStartMessage
	{
		FSoundIsAlivePingMessage() = default;
		FSoundIsAlivePingMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundStartMessage(InContext)
		{
		}

		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundIsAlivePing; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FSoundWaveStartMessage : public FSoundStartMessage
	{
		FSoundWaveStartMessage() = default;

		FSoundWaveStartMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundStartMessage(InContext)
		{
			WaveInstancePlayOrder = InContext.EventData.GetValue<uint32>("PlayOrder");
			ActiveSoundPlayOrder = InContext.EventData.GetValue<uint32>("ActiveSoundPlayOrder");
			SourceBusID = InContext.EventData.GetValue<int32>("SourceBusID", INDEX_NONE);

			InContext.EventData.GetString("ActorIconName", ActorIconName);
		}

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundWaveStart; }
		virtual uint64 GetID() const override { return SoundMessageUtils::GeneratePlayOrderUniqueID(ActiveSoundPlayOrder, WaveInstancePlayOrder); }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		uint32 WaveInstancePlayOrder = INDEX_NONE;
		FString ActorIconName;
		int32 SourceBusID = INDEX_NONE;
	};

	struct FSoundWaveIsAlivePingMessage : public FSoundWaveStartMessage
	{
		FSoundWaveIsAlivePingMessage() = default;
		FSoundWaveIsAlivePingMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundWaveStartMessage(InContext)
		{
		}

		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundWaveIsAlivePing; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FSoundStopMessage : public FSoundMessageBase
	{
		FSoundStopMessage() = default;
		FSoundStopMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundMessageBase(InContext)
		{
			ActiveSoundPlayOrder = InContext.EventData.GetValue<uint32>("PlayOrder");
		}

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundStop; }
		virtual uint64 GetID() const override { return ActiveSoundPlayOrder; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		uint32 ActiveSoundPlayOrder = INDEX_NONE;
	};

	struct FSoundWaveStopMessage : public FSoundMessageBase
	{
		FSoundWaveStopMessage() = default;

		FSoundWaveStopMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSoundMessageBase(InContext)
		{
			WaveInstancePlayOrder = InContext.EventData.GetValue<uint32>("PlayOrder");
			ActiveSoundPlayOrder = InContext.EventData.GetValue<uint32>("ActiveSoundPlayOrder");
		}

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SoundMessageNames::SoundWaveStop; }
		virtual uint64 GetID() const override { return SoundMessageUtils::GeneratePlayOrderUniqueID(ActiveSoundPlayOrder, WaveInstancePlayOrder); }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		uint32 WaveInstancePlayOrder = INDEX_NONE;
		uint32 ActiveSoundPlayOrder = INDEX_NONE;
	};

#define DEFINE_SOUND_PARAM_MESSAGE(ClassName, MessageName, ParamName, Type, Default)			\
	struct ClassName : public FSoundParameterMessage											\
	{																							\
		ClassName() = default;																	\
		ClassName(const Trace::IAnalyzer::FOnEventContext& InContext)							\
			: FSoundParameterMessage(InContext)													\
		{																						\
			const Trace::IAnalyzer::FEventData& EventData = InContext.EventData;				\
			ParamName = EventData.GetValue<Type>(#ParamName);									\
		}																						\
																								\
		virtual uint32 GetSizeOf() const override { return sizeof(ClassName); }					\
		virtual const FName GetMessageName() const override { return MessageName; }				\
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;						\
																								\
		Type ParamName = Default;																\
	};

	DEFINE_SOUND_PARAM_MESSAGE(FSoundPriorityMessage, SoundMessageNames::PriorityParam, Priority, float, 0.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundDistanceMessage, SoundMessageNames::DistanceParam, Distance, float, 0.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundDistanceAttenuationMessage, SoundMessageNames::DistanceAttenuationParam, DistanceAttenuation, float, 1.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundHPFFreqMessage, SoundMessageNames::HPFFreqParam, HPFFrequency, float, MIN_FILTER_FREQUENCY)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundLPFFreqMessage, SoundMessageNames::LPFFreqParam, LPFFrequency, float, MAX_FILTER_FREQUENCY)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundEnvelopeMessage, SoundMessageNames::EnvelopeParam, Envelope, float, 0.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundPitchMessage, SoundMessageNames::PitchParam, Pitch, float, 1.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundVolumeMessage, SoundMessageNames::VolumeParam, Volume, float, 1.0f)
	DEFINE_SOUND_PARAM_MESSAGE(FSoundRelativeRenderCostMessage, SoundMessageNames::RelativeRenderCostParam, RelativeRenderCost, float, 0.0f)
#undef DEFINE_SOUND_PARAM_MESSAGE

	// Trace message queues
	class FSoundMessages
	{
	public:
		TAnalyzerMessageQueue<FSoundStartMessage> ActiveSoundStartMessages;
		TAnalyzerMessageQueue<FSoundWaveStartMessage> SoundWaveStartMessages;
		TAnalyzerMessageQueue<FSoundIsAlivePingMessage> ActiveSoundIsAlivePingMessages;
		TAnalyzerMessageQueue<FSoundWaveIsAlivePingMessage> SoundWaveIsAlivePingMessages;
		TAnalyzerMessageQueue<FSoundPriorityMessage> PriorityMessages;
		TAnalyzerMessageQueue<FSoundDistanceMessage> DistanceMessages;
		TAnalyzerMessageQueue<FSoundDistanceAttenuationMessage> DistanceAttenuationMessages;
		TAnalyzerMessageQueue<FSoundHPFFreqMessage> HPFFreqMessages;
		TAnalyzerMessageQueue<FSoundLPFFreqMessage> LPFFreqMessages;
		TAnalyzerMessageQueue<FSoundEnvelopeMessage> AmplitudeMessages;
		TAnalyzerMessageQueue<FSoundVolumeMessage> VolumeMessages;
		TAnalyzerMessageQueue<FSoundPitchMessage> PitchMessages;
		TAnalyzerMessageQueue<FSoundRelativeRenderCostMessage> RelativeRenderCostMessages;
		TAnalyzerMessageQueue<FSoundStopMessage> StopMessages;
	};

	// Dashboard entry
	using FDataPoint = TPair<double, float>; // (Timestamp, Value)

	class FSoundDashboardEntry : public IObjectTreeDashboardEntry
	{
	public:
		FSoundDashboardEntry();
		virtual ~FSoundDashboardEntry() = default;

		virtual TObjectPtr<UObject> GetObject() override { return FSoftObjectPath(FullName).ResolveObject(); }
		virtual const TObjectPtr<UObject> GetObject() const override { return FSoftObjectPath(FullName).ResolveObject(); }

		void SetName(const FString& InName)
		{
			FullName = InName;

			const FSoftObjectPath AssetPath(InName);

			DisplayNameStr  = AssetPath.IsValid() ? AssetPath.GetAssetName() : InName;
			DisplayName     = AssetPath.IsValid() ? AssetPath.GetAssetFName() : FName(InName);
			DisplayNameText = FText::FromString(AssetPath.IsValid() ? DisplayNameStr : InName);
		}

		virtual const FText& GetDisplayName() const override
		{
			return DisplayNameText;
		}

		const FName& GetDisplayFName() const
		{
			return DisplayName;
		}

		const FString& GetDisplayNameStr() const
		{
			return DisplayNameStr;
		}

		virtual const FLinearColor& GetEntryColor() const override
		{
			return bIsCategory ? FLinearColor::White : EntryColor;
		}

		virtual void SetEntryColor(const FLinearColor& Color) override
		{
			EntryColor = Color;
		}

		virtual bool IsValid() const override { return ActiveSoundPlayOrder != INDEX_NONE; }
		virtual uint64 GetEntryID() const override { return SoundMessageUtils::GeneratePlayOrderUniqueID(ActiveSoundPlayOrder, WaveInstancePlayOrder); }

		virtual bool HasSetInitExpansion() const override { return bHasSetInitExpansion; }
		virtual void ResetHasSetInitExpansion() override { bHasSetInitExpansion = true; }

		void ResetDataBuffers(const uint32 DataPointsCapacity);

		enum class EPinnedEntryType : uint8
		{
			None = 0,
			PinnedCopy,
			HiddenOriginalEntry
		};

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 WaveInstancePlayOrder = INDEX_NONE;
		uint32 ActiveSoundPlayOrder = INDEX_NONE;
		double Timestamp = 0.0;
		double TimeoutTimestamp = INVALID_TIMEOUT;
		FText CategoryName;
		FLinearColor EntryColor = FColor::MakeRandomColor();
		ESoundDashboardEntryType EntryType = ESoundDashboardEntryType::None;
		EPinnedEntryType PinnedEntryType = EPinnedEntryType::None;
		bool bIsCategory = false;
		bool bHasSetInitExpansion = false;
		bool bIsVisible = false;
		bool bIsPlotActive = false;
		bool bForceKeepEntryAlive = false;

		::Audio::TCircularAudioBuffer<FDataPoint> PriorityDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> DistanceDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> DistanceAttenuationDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> LPFFreqDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> HPFFreqDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> AmplitudeDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> VolumeDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> PitchDataRange;
		::Audio::TCircularAudioBuffer<FDataPoint> RelativeRenderCostDataRange;

		float PriorityDataPoint = 0.0f;
		float DistanceDataPoint = 0.0f;
		float DistanceAttenuationDataPoint = 0.0f;
		float LPFFreqDataPoint = 20000.0f;
		float HPFFreqDataPoint = 0.0f;
		float AmplitudeDataPoint = MIN_VOLUME_DECIBELS;
		float VolumeDataPoint = 0.0f;
		float PitchDataPoint = 1.0f;
		float RelativeRenderCostDataPoint = 0.0f;

		FText ActorLabel;
		FText AudioComponentName;

	private:
		FString FullName;
		FName DisplayName;
		FString DisplayNameStr;
		FText DisplayNameText;
	};
} // namespace UE::Audio::Insights
