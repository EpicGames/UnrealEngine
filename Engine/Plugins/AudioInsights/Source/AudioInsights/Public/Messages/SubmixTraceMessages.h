// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Cache/IAudioCachedMessage.h"
#include "Providers/AudioMeterProvider.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Trace/Analyzer.h"
#include "Views/TableDashboardViewFactory.h"

#if !WITH_EDITOR
#include "Common/PagedArray.h"
#include "TraceServices/Model/AnalysisSession.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace SubmixMessageNames
	{
		extern const FName Loaded;
		extern const FName IsAlivePing;
		extern const FName HasActivity;
		extern const FName EnvelopeFollowerEnabled;
		extern const FName EnvelopeValues;
		extern const FName Unloaded;
	};

	struct FSubmixMessageBase : public IAudioCachedMessage
	{
		FSubmixMessageBase() = default;
		FSubmixMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return SubmixId; };
		virtual uint32 GetSizeOf() const override { return sizeof(FSubmixMessageBase); };

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 SubmixId = INDEX_NONE;
	};

	struct FSubmixLoadedMessage : public FSubmixMessageBase
	{
		FSubmixLoadedMessage() = default;
		FSubmixLoadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SubmixMessageNames::Loaded; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Name;
		int32 NumChannels = 0;
		bool bIsMainSubmix = false;
	};

	struct FSubmixAlivePingMessage : public FSubmixLoadedMessage
	{
		FSubmixAlivePingMessage() = default;
		FSubmixAlivePingMessage(const Trace::IAnalyzer::FOnEventContext& InContext)
			: FSubmixLoadedMessage(InContext)
		{
		}

		virtual const FName GetMessageName() const override { return SubmixMessageNames::IsAlivePing; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FSubmixHasActivityMessage : public FSubmixMessageBase
	{
		FSubmixHasActivityMessage() = default;
		FSubmixHasActivityMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FSubmixHasActivityMessage); };
		virtual const FName GetMessageName() const override { return SubmixMessageNames::HasActivity; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		bool bHasActivity = false;
	};

	struct FSubmixEnvelopeFollowerEnabledMessage : public FSubmixMessageBase
	{
		FSubmixEnvelopeFollowerEnabledMessage() = default;
		FSubmixEnvelopeFollowerEnabledMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FSubmixEnvelopeFollowerEnabledMessage); };
		virtual const FName GetMessageName() const override { return SubmixMessageNames::EnvelopeFollowerEnabled; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		bool bEnvelopeFollowerEnabled = false;
	};

	struct FSubmixEnvelopeValuesMessage : public FSubmixMessageBase
	{
		FSubmixEnvelopeValuesMessage() = default;
		FSubmixEnvelopeValuesMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override;
		virtual const FName GetMessageName() const override { return SubmixMessageNames::EnvelopeValues; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		TArray<float> EnvelopeValues;
	};

	struct FSubmixUnloadedMessage : public FSubmixMessageBase
	{
		FSubmixUnloadedMessage() = default;
		FSubmixUnloadedMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint32 GetSizeOf() const override { return sizeof(FSubmixUnloadedMessage); };
		virtual const FName GetMessageName() const override { return SubmixMessageNames::Unloaded; }
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	class FSubmixMessages
	{
		TAnalyzerMessageQueue<FSubmixLoadedMessage> LoadedMessages;
		TAnalyzerMessageQueue<FSubmixAlivePingMessage> AlivePingMessages;
		TAnalyzerMessageQueue<FSubmixHasActivityMessage> HasActivityMessages;
		TAnalyzerMessageQueue<FSubmixEnvelopeFollowerEnabledMessage> EnvelopeFollowerEnabledMessages;
		TAnalyzerMessageQueue<FSubmixEnvelopeValuesMessage> EnvelopeValuesMessages;
		TAnalyzerMessageQueue<FSubmixUnloadedMessage> UnloadedMessages;

		friend class FSubmixTraceProvider;
	};

	struct FSubmixDashboardEntry : public IObjectDashboardEntry
	{
		FSubmixDashboardEntry();
		FSubmixDashboardEntry(FSubmixDashboardEntry& Other); // Copy constructor - make sure we take a deep copy of FAudioMeterInfo

		virtual ~FSubmixDashboardEntry() = default;

		FSubmixDashboardEntry& operator=(const FSubmixDashboardEntry& Other);

		virtual FText GetDisplayName() const override { return FText::FromString(FSoftObjectPath(Name).GetAssetName()); }
		virtual FString GetObjectPath() const override { return Name; }
		virtual const UObject* GetObject() const override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual UObject* GetObject() override { return FSoftObjectPath(Name).ResolveObject(); }
		virtual bool IsValid() const override { return SubmixId != static_cast<uint32>(INDEX_NONE); }

		bool IsMainSubmix() const { return bIsMainSubmix; }

		uint32 SubmixId = INDEX_NONE;
		double Timestamp = 0.0;

		TSharedRef<FAudioMeterInfo> AudioMeterInfo;

		FString Name;
		bool bHasActivity = false;
		bool bEnvelopeFollowerEnabled = false;
		bool bIsMainSubmix = false;
	};
} // namespace UE::Audio::Insights
