// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Cache/IAudioCachedMessage.h"
#include "Math/NumericLimits.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Trace/Analyzer.h"
#include "Views/TableDashboardViewFactory.h"

#if !WITH_EDITOR
#include "Common/PagedArray.h"
#include "TraceServices/Model/AnalysisSession.h"
#endif // !WITH_EDITOR

namespace UE::Audio::Insights
{
	namespace VirtualLoopMessageNames
	{
		extern const FName Virtualize;
		extern const FName StopOrRealize;
		extern const FName Update;
	};

	struct FVirtualLoopMessageBase : public IAudioCachedMessage
	{
		FVirtualLoopMessageBase() = default;
		FVirtualLoopMessageBase(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return PlayOrder; }

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 PlayOrder = INDEX_NONE;
	};

	struct FVirtualLoopVirtualizeMessage : public FVirtualLoopMessageBase
	{
		FVirtualLoopVirtualizeMessage() = default;
		FVirtualLoopVirtualizeMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return VirtualLoopMessageNames::Virtualize; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Name;
		uint64 ComponentId = TNumericLimits<uint64>::Max();
	};

	struct FVirtualLoopStopOrRealizeMessage : public FVirtualLoopMessageBase
	{
		FVirtualLoopStopOrRealizeMessage() = default;
		FVirtualLoopStopOrRealizeMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return VirtualLoopMessageNames::StopOrRealize; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	struct FVirtualLoopUpdateMessage : public FVirtualLoopMessageBase
	{
		FVirtualLoopUpdateMessage() = default;
		FVirtualLoopUpdateMessage(const Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return VirtualLoopMessageNames::Update; }
		virtual uint32 GetSizeOf() const override;
		virtual FCacheWriteHandler GetCacheWriteHandler() const override;

		float TimeVirtualized = 0.0f;
		float PlaybackTime = 0.0f;
		float UpdateInterval = 0.0f;

		double LocationX = 0.0;
		double LocationY = 0.0;
		double LocationZ = 0.0;

		double RotatorPitch = 0.0;
		double RotatorYaw = 0.0;
		double RotatorRoll = 0.0;
	};

	class FVirtualLoopDashboardEntry : public FSoundAssetDashboardEntry
	{
	public:
		FVirtualLoopDashboardEntry() = default;
		virtual ~FVirtualLoopDashboardEntry() = default;

		virtual bool IsValid() const override
		{
			return PlayOrder != static_cast<uint32>(INDEX_NONE);
		}

		uint32 PlayOrder = static_cast<uint32>(INDEX_NONE);
		uint64 ComponentId = TNumericLimits<uint64>::Max();

		float TimeVirtualized = 0.0f;
		float PlaybackTime = 0.0f;
		float UpdateInterval = 0.0f;

		FVector Location = FVector::ZeroVector;
		FRotator Rotator = FRotator::ZeroRotator;
	};

	class FVirtualLoopMessages
	{
		TAnalyzerMessageQueue<FVirtualLoopVirtualizeMessage> VirtualizeMessages;
		TAnalyzerMessageQueue<FVirtualLoopStopOrRealizeMessage> StopOrRealizeMessages;
		TAnalyzerMessageQueue<FVirtualLoopUpdateMessage> UpdateMessages;

		friend class FVirtualLoopTraceProvider;
	};
} // namespace UE::Audio::Insights
