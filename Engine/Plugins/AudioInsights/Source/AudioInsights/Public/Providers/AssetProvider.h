// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "Delegates/Delegate.h"
#include "IAudioInsightsModule.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	template <typename TAsset>
	class TAssetProvider
	{
	public:
		TAssetProvider()
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

			AssetRegistryModule.Get().OnAssetAdded().AddRaw(this, &TAssetProvider::HandleOnAssetAdded);
			AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &TAssetProvider::HandleOnAssetRemoved);
			AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &TAssetProvider::HandleOnFilesLoaded);

			FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddRaw(this, &TAssetProvider::HandleOnActiveAudioDeviceCreated);
			FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &TAssetProvider::HandleOnActiveAudioDeviceDestroyed);

			FTraceAuxiliary::OnTraceStarted.AddRaw(this, &TAssetProvider::HandleOnTraceStarted);
		}

		virtual ~TAssetProvider()
		{
			if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
			{
				AssetRegistryModule->Get().OnAssetAdded().RemoveAll(this);
				AssetRegistryModule->Get().OnAssetRemoved().RemoveAll(this);
				AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(this);
			}

			FAudioDeviceManagerDelegates::OnAudioDeviceCreated.RemoveAll(this);
			FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.RemoveAll(this);

			FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
		}

		void RequestEntriesUpdate()
		{
			UpdateAssetNames();
		}

		DECLARE_DELEGATE_OneParam(FOnAssetAdded, const FString& /*AssetPath*/);
		FOnAssetAdded OnAssetAdded;

		DECLARE_DELEGATE_OneParam(FOnAssetRemoved, const FString& /*AssetPath*/);
		FOnAssetRemoved OnAssetRemoved;

		DECLARE_DELEGATE_OneParam(FOnAssetListUpdated, const TArray<FString>& /*AssetPaths*/);
		FOnAssetListUpdated OnAssetListUpdated;

	private:
		void UpdateAssetNames()
		{
			// Get all assets
			TArray<FAssetData> AssetDataArray;

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			AssetRegistryModule.Get().GetAssetsByClass(FTopLevelAssetPath(TAsset::StaticClass()), AssetDataArray);

			// Collect asset paths
			AssetPaths.Reset();

			constexpr bool bShouldNotify = false;
			for (const FAssetData& AssetData : AssetDataArray)
			{
				AddAssetPath(AssetData.GetObjectPathString(), bShouldNotify);
			}

			AssetPaths.Sort([](const FString& A, const FString& B)
			{
				return A.Compare(B, ESearchCase::IgnoreCase) < 0;
			});

			OnAssetListUpdated.ExecuteIfBound(AssetPaths);
		}

		void AddAssetPath(const FString& InAssetPath, const bool bInShouldNotify)
		{
			const bool bIsAssetPathAlreadAdded = AssetPaths.ContainsByPredicate(
				[&InAssetPath](const FString& AssetPath)
				{
					return AssetPath == InAssetPath;
				});

			if (!bIsAssetPathAlreadAdded)
			{
				AssetPaths.Emplace(InAssetPath);

				if (bInShouldNotify)
				{
					OnAssetAdded.ExecuteIfBound(InAssetPath);
				}
			}
		}

		void RemoveAssetPath(const FString& InAssetPath)
		{
			const int32 NumRemovedItems = AssetPaths.Remove(InAssetPath);
			if (NumRemovedItems > 0)
			{
				OnAssetRemoved.ExecuteIfBound(InAssetPath);
			}
		}

		void HandleOnAssetAdded(const FAssetData& InAssetData)
		{
			if (bAreFilesLoaded && InAssetData.AssetClassPath == FTopLevelAssetPath(TAsset::StaticClass()))
			{
				constexpr bool bShouldNotify = true;
				AddAssetPath(InAssetData.GetObjectPathString(), bShouldNotify);
			}
		}

		void HandleOnAssetRemoved(const FAssetData& InAssetData)
		{
			if (InAssetData.AssetClassPath == FTopLevelAssetPath(TAsset::StaticClass()))
			{
				RemoveAssetPath(InAssetData.GetObjectPathString());
			}
		}

		void HandleOnFilesLoaded()
		{
			bAreFilesLoaded = true;
			UpdateAssetNames();
		}

		void HandleOnActiveAudioDeviceCreated(::Audio::FDeviceId InDeviceId)
		{
			UpdateAssetNames();
		}

		void HandleOnActiveAudioDeviceDestroyed(::Audio::FDeviceId InDeviceId)
		{
			if (GIsRunning)
			{
				UpdateAssetNames();
			}
		}

		void HandleOnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
		{
			UpdateAssetNames();
		}

		bool bAreFilesLoaded = false;

		TArray<FString> AssetPaths;
	};
} // namespace UE::Audio::Insights

#undef UE_API

#endif // WITH_EDITOR
