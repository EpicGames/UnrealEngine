// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Engine/DeveloperSettings.h"
#include "Settings/VisibleColumnsSettings.h"

#include "AudioEventLogSettings.generated.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

/** FAudioEventLogCustomEvents
 *
 * Custom event types that can be filtered by in the Event Log
 */
USTRUCT()
struct FAudioEventLogCustomEvents
{
	GENERATED_BODY()

	/** 
	* The names of any custom events that can be sent to the Event Log.
	* This should match the event name used to send the event to the Event Log.
	*/
	UPROPERTY(EditAnywhere, config, Category = EventLog, meta = (DisplayName = "Custom event names"))
	TSet<FString> EventNames;
};

/** FAudioEventLogVisibleColumns
 *
 * Control which columns are visible in the Audio Event Log tab.
 */
USTRUCT()
struct FAudioEventLogVisibleColumns : public FVisibleColumnsSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bCacheStatus = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bMessageID = false;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bTimestamp = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bPlayOrder = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bEvent = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bAsset = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bActor = true;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bAudioComponent = false;

	UPROPERTY(EditAnywhere, config, Category = EventLog)
	bool bCategory = true;

	virtual const FProperty* FindProperty(const FName& PropertyName) const override
	{
		return StaticStruct()->FindPropertyByName(PropertyName);
	}
};

USTRUCT()
struct FAudioEventLogSettings
{
	GENERATED_BODY()

	/** Which event types are being filtered by in the Event Log - hidden in the Editor Preferences */
	UPROPERTY(config, meta = (HideInDetailPanel))
	TSet<FString> EventFilters;

	/** 
	* Categories for custom events being sent to the Audio Insights Event Log.
	* These categories and events will appear in the event filter menu.
	*/
	UPROPERTY(EditAnywhere, config, Category = EventLog, meta = (DisplayName = "Custom Event Log categories"))
	TMap<FString, FAudioEventLogCustomEvents> CustomCategoriesToEvents;

	/** Visibility of columns in the Event Log tab */
	UPROPERTY(EditAnywhere, config, Category = EventLog)
	FAudioEventLogVisibleColumns VisibleColumns;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReadEventLogSettings, const FAudioEventLogSettings&);
	static AUDIOINSIGHTS_API FOnReadEventLogSettings OnReadSettings;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWriteEventLogSettings, FAudioEventLogSettings&);
	static AUDIOINSIGHTS_API FOnWriteEventLogSettings OnWriteSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestReadEventLogSettings);
	static AUDIOINSIGHTS_API FOnRequestReadEventLogSettings OnRequestReadSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestWriteEventLogSettings);
	static AUDIOINSIGHTS_API FOnRequestWriteEventLogSettings OnRequestWriteSettings;
#endif // WITH_EDITOR
};

#undef LOCTEXT_NAMESPACE