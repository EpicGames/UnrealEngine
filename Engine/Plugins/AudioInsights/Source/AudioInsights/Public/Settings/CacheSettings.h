// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/DelegateCombinations.h"

#include "CacheSettings.generated.h"

#define UE_API AUDIOINSIGHTS_API

/** EStopCacheWhenPausedBehaviour
 *
 * Controls whether the cache stops collecting new data when paused.
 */
UENUM()
enum class EStopCacheWhenPausedBehaviour : uint8
{
	/** Stops caching new data when the cache playhead is paused in the cache region marked for deletion. */
	WhenMarkedForDeletion			UMETA(DisplayName = "When marked for deletion"),

	/** Always stops caching new data when paused. */
	Always							UMETA(DisplayName = "Always"),

	/** Never stops caching new data. When the cache playhead is paused in the cache region being deleted, the cache will resume. */
	Never							UMETA(DisplayName = "Never")
};

USTRUCT()
struct FCacheSettings
{
	GENERATED_BODY()

	static constexpr uint32 MinCacheSizeMB = 8u;
	static constexpr uint32 MaxCacheSizeMB = 512u;
	static constexpr uint32 DefaultCacheSizeMB = 32u;

	static constexpr float MinNudgeStepSeconds = 0.1f;
	static constexpr float MaxNudgeStepSeconds = 5.0f;
	static constexpr float DefaultNudgeStepSeconds = 0.5f;

	static constexpr EStopCacheWhenPausedBehaviour DefaultStopCacheWhenPausedBehaviour = EStopCacheWhenPausedBehaviour::WhenMarkedForDeletion;

	/** Whether to show cache stats in the Audio Insights toolbar. */
	UPROPERTY(EditAnywhere, config, Category = Cache)
	bool bShowCacheStats = true;

	/** When the time marker is paused, controls whether the cache also stops collecting new data. */
	UPROPERTY(EditAnywhere, config, Category = Cache, meta = (
		DisplayName = "Stop Cache When Paused",
		ToolTip = "Controls whether the cache stops collecting new data when paused."))
	EStopCacheWhenPausedBehaviour StopCacheWhenPausedBehaviour = DefaultStopCacheWhenPausedBehaviour;

	/** Maximum cache size. Changing this value will reset the current cache. */
	UPROPERTY(EditAnywhere, config, Category = Cache, meta = (
		DisplayName = "Cache Size",
		ToolTip = "Maximum cache size in megabytes (default: 32). Changing this value will reset the current cache.",
		ClampMin = "8", UIMin = "8",
		ClampMax = "512", UIMax = "512",
		Units = "Megabytes"))
	uint32 CacheSizeMB = DefaultCacheSizeMB;

	/** Amount the cache playhead moves when nudged back or forward. */
	UPROPERTY(EditAnywhere, config, Category = Cache, meta = (
		DisplayName = "Cache Nudge Step",
		ToolTip = "Amount in seconds the cache playhead moves when nudged back or forward (default: 0.5).",
		ClampMin = "0.1", UIMin = "0.1",
		ClampMax = "5.0", UIMax = "5.0",
		Units = "Seconds"))
	float NudgeStepSeconds = DefaultNudgeStepSeconds;

	DECLARE_MULTICAST_DELEGATE(FOnCacheSizeSettingsChanged);
	static UE_API FOnCacheSizeSettingsChanged OnCacheSizeSettingsChanged;
};

#undef UE_API
