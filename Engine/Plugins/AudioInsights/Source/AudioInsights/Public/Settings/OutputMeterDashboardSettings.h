// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OutputMeterDashboardSettings.generated.h"

USTRUCT()
struct FOutputMeterDashboardSettings
{
	GENERATED_BODY()

	UPROPERTY(config)
	float LoudnessMetersSlotSize = 0.75f;

	UPROPERTY(config)
	float RMSMeterSlotSize = 0.25f;

	UPROPERTY(config)
	bool bLoudnessMetersCollapsed = false;

	UPROPERTY(config)
	bool bRMSMeterCollapsed = false;
};
