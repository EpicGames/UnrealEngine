// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Engine/DeveloperSettings.h"
#include "Messages/SignalFlowEntryKey.h"
#include "Settings/SoundDashboardSettings.h"
#include "Styling/SlateTypes.h"

#include "SignalFlowSettings.generated.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

/** ESignalFlowJustification
 *
 * Where the nodes in the signal flow graph are aligned
 */
UENUM(BlueprintType)
enum class ESignalFlowJustification : uint8
{
	/** Position nodes towards one edge of the graph */
	Edge		UMETA(DisplayName = "Edge"),

	/** Position nodes around the center of the graph */
	Center		UMETA(DisplayName = "Center")
};

USTRUCT()
struct FSignalFlowNodeDetailFilterSettings
{
	GENERATED_BODY()

	bool GetParameterIsVisible(const UE::Audio::Insights::ESignalFlowNodeDetailParam Param) const;
	int32 GetNumVisibleParams(const TSet<UE::Audio::Insights::ESignalFlowNodeDetailParam>& IgnoredParams = TSet<UE::Audio::Insights::ESignalFlowNodeDetailParam>()) const;
	ECheckBoxState GetShowAllNodeDetailFiltersCheckboxState() const;

	void ToggleEnableAllNodeDetailFilters();
	void SetParameterVisibility(const UE::Audio::Insights::ESignalFlowNodeDetailParam Param, const bool bIsVisible);

	/** Toggles visibility of amplitude parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bAmplitude = true;

	/** Toggles visibility of volume parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bVolume = true;

	/** Toggles visibility of pitch parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bPitch = false;

	/** Toggles visibility of low pass filter frequency parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bLPFFreq = true;

	/** Toggles visibility of high pass filter frequency parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bHPFFreq = true;

	/** Toggles visibility of priority parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bPriority = false;

	/** Toggles visibility of distance parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bDistance = false;

	/** Toggles visibility of distance/occlusion attenuation parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bDistanceOcclusionAttenuation = false;

	/** Toggles visibility of relative render cost parameters. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bRelativeRenderCost = false;

	/** Toggles visibility of audio component name on sound source nodes. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bAudioComponentName = false;

	/** Toggles visibility of send output volume parameters on node output pins. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bSendOutputVolume = true;
};

USTRUCT()
struct FSignalFlowEntryTypeListExpansionSettings
{
	GENERATED_BODY()

	/** Toggles expansion of the left hand Active Owner Objects menu. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bShowActiveOwnerObjects = false;

	/** Toggles expansion of the left hand Active Sources menu. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bShowActiveSources = true;

	/** Toggles expansion of the left hand Active Buses menu. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bShowActiveBuses = true;

	/** Toggles expansion of the left hand Active Submixes menu. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bShowActiveSubmixes = true;

	// Internal settings: persisted height ratios for the category splitter slots
	UPROPERTY(config)
	float OwnerObjectsSlotSize = 1.0f;

	UPROPERTY(config)
	float SourcesSlotSize = 2.5f;

	UPROPERTY(config)
	float BusesSlotSize = 1.0f;

	UPROPERTY(config)
	float SubmixesSlotSize = 1.5f;
};

USTRUCT()
struct FSignalFlowSettings
{
	GENERATED_BODY()

	/** Toggles between decibel/linear scales when displaying amplitude (peak) values inside the signal flow graph. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (DisplayName = "Amp (Peak) Display Mode"))
	EAudioAmplitudeDisplayMode AmplitudeDisplayMode = EAudioAmplitudeDisplayMode::Decibels;

	/** Toggles between horizontally/vertically orienting the Signal Flow graph. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bHorizontalFlow = true;

	/** Toggles pausing the graph at the current timestamp when selecting a node. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bPauseGraphOnSelect = false;

	/** Where the nodes in the signal flow graph are aligned */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (DisplayName = "Graph Justification"))
	ESignalFlowJustification GraphJustification = ESignalFlowJustification::Edge;

	/** Toggles visibility of node details. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bShowNodeDetails = false;

	/** Which parameter types are visible in the Signal Flow graph when Node Details is enabled */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	FSignalFlowNodeDetailFilterSettings NodeDetailFilters;

	/** Expansion settings for the Active Entry menus on the left hand side of the Signal Flow Tab */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (DisplayName = "Active Entry Menu Expansion"))
	FSignalFlowEntryTypeListExpansionSettings ActiveEntryMenuExpansionSettings;

	/** Toggles whether connection wire animations driven by amplitude envelope values are enabled in the Signal Flow graph. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow)
	bool bAnimateWires = true;

	/** Power factor applied to amplitude values when drawing animated signal flow connection wires. Controls how much additional weight is applied to quieter nodes. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (UIMin = "0.1", UIMax = "1.0", ClampMin = "0.1", ClampMax = "1.0"))
	float WireSplineAmplitudePowerFactor = 0.4f;

	/** The maximum thickness multiplier applied to animated signal flow connection wires when a node is at high amplitudes. */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (UIMin = "1.0", UIMax = "40.0", ClampMin = "1.0", ClampMax = "40.0"))
	float WireSplineMaxThicknessScalar = 14.0f;

	/** Ratio of the horizontal space allocated to the Signal Flow graph (vs. the selection panel on the left). */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (UIMin = "0.25", UIMax = "1.0", ClampMin = "0.25", ClampMax = "1.0"))
	float GraphWidthRatio = 0.8f;

	/** Padding applied between nodes along the flow direction of the Signal Flow graph. */
	UPROPERTY(config, meta = (ClampMin = "0.25", ClampMax = "500.0"))
	float LargeNodePadding = 128.0f;

	/** Padding applied between nodes perpendicular to the flow direction of the Signal Flow graph. */
	UPROPERTY(config, meta = (ClampMin = "0.25", ClampMax = "500.0"))
	float SmallNodePadding = 8.0f;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReadSignalFlowSettings, const FSignalFlowSettings&);
	static AUDIOINSIGHTS_API FOnReadSignalFlowSettings OnReadSettings;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWriteSignalFlowSettings, FSignalFlowSettings&);
	static AUDIOINSIGHTS_API FOnWriteSignalFlowSettings OnWriteSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestReadSignalFlowSettings);
	static AUDIOINSIGHTS_API FOnRequestReadSignalFlowSettings OnRequestReadSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestWriteSignalFlowSettings);
	static AUDIOINSIGHTS_API FOnRequestWriteSignalFlowSettings OnRequestWriteSettings;
#endif // WITH_EDITOR
};

#undef LOCTEXT_NAMESPACE