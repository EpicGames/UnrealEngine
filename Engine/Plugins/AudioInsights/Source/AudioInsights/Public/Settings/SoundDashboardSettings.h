// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "Delegates/DelegateCombinations.h"
#include "Engine/DeveloperSettings.h"
#include "Settings/VisibleColumnsSettingsMenu.h"

#include "SoundDashboardSettings.generated.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

/** EAudioAmplitudeDisplayMode
 *
 * The units that audio amplitude and volume values are displayed in
 */
UENUM(BlueprintType)
enum class EAudioAmplitudeDisplayMode : uint8
{
	/** Displays amplitude values in decibels. */
	Decibels		UMETA(DisplayName = "dB"),

	/** Displays amplitude values in linear scale. */
	Linear			UMETA(DisplayName = "Linear")
};

/** ESoundDashboardTreeViewingOptions
*
* Change how sounds are organized within the Sound Dashboard tab in Audio Insights.
*/
UENUM(BlueprintType)
enum class ESoundDashboardTreeViewingOptions : uint8
{
	/** Organize sounds into categories. */
	FullTree		UMETA(DisplayName = "Tree View"),

	/** Organize sounds into Active Sounds. */
	ActiveSounds	UMETA(DisplayName = "Active Sounds"),

	/** Display sounds as individual waves in a flat list. */
	FlatList		UMETA(DisplayName = "Flat List")
};

/** ESoundDashboardAutoExpandOptions
 *
 * Control whether new sounds entering the Sound Dashboard are auto-expanded/collapsed by default.
 */
UENUM(BlueprintType)
enum class ESoundDashboardAutoExpandOptions : uint8
{
	/** Auto-expand new categories. */
	Categories		UMETA(DisplayName = "Categories"),

	/** Auto-expand all new categories and sounds. */
	Everything		UMETA(DisplayName = "Everything"),

	/** Don't auto-expand anything. */
	Nothing			UMETA(DisplayName = "Nothing")
};

USTRUCT()
struct FSoundDashboardVisibleColumns : public FVisibleColumnsSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bMute = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bSolo = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bPlot = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bName = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bPlayOrder = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bPriority = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bDistance = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bDistanceAttenuation = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "Amp (Peak)"))
	bool bAmplitude = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bVolume = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "LPF Freq"))
	bool bLPFFreq = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "HPF Freq"))
	bool bHPFFreq = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bPitch = true;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bRelativeRenderCost = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bActorLabel = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bAudioComponentName = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	bool bCategory = false;

	virtual const FProperty* FindProperty(const FName& PropertyName) const override
	{
		return StaticStruct()->FindPropertyByName(PropertyName);
	}
};

USTRUCT()
struct FSoundPlotsDashboardPlotRanges
{
	GENERATED_BODY()


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomAmplitude_dBRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomAmplitude_dBRange", UIMin = "-160", UIMax = "0", ClampMin = "-160", ClampMax = "24", DisplayName = "Amplitude (dB)"))
	FFloatInterval Amplitude_dB = { MIN_VOLUME_DECIBELS, 0.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomAmplitude_LinearRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomAmplitude_LinearRange", UIMin = "0", UIMax = "1", ClampMin = "-16", ClampMax = "16", DisplayName = "Amplitude (Linear)"))
	FFloatInterval Amplitude_Linear = { 0.0f, 1.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomVolumeRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomVolumeRange", UIMin = "0", UIMax = "4", ClampMin = "0", ClampMax = "4"))
	FFloatInterval Volume = { 0.0f, 1.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomDistanceRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomDistanceRange", UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "100000"))
	FFloatInterval Distance = { 0.0f, 10000.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomDistanceAttenuationRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomDistanceAttenuationRange", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FFloatInterval DistanceAttenuation = { 0.0f, 1.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomPitchRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomPitchRange", UIMin = "0.125", UIMax = "4", ClampMin = "0", ClampMax = "32", Units = "Multiplier"))
	FFloatInterval Pitch = { 0.125f, 4.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomPriorityRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomPriorityRange", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "100"))
	FFloatInterval Priority = { 0.0f, 100.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomLPFFreqRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomLPFFreqRange", UIMin = "20", UIMax = "20000", DisplayName = "LPF Freq", ClampMin = "0", ClampMax = "48000", Units = "Hertz"))
	FFloatInterval LPFFreq = { 20.0f, 20000.0f };


	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (InlineEditConditionToggle))
	bool bUseCustomHPFFreqRange = false;

	UPROPERTY(EditAnywhere, config, Category = "SoundDashboard|CustomPlotRanges", meta = (EditCondition = "bUseCustomHPFFreqRange", UIMin = "20", UIMax = "20000", DisplayName = "HPF Freq", ClampMin = "0", ClampMax = "48000", Units = "Hertz"))
	FFloatInterval HPFFreq = { 20.0f, 20000.0f };
};

USTRUCT()
struct FSoundDashboardSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "Amp (Peak) Display Mode"))
	EAudioAmplitudeDisplayMode AmplitudeDisplayMode = EAudioAmplitudeDisplayMode::Decibels;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "View"))
	ESoundDashboardTreeViewingOptions TreeViewMode = ESoundDashboardTreeViewingOptions::FullTree;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "Auto-Expand"))
	ESoundDashboardAutoExpandOptions AutoExpandMode = ESoundDashboardAutoExpandOptions::Categories;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (DisplayName = "Show Stopped Sounds"))
	bool bShowStoppedSounds = false;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (ClampMin = "0", DisplayName = "Recently Stopped Sounds Timeout"))
	float StoppedSoundTimeoutTime = 3.0f;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard)
	FSoundDashboardVisibleColumns VisibleColumns;

	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (ShowOnlyInnerProperties))
	FSoundPlotsDashboardPlotRanges DefaultPlotRanges;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReadSoundDashboardSettings, const FSoundDashboardSettings&);
	static AUDIOINSIGHTS_API FOnReadSoundDashboardSettings OnReadSettings;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWriteSoundDashboardSettings, FSoundDashboardSettings&);
	static AUDIOINSIGHTS_API FOnWriteSoundDashboardSettings OnWriteSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestReadSoundDashboardSettings);
	static AUDIOINSIGHTS_API FOnRequestReadSoundDashboardSettings OnRequestReadSettings;

	DECLARE_MULTICAST_DELEGATE(FOnRequestWriteSoundDashboardSettings);
	static AUDIOINSIGHTS_API FOnRequestWriteSoundDashboardSettings OnRequestWriteSettings;
#endif
};

#undef LOCTEXT_NAMESPACE