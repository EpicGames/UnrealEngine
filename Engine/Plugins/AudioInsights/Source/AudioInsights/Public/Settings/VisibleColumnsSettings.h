// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisibleColumnsSettings.generated.h"

#define UE_API AUDIOINSIGHTS_API

/** FVisibleColumnsSettings
*
* Helper struct for adding visible column settings to an Audio Insights dashboard
* Used in conjunction with FVisibleColumnsSettingsMenu to populate a menu with enable/disable checkboxes for the columns in a table/tree
* Inherit from this in a USTRUCT that contains boolean UPROPERTIES for each column in the table, with names matching the column IDs with a 'b' prefix
* 
* e.g.
* Table : | Column01 | Column02 | Column03 |
* USTRUCT : bColumn01, bColumn02, bColumn03
*/
USTRUCT()
struct FVisibleColumnsSettings
{
	GENERATED_BODY()

	virtual ~FVisibleColumnsSettings() = default;

	UE_API virtual bool GetIsVisible(const FName& ColumnId) const;
	UE_API virtual void SetIsVisible(const FName& ColumnId, const bool bIsVisible);

	// Override in derived class to enable finding properties by name in the derived USTRUCT
	// E.g. return StaticStruct()->FindPropertyByName(PropertyName);
	virtual const FProperty* FindProperty(const FName& PropertyName) const { return nullptr; }
};

#undef UE_API
