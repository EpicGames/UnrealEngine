// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Settings/VisibleColumnsSettings.h"
#include "Styling/SlateTypes.h"

class FMenuBuilder;
class SHeaderRow;

template<typename T>
concept TIsVisibleColumnsSettingsType = std::is_base_of_v<FVisibleColumnsSettings, T>;

namespace UE::Audio::Insights
{
	/** FVisibleColumnsSettingsMenu
	*
	* Helper class for creating a settings sub-menu for the visibility of columns in a table
	* Requires a settings USTRUCT that inherits from FVisibleColumnsSettings to be passed in on construction.
	*/
	template<TIsVisibleColumnsSettingsType T>
	class FVisibleColumnsSettingsMenu : public TSharedFromThis<FVisibleColumnsSettingsMenu<T>>
	{
	public:
		/*
		* Constructor
		* @param 	InHeaderRowWidget			Header Row widget from the table (SListView, STreeView, etc.)
		* @param 	InVisibleColumnSettings		A USTRUCT that inherits from FVisibleColumnsSettings, typically to be stored in the Editor Preferences
		*/
		FVisibleColumnsSettingsMenu(TSharedRef<SHeaderRow> InHeaderRowWidget, const T& InVisibleColumnSettings) 
			: HeaderRowWidget(InHeaderRowWidget)
			, VisibleColumnSettings(InVisibleColumnSettings)
		{
		}

		virtual ~FVisibleColumnsSettingsMenu() = default;
		
		/* Pass in a FMenuBuilder object to populate a sub-menu with visible columns settings */
		void BuildVisibleColumnsMenuContent(FMenuBuilder& OutMenuBuilder);
		
		/* Call from SHeaderRow::OnHiddenColumnsListChanged callback to update the settings when the column visibility is changed from the header row  */
		void OnHiddenColumnsListChanged();

		/* Call to cache/write the current column visibility settings from/to editor preferences */
		void ReadFromSettings(const T& InVisibleColumnSettings);
		void WriteToSettings(T& OutVisibleColumnSettings);

		/* Delegate that notifies when the cached VisibleColumnSettings have been updated */
		DECLARE_MULTICAST_DELEGATE(FOnVisibleColumnsSettingsUpdated);
		FOnVisibleColumnsSettingsUpdated OnVisibleColumnsSettingsUpdated;

	private:
		void ToggleColumnVisibility(FName ColumnId);
		void ToggleAllColumnsVisibility();

		void UpdateColumnVisabilityFromSettings(const FVisibleColumnsSettings& InVisibleColumnSettings);

		ECheckBoxState GetIsColumnVisibleCheckedState(FName ColumnId) const;
		ECheckBoxState GetToggleAllColumnsVisibilityCheckState() const;
		FText GetToggleAllColumnsVisibilityText() const;

		TWeakPtr<SHeaderRow> HeaderRowWidget;
		TOptional<T> VisibleColumnSettings;
	};
}

#include "Settings/VisibleColumnsSettingsMenu.inl"