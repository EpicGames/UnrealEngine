// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AnyOf.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::Audio::Insights
{
	template<TIsVisibleColumnsSettingsType T>
	inline void FVisibleColumnsSettingsMenu<T>::BuildVisibleColumnsMenuContent(FMenuBuilder& OutMenuBuilder)
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid())
		{
			return;
		}

		OutMenuBuilder.AddMenuEntry
		(
			TAttribute<FText>::CreateSP(this, &FVisibleColumnsSettingsMenu::GetToggleAllColumnsVisibilityText),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateSP(this, &FVisibleColumnsSettingsMenu::ToggleAllColumnsVisibility),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FGetActionCheckState::CreateSP(this, &FVisibleColumnsSettingsMenu::GetToggleAllColumnsVisibilityCheckState)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		for (const SHeaderRow::FColumn& Column : HeaderRowWidgetShared->GetColumns())
		{
			OutMenuBuilder.AddMenuEntry
			(
				Column.DefaultText,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateSP(this, &FVisibleColumnsSettingsMenu::ToggleColumnVisibility, Column.ColumnId),
					FCanExecuteAction::CreateLambda([]() { return true; }),
					FGetActionCheckState::CreateSP(this, &FVisibleColumnsSettingsMenu::GetIsColumnVisibleCheckedState, Column.ColumnId)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	template<TIsVisibleColumnsSettingsType T>
	inline void FVisibleColumnsSettingsMenu<T>::OnHiddenColumnsListChanged()
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid() || !VisibleColumnSettings)
		{
			return;
		}

		// Update and save settings to reload it on the next Editor sessions
		for (const SHeaderRow::FColumn& Column : HeaderRowWidgetShared->GetColumns())
		{
			VisibleColumnSettings->SetIsVisible(Column.ColumnId, Column.bIsVisible);
		}

		OnVisibleColumnsSettingsUpdated.Broadcast();
	}

	template<TIsVisibleColumnsSettingsType T>
	void FVisibleColumnsSettingsMenu<T>::ReadFromSettings(const T& InVisibleColumnSettings)
	{
		const UScriptStruct* VisibleColumnSettingsStruct = FVisibleColumnsSettings::StaticStruct();
		const UScriptStruct* SettingsStruct = T::StaticStruct();

		if (SettingsStruct->IsChildOf(VisibleColumnSettingsStruct))
		{
			// Null our VisibleColumnSettings TOptional while updating HeaderRowWidget so that we ignore OnHiddenColumnsListChanged callbacks:
			VisibleColumnSettings.Reset();

			UpdateColumnVisabilityFromSettings(InVisibleColumnSettings);

			VisibleColumnSettings.Emplace();

			T& CastStruct = static_cast<T&>(VisibleColumnSettings.GetValue());
			CastStruct = InVisibleColumnSettings;
		}
	}

	template<TIsVisibleColumnsSettingsType T>
	void FVisibleColumnsSettingsMenu<T>::WriteToSettings(T& OutVisibleColumnSettings)
	{
		if (VisibleColumnSettings)
		{
			const UScriptStruct* VisibleColumnSettingsStruct = FVisibleColumnsSettings::StaticStruct();
			const UScriptStruct* SettingsStruct = T::StaticStruct();

			if (SettingsStruct->IsChildOf(VisibleColumnSettingsStruct))
			{
				T& CastStruct = static_cast<T&>(VisibleColumnSettings.GetValue());
				OutVisibleColumnSettings = CastStruct;
			}
		}
	}

	template<TIsVisibleColumnsSettingsType T>
	inline void FVisibleColumnsSettingsMenu<T>::ToggleColumnVisibility(FName ColumnId)
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid())
		{
			return;
		}

		const bool bIsColumnVisible = HeaderRowWidgetShared->IsColumnVisible(ColumnId);
		HeaderRowWidgetShared->SetShowGeneratedColumn(ColumnId, !bIsColumnVisible);
	}

	template<TIsVisibleColumnsSettingsType T>
	inline void FVisibleColumnsSettingsMenu<T>::ToggleAllColumnsVisibility()
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid())
		{
			return;
		}

		const ECheckBoxState CurrentState = GetToggleAllColumnsVisibilityCheckState();
		const bool bVisibleIsDesired = CurrentState == ECheckBoxState::Unchecked || CurrentState == ECheckBoxState::Undetermined;

		// Null our VisibleColumnSettings TOptional while updating HeaderRowWidget so that we ignore OnHiddenColumnsListChanged callbacks:
		VisibleColumnSettings.Reset();

		for (const SHeaderRow::FColumn& Column : HeaderRowWidgetShared->GetColumns())
		{
			if (Column.bIsVisible != bVisibleIsDesired)
			{
				HeaderRowWidgetShared->SetShowGeneratedColumn(Column.ColumnId, bVisibleIsDesired);
			}
		}

		// Reinitialize the VisibleColumnSettings and perform a single OnHiddenColumnsListChanged call to update it and persist the settings:
		VisibleColumnSettings.Emplace();
		OnHiddenColumnsListChanged();
	}

	template<TIsVisibleColumnsSettingsType T>
	inline void FVisibleColumnsSettingsMenu<T>::UpdateColumnVisabilityFromSettings(const FVisibleColumnsSettings& InVisibleColumnSettings)
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid())
		{
			return;
		}

		for (const SHeaderRow::FColumn& Column : HeaderRowWidgetShared->GetColumns())
		{
			const bool bShowColumn = InVisibleColumnSettings.GetIsVisible(Column.ColumnId);
			if (Column.bIsVisible != bShowColumn)
			{
				HeaderRowWidgetShared->SetShowGeneratedColumn(Column.ColumnId, bShowColumn);
			}
		}
	}

	template<TIsVisibleColumnsSettingsType T>
	inline ECheckBoxState FVisibleColumnsSettingsMenu<T>::GetIsColumnVisibleCheckedState(FName ColumnId) const
	{
		return HeaderRowWidget.Pin().IsValid() ? HeaderRowWidget.Pin()->IsColumnVisible(ColumnId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked
											   : ECheckBoxState::Undetermined;
	}

	template<TIsVisibleColumnsSettingsType T>
	inline ECheckBoxState FVisibleColumnsSettingsMenu<T>::GetToggleAllColumnsVisibilityCheckState() const
	{
		const TSharedPtr<SHeaderRow> HeaderRowWidgetShared = HeaderRowWidget.Pin();
		if (!HeaderRowWidgetShared.IsValid())
		{
			return ECheckBoxState::Undetermined;
		}

		const TIndirectArray<SHeaderRow::FColumn>& Columns = HeaderRowWidgetShared->GetColumns();
		const bool bAnyColumnsVisible = Algo::AnyOf(Columns, [](const SHeaderRow::FColumn& Column) { return Column.bIsVisible; });
		const bool bAnyColumnsInvisible = Algo::AnyOf(Columns, [](const SHeaderRow::FColumn& Column) { return !Column.bIsVisible; });

		if (bAnyColumnsVisible && bAnyColumnsInvisible)
		{
			return ECheckBoxState::Undetermined;
		}
		else if (bAnyColumnsVisible)
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}

	template<TIsVisibleColumnsSettingsType T>
	inline FText FVisibleColumnsSettingsMenu<T>::GetToggleAllColumnsVisibilityText() const
	{
		switch (GetToggleAllColumnsVisibilityCheckState())
		{
			case ECheckBoxState::Unchecked:
			case ECheckBoxState::Undetermined:
				return NSLOCTEXT("AudioInsights", "VisibleColumnsSettings_SelectAll", "Select: All");
			case ECheckBoxState::Checked:
				return NSLOCTEXT("AudioInsights", "VisibleColumnsSettings_SelectNone", "Select: None");
			default:
				return FText::GetEmpty();
		}
	}
}