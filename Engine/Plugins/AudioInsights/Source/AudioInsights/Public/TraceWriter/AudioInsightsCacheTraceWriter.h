// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

namespace UE::Trace
{
	class FTraceWriter;
} // namespace UE::Trace

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FAudioInsightsCacheManager;

	class FAudioInsightsCacheTraceWriter
	{
	public:
		UE_API static bool WriteCacheToFile(const FAudioInsightsCacheManager& InCacheManager, const FString& InFilePath);
	};
} // namespace UE::Audio::Insights

#undef UE_API
