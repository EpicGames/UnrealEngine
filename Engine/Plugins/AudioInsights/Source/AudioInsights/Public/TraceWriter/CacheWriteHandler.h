// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/Function.h"

namespace UE::Trace
{
	class FTraceWriter;
} // namespace UE::Trace

namespace UE::Audio::Insights
{
	class IAudioCachedMessage;

	struct FCacheWriteHandler
	{
		using FDeclareEventFunc = TFunction<uint32(UE::Trace::FTraceWriter& Writer)>;
		using FWriteEventFunc   = TFunction<void(UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& Message, uint64 TimestampCycles)>;

		FDeclareEventFunc DeclareEvent;
		FWriteEventFunc WriteEvent;

		bool IsValid() const { return DeclareEvent && WriteEvent; }
	};
} // namespace UE::Audio::Insights
