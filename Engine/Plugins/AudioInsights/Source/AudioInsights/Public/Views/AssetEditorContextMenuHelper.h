// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

#define UE_API AUDIOINSIGHTS_API

#if WITH_EDITOR
struct FKeyEvent;
class FUICommandList;
class SWidget;
class UObject;

namespace UE::Audio::Insights
{
	class IObjectDashboardEntry;

	// Helper for adding editor-only commands to the context menu of an Audio Insights dashboard
	class FAssetEditorContextMenuHelper
	{
	public:
		UE_API FAssetEditorContextMenuHelper();

		UE_API TSharedPtr<SWidget> ContructContextMenuOptions();
		UE_API bool ProcessCommandBindings(const FKeyEvent& InKeyEvent) const;

		UE_API void SetAssetEntry(const TSharedPtr<IObjectDashboardEntry>& InEntry);
		UE_API void ResetAssetEntry();

	private:
		void BindCommands();

		bool BrowseToAsset() const;
		bool OpenAsset() const;

		TObjectPtr<UObject> GetEditableAsset() const;

		TSharedPtr<FUICommandList> CommandList;
		TWeakPtr<IObjectDashboardEntry> Entry;
	};
} // namespace UE::Audio::Insights

#endif // WITH_EDITOR

#undef UE_API