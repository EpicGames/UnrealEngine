// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Views/TableDashboardViewFactory.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FAudioBusTraceProvider;
	struct FAudioBusDashboardEntry;

	class FAudioBusDashboardViewFactory : public FTraceObjectTableDashboardViewFactory
	{
	public:
		UE_API FAudioBusDashboardViewFactory();
		UE_API virtual ~FAudioBusDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	protected:
		enum class UE_API EAudioBusTypeComboboxSelection : uint8
		{
			AssetBased,
			CodeGenerated,
			All
		};

		UE_API TSharedRef<SWidget> MakeAudioBusTypeFilterWidget();

		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& InColumnName) override;
		UE_API virtual void ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason) override;
		UE_API virtual const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& GetColumns() const override;

		UE_API void RebuildAudioMeters() const;
		UE_API bool ShouldUpdateAudioMeter(const FAudioBusDashboardEntry& InAudioBusDashboardEntry) const;
		UE_API void UpdateAudioMeters() const;
		UE_API void ReactivateAudioMeters();

		UE_API void SetAudioBusEnvelopeFollowerEnabled(const FAudioBusDashboardEntry& InAudioBusDashboardEntry, const bool bInEnabled);

		UE_API virtual void SortTable() override;

		UE_API void FilterByAudioBusName();
		UE_API void FilterByAudioBusType();

		UE_API void HandleOnAudioBusAdded(const uint32 InAudioBusId);
		UE_API void HandleOnAudioBusRemoved(const uint32 InAudioBusId);
		UE_API void HandleOnAudioBusStarted(const uint32 InAudioBusId);

		UE_API void HandleOnTimeMarkerUpdated();
		UE_API void HandleOnTimeControlMethodReset();

		UE_API void RequestListRefresh();

		TSharedPtr<FAudioBusTraceProvider> AudioBusProvider;

		TSet<uint32> CheckedAudioBuses;
		TSet<uint32> PendingToReactivateAudioBusIds;

		bool bShouldRebuildAudioMeters = false;
		bool bListRefreshed = false;
		bool bShouldReactivateAudioMeters = false;

		using FComboboxSelectionItem = TPair<EAudioBusTypeComboboxSelection, FText>;
		TArray<TSharedPtr<FComboboxSelectionItem>> AudioBusTypes;
		TSharedPtr<FComboboxSelectionItem> SelectedAudioBusType;
	};
} // namespace UE::Audio::Insights

#undef UE_API
