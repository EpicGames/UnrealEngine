// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Filters/GenericFilter.h"
#include "Settings/AudioEventLogSettings.h"
#include "Settings/VisibleColumnsSettingsMenu.h"
#include "Templates/SharedPointer.h"
#include "Views/SAudioFilterBar.h"
#include "Views/TableDashboardViewFactory.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FAudioEventLogTraceProvider;

	using FEventLogFilterID = int32;

	class FAudioEventLogFilter : public FGenericFilter<FEventLogFilterID>
	{
	public:
		FAudioEventLogFilter
		(
			const FEventLogFilterID InFilterID,
			const FString& InName,
			const FText& InDisplayName,
			const FName& InIconName,
			const FText& InToolTipText,
			FLinearColor InColor,
			TSharedPtr<FFilterCategory> InCategory
		)
		: FGenericFilter<FEventLogFilterID>(InCategory, InName, InDisplayName, FGenericFilter<FEventLogFilterID>::FOnItemFiltered())
		, FilterID(InFilterID)
		{
			ToolTip = InToolTipText;
			Color = InColor;
			IconName = InIconName;
		}

		bool IsActive() const { return bIsActive; }
		FEventLogFilterID GetFilterID() const { return FilterID; }

	private:
		virtual void ActiveStateChanged(bool bActive) override { bIsActive = bActive; }
		virtual bool PassesFilter(FEventLogFilterID InItem) const override { return FilterID == InItem; }

		const FEventLogFilterID FilterID;
		bool bIsActive = false;
	};

	class FAudioEventLogDashboardViewFactory : public FTraceObjectTableDashboardViewFactory
	{
	public:
		UE_API FAudioEventLogDashboardViewFactory();
		UE_API virtual ~FAudioEventLogDashboardViewFactory();

		UE_API static const TMap<FString, TSet<FString>> GetInitEventTypeFilters();

		UE_API void RegisterExternalEventTypes(const TMap<FString, TSet<FString>>& InCategoriesToEvents);
		UE_API void RegisterExternalDisplayNames(const TMap<FString, FText>& InDisplayNames);

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column) override;
		UE_API virtual void RefreshFilteredEntriesListView() override;

	protected:
		UE_API virtual void ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason) override;
		UE_API virtual const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& GetColumns() const override;

		UE_API virtual void SortTable() override;
		UE_API virtual bool IsColumnSortable(const FName& ColumnId) const override;
		UE_API void OnHiddenColumnsListChanged() override;

		UE_API virtual TSharedPtr<SWidget> CreateFilterBarButtonWidget() override;
		UE_API virtual TSharedRef<SWidget> CreateFilterBarWidget() override;

		UE_API virtual TSharedRef<SWidget> CreateSettingsButtonWidget() override;
		UE_API virtual TSharedRef<SWidget> OnGetSettingsMenuContent() override;
		UE_API virtual FText OnGetSettingsMenuToolTip() override;

		UE_API virtual TSharedPtr<SWidget> OnConstructContextMenu() override;

		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;
		virtual bool ClearSelectionOnClick() const override { return true; }
		virtual bool EnableHorizontalScrollBox() const override { return false; }
		UE_API virtual void OnListViewScrolled(double InScrollOffset) override;
		UE_API virtual void OnFinishedScrolling();
		UE_API virtual const FTableRowStyle* GetRowStyle() const override;

#if WITH_EDITOR
		virtual TSharedRef<SWidget> MakeAssetMenuBar() const override { return SNullWidget::NullWidget; }
#endif // WITH_EDITOR

	private:
		void OnAnalysisStarting(const double Timestamp);

		void UpdateCustomEventLogFilters(const IDashboardDataViewEntry& Entry);
		TSharedRef<FAudioEventLogFilter> CreateNewEventFilterType(const TSharedPtr<FFilterCategory>& FilterCategory, const FString& EventType);
		const TSharedPtr<FFilterCategory>& FindOrAddFilterCategory(const FString& CategoryName);
		bool PassesEventTypeFilter(const FEventLogFilterID EventLogID) const;

		void BuildVisibleColumnsMenuContent(FMenuBuilder& MenuBuilder);

		void SortByPredicate(TFunctionRef<bool(const class FAudioEventLogDashboardEntry&, const class FAudioEventLogDashboardEntry&)> Predicate);

#if WITH_EDITOR
		void OnReadEditorSettings(const FAudioEventLogSettings& InSettings);
		void OnWriteEditorSettings(FAudioEventLogSettings& OutSettings);

		void RemoveDeletedCustomEvents(const TMap<FString, FAudioEventLogCustomEvents>& CustomEventsFromSettings);
		void AddNewCustomEvents(const TMap<FString, FAudioEventLogCustomEvents>& CustomEventsFromSettings);

		void RefreshFilterBarFromSettings(const TSet<FString>& EventFilters);
#endif // WITH_EDITOR

		void OnTimingViewTimeMarkerChanged(double InTimeMarker);
		void OnTimeControlMethodReset();

		void BindCommands();
		void ClearInspectState();
		void ResetInspectTimestamp();

		void ClearActiveAudioDeviceEntries();

		void OnUpdateAutoScroll();

		double GetTimestampRelativeToAnalysisStart(const double Timestamp) const;

		TSharedPtr<FAudioEventLogTraceProvider> AudioEventLogTraceProvider;

		TSharedPtr<SWidget> DashboardWidget;
		TSharedPtr<SAudioFilterBar<FEventLogFilterID>> FilterBar;

		TSharedPtr<SWidget> SettingsAreaWidget;

		TMap<FString, TSharedPtr<FFilterCategory>> FilterCategories;
		TMap<FString, FEventLogFilterID> EventFilterTypes;

		TSharedPtr<FUICommandList> CommandList;
		TSharedPtr<FVisibleColumnsSettingsMenu<FAudioEventLogVisibleColumns>> VisibleColumnsSettingsMenu;

		TWeakPtr<IDashboardDataViewEntry> FocusedItem;
		
		const TMap<FString, TSet<FString>> InitEventTypeFilters;
		TMap<FString, FAudioEventLogCustomEvents> CachedCustomEventSettings;
		TMap<FString, TSet<FString>> ExternalEventTypeFilters;
		TMap<FString, FText> ExternalDisplayNames;

		double BeginTimestamp = 0.0;
		bool bAutoScroll = true;
	};
} // namespace UE::Audio::Insights

#undef UE_API
