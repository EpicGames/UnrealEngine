// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Views/DashboardViewFactory.h"

#define UE_API AUDIOINSIGHTS_API

class SHorizontalBox;
class SScrollBox;
class SWidget;

namespace UE::Audio::Insights
{
	class SAudioMeterView;
	struct FAudioMeterInfo;

	class FAudioMetersPanelDashboardViewFactory : public IDashboardViewFactory, public TSharedFromThis<FAudioMetersPanelDashboardViewFactory>
	{
	public:
		UE_API FAudioMetersPanelDashboardViewFactory();
		UE_API virtual ~FAudioMetersPanelDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	private:
		void HandleOnAddAudioMeter(const uint32 InEntryId, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo, const FText& InName, const FSlateColor& InNameColor);
		void HandleOnRemoveAudioMeter(const uint32 InEntryId);
		void HandleOnUpdateAudioMeterInfo(const uint32 InEntryId, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo, const FText& InName);

		void AddAudioMeterView(const TSharedRef<SAudioMeterView>& InAudioMeterView);

		TSharedPtr<SScrollBox> AudioMeterWidgetsScrollBox;
		TSharedPtr<SHorizontalBox> AudioMeterWidgetsContainer;
		TMap<uint32, TSharedRef<SAudioMeterView>> AudioMeterViews;
	};
} // namespace UE::Audio::Insights

#undef UE_API
