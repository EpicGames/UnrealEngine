// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/SWidget.h"

class FSpawnTabArgs;
class SDockTab;

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	enum class EDefaultDashboardTabStack : uint8
	{
		Viewport,
		Log,
		Analysis,
		AudioMeters,
		Plots,
		OutputMetering,
		AudioAnalyzerRack
	};

	class IDashboardViewFactory
	{
	public:
		virtual ~IDashboardViewFactory() = default;

		virtual EDefaultDashboardTabStack GetDefaultTabStack() const = 0;
		virtual FText GetDisplayName() const = 0;
		virtual FName GetName() const = 0;
		virtual FSlateIcon GetIcon() const = 0;
		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) = 0;
	};

	class FTraceDashboardViewFactoryBase : public IDashboardViewFactory
	{
	public:
		UE_API FTraceDashboardViewFactoryBase();
		UE_API virtual ~FTraceDashboardViewFactoryBase();

		const TArray<TSharedPtr<FTraceProviderBase>>& GetProviders() const
		{
			return Providers;
		}

		template <typename ProviderType>
		TSharedPtr<ProviderType> FindProvider(bool bEnsureIfMissing = true) const
		{
			for (const TSharedPtr<FTraceProviderBase>& Provider : Providers)
			{
				if (Provider->GetName() == ProviderType::GetName_Static())
				{
					return StaticCastSharedPtr<ProviderType>(Provider);
				}
			}

			if (bEnsureIfMissing)
			{
				ensureMsgf(false, TEXT("Failed to find associated provider '%s'"), *ProviderType::GetName_Static().ToString());
			}

			return TSharedPtr<ProviderType>();
		}

	protected:
		enum class EProcessReason : uint8
		{
			None,
			FilterUpdated,
			EntriesUpdated
		};

		UE_API virtual void Tick(float InElapsed);

		virtual void ProcessEntries(EProcessReason Reason) { }
		virtual void RefreshFilteredEntriesListView() { }

#if WITH_EDITOR
		virtual void UpdateDebugDraw(const float DeltaTime) { }
#endif // WITH_EDITOR

		TArray<TSharedPtr<FTraceProviderBase>> Providers;

		EProcessReason UpdateFilterReason = EProcessReason::None;
		FTSTicker::FDelegateHandle TickerHandle;

	private:
		TMap<FName, uint64> UpdateIds;
	};
} // namespace UE::Audio::Insights

#undef UE_API
