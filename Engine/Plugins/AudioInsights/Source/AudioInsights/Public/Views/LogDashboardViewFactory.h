// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "Misc/OutputDevice.h"
#include "OutputLogCreationParams.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/SWidget.h"

#define UE_API AUDIOINSIGHTS_API

#define LOCTEXT_NAMESPACE "AudioInsights"


namespace UE::Audio::Insights
{
	class FLogDashboardViewFactory : public IDashboardViewFactory
	{
	public:
		UE_API FLogDashboardViewFactory();
		UE_API ~FLogDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	private:
		struct FLogCategoryCollector : public FOutputDevice
		{
			FLogCategoryCollector();
			~FLogCategoryCollector();

			FDefaultCategorySelectionMap GetCollectedCategories() const;

			virtual bool IsMemoryOnly() const override;
			virtual void Serialize(const TCHAR* InMsg, ELogVerbosity::Type Verbosity, const FName& InCategory) override;

		private:
			mutable FCriticalSection CollectionCritSec;
			FDefaultCategorySelectionMap CollectedCategories;
		};

		FLogCategoryCollector CategoryCollector;
	};
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE

#undef UE_API
