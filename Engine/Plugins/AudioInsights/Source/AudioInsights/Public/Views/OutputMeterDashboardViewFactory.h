// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMeterChannelInfo.h"
#include "DashboardViewFactory.h"
#include "LoudnessMeterWidgetView.h"
#include "Misc/Timespan.h"

#define UE_API AUDIOINSIGHTS_API

class SSplitter;
struct FLoudnessMeterSettings;
struct FOutputMeterDashboardSettings;

namespace UE::Audio::Insights
{
	class FOutputMeterTraceProvider;
	class FSubmixDashboardViewFactory;
	class FSubmixTraceProvider;
	class IDashboardDataViewEntry;
	struct FAudioMeterInfo;

	class FOutputMeterDashboardViewFactory : public FTraceDashboardViewFactoryBase, public TSharedFromThis<FOutputMeterDashboardViewFactory>
	{
	public:
		UE_API FOutputMeterDashboardViewFactory(TSharedRef<FSubmixDashboardViewFactory> SubmixDashboard);
		UE_API virtual ~FOutputMeterDashboardViewFactory();

		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FName GetName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	private:
		struct FLoudnessMeterResults
		{
			float LongTermLoudness  = MIN_VOLUME_DECIBELS;
			float ShortTermLoudness = MIN_VOLUME_DECIBELS;
			float MomentaryLoudness = MIN_VOLUME_DECIBELS;
			float LoudnessRangeLowerBound = 0.0f;
			float LoudnessRangeUpperBound = 0.0f;

			float TruePeakDb = MIN_VOLUME_DECIBELS;
		};

		struct FLoudnessMeterMaxResults
		{
			float ShortTermLoudness = MIN_VOLUME_DECIBELS;
			float MomentaryLoudness = MIN_VOLUME_DECIBELS;

			float TruePeakDb = MIN_VOLUME_DECIBELS;
		};

		FLoudnessMeterSettings& GetLoudnessMeterSettings() const;

		FOutputMeterDashboardSettings& GetOutputMeterDashboardSettings();
		void SaveOutputMeterDashboardSettings();

		void InitLoudnessMeterWidgetView();

		void UpdateDataViewEntries();
		void UpdateAudioMeterInfo();
		void UpdateLoudnessValues();

		virtual void ProcessEntries(FTraceDashboardViewFactoryBase::EProcessReason Reason) override;

		TArray<FAudioMeterChannelInfo> GetAudioMeterChannelInfo() const;

		FTimespan GetAnalysisTime() const;
		FReply HandleResetButtonClicked();

		void OnLoudnessMeterExpansionChanged(const bool bIsExpanded);
		void OnChannelMeterExpansionChanged(const bool bIsExpanded);
		void OnSplitterFinishedResizing();

		TSharedPtr<FOutputMeterTraceProvider> OutputMeterProvider;
		TArray<TSharedPtr<IDashboardDataViewEntry>> DataViewEntries;

		// Audio meter related members
		TSharedPtr<const FSubmixTraceProvider> SubmixProvider;
		TOptional<uint64> ProcessedUpdateId;
		TUniquePtr<FAudioMeterInfo> AudioMeterInfo;

		// Loudness meters related members
		AudioWidgetsCore::FLoudnessMeterWidgetView LoudnessMeterWidgetView;
		bool bIsLoudnessMeterInitialized = false;

		// Layout splitter for collapse/expand support
		TSharedPtr<SSplitter> ContentSplitter;

		FLoudnessMeterResults LoudnessMeterResults;
		FLoudnessMeterMaxResults LoudnessMeterMaxResults;

		double AnalysisStartTimestamp = 0.0;
		double LatestTimestamp = 0.0;
		bool bWasReceivingData = false;

#if WITH_EDITOR
		::Audio::FDeviceId LastSeenDeviceId = INDEX_NONE;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API
