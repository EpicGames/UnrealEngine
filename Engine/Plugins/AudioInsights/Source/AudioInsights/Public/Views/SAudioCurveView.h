// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPaintArgs;
class FSlateWindowElementList;
class SToolTip;

// Loosely based on SCurveTimelineView
// TODO move this to a more general place like the Audio Widgets plugin once it's further developed
class SAudioCurveView : public SCompoundWidget
{
public:
	// A curve point is a (double XValue, float YValue) pair
	using FCurvePoint = TPair<double, float>;

	DECLARE_DELEGATE_TwoParams(FOnScrubPositionChanged, double, float)

	struct FCurveMetadata
	{
		FLinearColor CurveColor;
		FText DisplayName;
		float PlotThickness = 1.0f;
	};

	SLATE_BEGIN_ARGS(SAudioCurveView)
		: _ViewRange(TRange<double>(0,5))
		, _AutoRangeYAxis(false)
		, _YMargin(0.05f)
		, _HorizontalAxisIncrement(0.5)
		, _GridLineColor(FLinearColor(0.5f, 0.5f, 0.5f, 0.25f))
		, _AxesLabelColor(FLinearColor::White)
		, _DesiredSize(FVector2D(100.f,100.f))
		, _AdditionalToolTipText(FText::GetEmpty())
	{}
    	/** View X axis range (in value space) */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);

		SLATE_ATTRIBUTE(bool, AutoRangeYAxis);

		/** Margin for Y axis, as a 0 - 0.5f proportion, for the space each of above and below the data range. (ex. 0.05 means a 5% margin on the top and bottom, with 90% of the widget's vertical size corresponding to the data range). */
		SLATE_ATTRIBUTE(float, YMargin);

		/** X axis increment for grid lines. */
		SLATE_ATTRIBUTE(double, HorizontalAxisIncrement);

		SLATE_ATTRIBUTE(FLinearColor, GridLineColor);

		SLATE_ATTRIBUTE(FLinearColor, AxesLabelColor);

		/** Desired widget size */
		SLATE_ATTRIBUTE(FVector2D, DesiredSize);

		/** Additional text displayed below the standard tooltip */
		SLATE_ATTRIBUTE(FText, AdditionalToolTipText);

		/** Called when user clicks/drags on the widget. Only returns when there is a valid crosshair position */
		SLATE_EVENT(FOnScrubPositionChanged, OnScrubPositionChanged);

		/** Called when user presses a key when the widget is in focus */
		SLATE_EVENT(FOnKeyDown, OnKeyDown);
	
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

	void SetYValueFormattingOptions(const FNumberFormattingOptions InValueFormattingOptions);
	void SetYAxisRange(const FFloatInterval& InRange);
	void SetCurvesPointData(TSharedPtr<TMap<uint64, TArray<FCurvePoint>>> InCurvesPointData);
	void SetCurvesMetadata(TSharedPtr<TMap<uint64, FCurveMetadata>> InMetadataPerCurve);

	FFloatInterval GetYAxisRange() const { return YDataRange; }

	FText GetCurveToolTipXValueText() const { return CurveToolTipXValueText; }
	FText GetCurveToolTipYValueText() const { return CurveToolTipYValueText; }
	FText GetCurveToolTipDisplayNameText() const { return CurveToolTipDisplayNameText; }

	/** Helper functions for converting between widget local Y position and a given data value (within DataRange). */
	float ValueToLocalY(const FVector2f AllottedLocalSize, const float Value) const;
	float LocalYToValue(const FVector2f AllottedLocalSize, const float LocalY) const;
	
#if !WITH_EDITOR
	void UpdateYDataRangeFromTimestampRange(const double InLowerBoundTimestamp, const double InUpperBoundTimestamp);
#endif // !WITH_EDITOR

protected:
	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnPreviewKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	
	int32 PaintCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	int32 PaintGridLines(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled, const SSimpleTimeSlider::FScrubRangeToScreen& RangeToScreen) const;
	int32 PaintCrosshair(const FGeometry& MyGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const SSimpleTimeSlider::FScrubRangeToScreen& RangeToScreen) const;
	int32 PaintYAxisLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void UpdateCurveToolTip(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void UpdateYDataRange();
	TSharedRef<SToolTip> CreateCurveTooltip();

	void BroadcastScrubPosition();

	TAttribute<TRange<double>> ViewRange;
	TAttribute<bool> AutoRangeYAxis;
	TAttribute<float> YMargin;
	TAttribute<double> HorizontalAxisIncrement;
	TAttribute<FLinearColor> GridLineColor;
	TAttribute<FLinearColor> AxesLabelColor;
	TAttribute<FVector2D> DesiredSize;
	TAttribute<FText> AdditionalToolTipText;

	FOnScrubPositionChanged OnScrubPositionChanged;
	FOnKeyDown OnKeyDownHandler;

	// Point data and metadata, keyed by curve id
	TSharedPtr<TMap<uint64, TArray<FCurvePoint>>> PointDataPerCurve;
	TSharedPtr<TMap<uint64, FCurveMetadata>> MetadataPerCurve;
	// Y axis data range in value space
	FFloatInterval YDataRange;

	// Tooltip text
	FText CurveToolTipXValueText;		
	FText CurveToolTipYValueText;
	FText CurveToolTipDisplayNameText;
	
	// Tooltip and axis text formatting
	FNumberFormattingOptions XValueFormattingOptions;
	FNumberFormattingOptions YValueFormattingOptions;

	ESlateDrawEffect LineDrawEffects;
	uint32 NumHorizontalGridLines;
	FSlateFontInfo LabelFont;

	TOptional<FCurvePoint> CrosshairPosition;
	bool bIsMouseDragging = false;
};
