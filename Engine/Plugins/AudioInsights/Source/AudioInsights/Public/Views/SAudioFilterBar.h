// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsStyle.h"
#include "Filters/FilterBase.h"
#include "Filters/SBasicFilterBar.h"

namespace UE::Audio::Insights
{
	template<typename EnumType>
	class SAudioFilterBar : public SBasicFilterBar<EnumType>
	{
	public:
		using FOnFilterChanged = typename SBasicFilterBar<EnumType>::FOnFilterChanged;

		SLATE_BEGIN_ARGS(SAudioFilterBar)
			: _UseSectionsForCategories(true)
		{}
			SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<EnumType>>>, CustomFilters)
			SLATE_ARGUMENT(bool, UseSectionsForCategories)
			SLATE_EVENT(SAudioFilterBar<EnumType>::FOnFilterChanged, OnFilterChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			typename SBasicFilterBar<EnumType>::FArguments Args;
			Args._OnFilterChanged = InArgs._OnFilterChanged;
			Args._CustomFilters = InArgs._CustomFilters;

			SBasicFilterBar<EnumType>::Construct(Args
				.CanChangeOrientation(false)
				.FilterBarLayout(EFilterBarLayout::Horizontal)
				.UseSectionsForCategories(InArgs._UseSectionsForCategories)
			);
		}

		void DeleteCategory(const TSharedRef<FFilterCategory> InCategory)
		{
			SBasicFilterBar<EnumType>::AllFilterCategories.Remove(InCategory);
		}

		void DeleteFromFilter(const TSharedRef<FFilterBase<EnumType>> InFilter)
		{
			SBasicFilterBar<EnumType>::SetFilterCheckState(InFilter, ECheckBoxState::Unchecked);
			SBasicFilterBar<EnumType>::AllFrontendFilters.Remove(InFilter);
		}

	protected:
		virtual TSharedRef<SWidget> MakeAddFilterMenu() override
		{
			TObjectPtr<UToolMenus> ToolsMenus = UToolMenus::Get();

			if (ToolsMenus == nullptr)
			{
				return SNullWidget::NullWidget;
			}

			const FName FilterMenuName = "FilterBar.FilterMenu";

			if (!ToolsMenus->IsMenuRegistered(FilterMenuName))
			{
				const TObjectPtr<UToolMenu> ToolMenu = ToolsMenus->RegisterMenu(FilterMenuName);

				ToolMenu->bShouldCloseWindowAfterMenuSelection = true;
				ToolMenu->bCloseSelfOnly = true;

				ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					if (const TObjectPtr<UFilterBarContext> FilterBarContext = InMenu->FindContext<UFilterBarContext>())
					{
						FilterBarContext->PopulateFilterMenu.ExecuteIfBound(InMenu);
						FilterBarContext->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
					}
				}));
			}

			TObjectPtr<UFilterBarContext> FilterBarContext = NewObject<UFilterBarContext>();

			FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateSP(this, &SAudioFilterBar::PopulateAddFilterMenu);
			FilterBarContext->OnExtendAddFilterMenu = SBasicFilterBar<EnumType>::OnExtendAddFilterMenu;

			FToolMenuContext ToolMenuContext(FilterBarContext);

			return ToolsMenus->GenerateWidget(FilterMenuName, ToolMenuContext);
		}

		void PopulateAddFilterMenu(UToolMenu* InMenu)
		{
			SBasicFilterBar<EnumType>::PopulateCommonFilterSections(InMenu);
			PopulateCustomFilters(InMenu);
		}

		void PopulateCustomFilters(UToolMenu* InMenu)
		{
			if (SBasicFilterBar<EnumType>::bUseSectionsForCategories)
			{
				for (const TSharedPtr<FFilterCategory>& Category : SBasicFilterBar<EnumType>::AllFilterCategories)
				{
					if (!Category.IsValid())
					{
						continue;
					}

					FToolMenuSection& Section = InMenu->AddSection(*Category->Title.ToString(), Category->Title);

					for (const TSharedRef<FFilterBase<EnumType>>& FrontendFilter : SBasicFilterBar<EnumType>::AllFrontendFilters)
					{
						if (FrontendFilter->GetCategory() == Category)
						{
							Section.AddMenuEntry(
								NAME_None,
								FrontendFilter->GetDisplayName(),
								FrontendFilter->GetToolTipText(),
								FSlateStyle::Get().CreateIcon(FrontendFilter->GetIconName()),
								FUIAction(
									FExecuteAction::CreateSP(const_cast<SAudioFilterBar*>(this), &SAudioFilterBar::FrontendFilterClicked, FrontendFilter),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(this, &SAudioFilterBar::IsFrontendFilterInUse, FrontendFilter)),
								EUserInterfaceActionType::ToggleButton
							);
						}
					}
				}
			}
			else
			{
				SBasicFilterBar<EnumType>::PopulateCustomFilters(InMenu);
			}
			
		}
	};
} // namespace UE::Audio::Insights
