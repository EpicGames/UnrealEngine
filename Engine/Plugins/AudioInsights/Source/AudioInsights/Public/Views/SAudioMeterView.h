// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Widgets/SCompoundWidget.h"

class SAudioMeterWidget;

namespace UE::Audio::Insights
{
	struct FAudioMeterInfo;

	class SAudioMeterView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAudioMeterView)
			: _NameText(FText::GetEmpty())
			, _NameTextColor(FSlateColor(FColor::White))
		{}
			SLATE_ARGUMENT(FText, NameText)
			SLATE_ARGUMENT(FSlateColor, NameTextColor)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo);

		void SetAudioMeterChannelInfo(const TSharedRef<FAudioMeterInfo>& InAudioMeterInfo);
		void SetName(const FText& InName);

	protected:
		TSharedPtr<SAudioMeterWidget> AudioMeterWidget;
		FText NameText;
	};
} // namespace UE::Audio::Insights
