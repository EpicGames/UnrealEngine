// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Messages/SignalFlowEntryKey.h"
#include "Settings/SignalFlowSettings.h"
#include "SSignalFlowGraphNode.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SCanvas.h"
#include "Widgets/SCompoundWidget.h"

#if WITH_EDITOR
#include "Views/AssetEditorContextMenuHelper.h"
#endif // WITH_EDITOR

#define UE_API AUDIOINSIGHTS_API

struct FSignalFlowNodeDetailFilterSettings;

#if WITH_EDITOR
class IMenu;
#endif // WITH_EDITOR

namespace UE::Audio::Insights
{
	struct FSignalFlowDashboardEntry;
	class ISignalFlowNode;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsEntryFilteredOutByText, const FSignalFlowDashboardEntry&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsInHighlightedPath, const FSignalFlowEntryKey&);

	class SSignalFlowScrollBox final : public SScrollBox
	{
	public:
		void ResetAnimatingScroll() 
		{ 
			if (!bIsScrolling)
			{
				bAnimateScroll = false;
			}
		}
	};

	class SSignalFlowGraph : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSignalFlowGraph)
			: _ListSource(nullptr)
			, _NodeDetailFilterSettings(nullptr)
			, _GraphJustification(ESignalFlowJustification::Edge)
			, _ShowNodeDetails(false)
			, _DisplayAmpPeakInDb(true)
			, _AnimateWires(true)
			, _HighlightPathActive(false)
			, _LargeNodePadding(128.0f)
			, _SmallNodePadding(8.0f)
		{
		}

		SLATE_ARGUMENT(TArray<TSharedPtr<ISignalFlowNode>>*, ListSource)
		SLATE_ARGUMENT(FSignalFlowNodeDetailFilterSettings*, NodeDetailFilterSettings)

		SLATE_ATTRIBUTE(TSharedPtr<FSignalFlowDashboardEntry>, SelectedItem)
		SLATE_ATTRIBUTE(TSharedPtr<FSignalFlowDashboardEntry>, HighlightedItem)
		SLATE_ATTRIBUTE(TSharedPtr<FSignalFlowDashboardEntry>, ActiveAudioDeviceEntry)
		SLATE_ATTRIBUTE(TSharedPtr<FSignalFlowDashboardEntry>, FocusedItem)
		SLATE_ATTRIBUTE(TWeakPtr<SSignalFlowScrollBox>, HorizontalScrollBox)
		SLATE_ATTRIBUTE(TWeakPtr<SSignalFlowScrollBox>, VerticalScrollBox)
		SLATE_ATTRIBUTE(ESignalFlowJustification, GraphJustification)
		SLATE_ATTRIBUTE(bool, ShowNodeDetails)
		SLATE_ATTRIBUTE(bool, DisplayAmpPeakInDb)
		SLATE_ATTRIBUTE(bool, AnimateWires)
		SLATE_ATTRIBUTE(bool, HighlightPathActive)
		SLATE_ATTRIBUTE(float, LargeNodePadding)
		SLATE_ATTRIBUTE(float, SmallNodePadding)

		SLATE_EVENT(FOnSignalFlowNodeSelected, OnNodeSelected)
		SLATE_EVENT(FIsEntryFilteredOutByText, IsEntryFilteredOutByText)
		SLATE_EVENT(FIsInHighlightedPath, IsInHighlightedPath)

		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
		void SetHorizontalScrollBox(const TWeakPtr<SSignalFlowScrollBox> ScrollBox) { HorizontalScrollBox = ScrollBox; }
		void SetVerticalScrollBox(const TWeakPtr<SSignalFlowScrollBox> ScrollBox) { VerticalScrollBox = ScrollBox; }

		UE_API void RefreshGraph();
		UE_API void ResetGraph();

		UE_API void ToggleOrientation();
		EOrientation GetGraphOrientation() const { return Orientation; }

		UE_API void SetWireAnimationSettings(const float AmpPowerFactor, const float WireThicknessScalarMax);

		UE_API void ResetZoom();

		UE_API TSharedPtr<SSignalFlowGraphNode> FindNodeWidget(const FSignalFlowEntryKey& NodeEntryKey) const;

		UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
		
		UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		UE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	protected:
		UE_API virtual FVector2D ComputeDesiredSize(float InLayoutScaleMultiplier) const override;

	private:
		int32 OnPaintConnections(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

		int32 PaintConnectionToNode(const FGeometry& AllottedGeometry,
									FSlateWindowElementList& OutDrawElements,
									int32 LayerId,
									const TSharedPtr<FSignalFlowDashboardEntry>& SenderEntry,
									const SSignalFlowGraphNode& SenderNode,
									const SSignalFlowGraphNode& ReceiverNode,
									const FBox2D& LocalBoundsBox,
									const FVector2D& ViewportPosInGraph,
									const FVector2D& CursorPosInViewport,
									const float SendLevel,
									const int32 SendLabelIndex,
									const bool bIsSourceBusConnection,
									const bool bHighlightPathModeActive) const;

		int32 AnimateWire(const FGeometry& AllottedGeometry,
							FSlateWindowElementList& OutDrawElements,
							int32 LayerId,
							const TSharedPtr<FSignalFlowDashboardEntry>& Entry,
							const ESignalFlowEntryType ReceiverEntryType,
							const FVector2D& ConnectionStartPos,
							const FVector2D& ConnectionEndPos,
							const float SendLevel,
							const bool bIsSourceBusConnection,
							const bool bIsSendLabelHovered = false) const;

		void CollectAmplitudeConnectionPoints(const float Amplitude,
												const FVector2D& ConnectionStartPos,
												const FVector2D& StartDirection,
												const FVector2D& ConnectionEndPos,
												const FVector2D& EndDirection,
												const float Prop,
												TArray<FVector2D>& OutTopPoints,
												TArray<FVector2D>& OutBottomPoints,
												FVector2D& OutLastPoint,
												const float HeightMultiplier = 1.0f) const;

		struct FGraphLimits
		{
			float MinPos = TNumericLimits<float>::Max();
			float MaxPos = TNumericLimits<float>::Lowest();

			bool IsValid() const
			{
				return MinPos != TNumericLimits<float>::Max() && MaxPos != TNumericLimits<float>::Lowest();
			}
		};

		FGraphLimits GetGraphWidthLimits() const;
		float GetNodeGraphSlotWidth() const;
		float GetNodeGraphSlotHeight() const;

		void CreateGraphNodes(TSet<FSignalFlowEntryKey>& OutRemainingNodeKeys, const FGraphLimits& GraphWidthLimits, TArray<struct TreeDepthPair>& OutTreeDepthStructure);
		void CreateSendLabels();
		void CleanUpOldWidgets(const TSet<FSignalFlowEntryKey>& NodeKeysToRemove);
		
		void MaintainOldViewportPosition(const FVector2D& OriginalSize, const FGraphLimits& NewGraphWidthLimits, const TArray<struct TreeDepthPair>& NewTreeDepthOrder);

		FVector2D ComputeNodePositionInGraph(const SSignalFlowGraphNode& Node) const;

		void UpdateContentPadding(const FGraphLimits& GraphWidthLimits, const TArray<struct TreeDepthPair>& TreeDepthOrder);
		void ApplyContentPaddingAndCompensateScroll(const TSharedPtr<SSignalFlowScrollBox>& InHorizontalScrollBox, const TSharedPtr<SSignalFlowScrollBox>& InVerticalScrollBox);
		void DetectAndHandleViewportResize();
		FVector2D GetRawContentSize(const FGraphLimits& GraphWidthLimits, const TArray<struct TreeDepthPair>& TreeDepthOrder) const;

		void CreateGraphNode(const TSharedPtr<ISignalFlowNode>& Node, const float MinXPos, const float MaxXPos, const int32 DepthTracker, TSet<FSignalFlowEntryKey>& OutRemainingNodeKeys);

		enum class ENodeConnectionType : uint8
		{
			Input,
			Output
		};

		TSharedPtr<SSignalFlowGraphNode> FindRealNode(const FSignalFlowEntryKey& EntryKey, const ENodeConnectionType ConnectionType) const;
		TOptional<FSignalFlowEntryKey> FindRealNodeKey(const FSignalFlowEntryKey& EntryKey, const ENodeConnectionType ConnectionType) const;

		enum class EAnimationResult : uint8
		{
			InProgress,
			Complete
		};

		EAnimationResult AnimateScrollTo(const TSharedRef<SScrollBox>& ScrollBox, const FVector2D& ScrollToPos, const float DeltaTime);

#if WITH_EDITOR
		void OnRequestedContextMenu(const FPointerEvent& MouseEvent, const TSharedPtr<FSignalFlowDashboardEntry> Entry);
#endif // WITH_EDITOR

		TSharedPtr<SCanvas> Canvas;

		TMap<FSignalFlowEntryKey, TSharedPtr<SSignalFlowGraphNode>> NodeWidgets;

		struct FConnectedNodeKeys
		{
			FSignalFlowEntryKey RealInputKey;
			FSignalFlowEntryKey RealOutputKey;
		};

		TMap<FSignalFlowEntryKey, FConnectedNodeKeys> DummyToRealConnectionKeys;

		TArray<TSharedPtr<ISignalFlowNode>>* ListSource = nullptr;
		FSignalFlowNodeDetailFilterSettings* NodeDetailFilterSettings = nullptr;

		TAttribute<TSharedPtr<FSignalFlowDashboardEntry>> SelectedItem;
		TAttribute<TSharedPtr<FSignalFlowDashboardEntry>> HighlightedItem;
		TAttribute<TSharedPtr<FSignalFlowDashboardEntry>> ActiveAudioDeviceEntry;
		TAttribute<TSharedPtr<FSignalFlowDashboardEntry>> FocusedItem;
		TAttribute<ESignalFlowJustification> GraphJustification;
		TAttribute<bool> ShowNodeDetails;
		TAttribute<bool> DisplayAmpPeakInDb;
		TAttribute<bool> AnimateWires;
		TAttribute<bool> HighlightPathActive;

		TAttribute<TWeakPtr<SSignalFlowScrollBox>> HorizontalScrollBox;
		TAttribute<TWeakPtr<SSignalFlowScrollBox>> VerticalScrollBox;

		FOnSignalFlowNodeSelected OnNodeSelected;
		FIsEntryFilteredOutByText IsEntryFilteredOutByText;
		FIsInHighlightedPath IsInHighlightedPath;

#if WITH_EDITOR
		TSharedPtr<IMenu> GraphContextMenu;
		FAssetEditorContextMenuHelper AssetContextMenuHelper;
#endif // WITH_EDITOR

		EOrientation Orientation = EOrientation::Orient_Horizontal;

		FGraphLimits CachedGraphWidthLimits;
		TArray<struct TreeDepthPair> CachedTreeDepthPairs;
		int32 MaxNumNodeOutputsInGraph = 0;

		TAttribute<float> LargeNodePadding;
		TAttribute<float> SmallNodePadding;

		float ZoomScaleFactor = 0.5f;
		float WireSplineAmplitudePowerFactor = 0.4f;
		float WireSplineThicknessScalarMax = 14.0f;

		// Dynamic padding (in unzoomed graph space) applied symmetrically to inflate desired size
		// so there is always enough scroll range for MaintainOldViewportPosition to work
		// even when the graph content is smaller than the viewport.
		FVector2D ContentPadding = FVector2D::ZeroVector;

		// Last-seen viewport size of the wrapping scroll box. Compared each tick so we can
		// react to splitter drags / window resizes and keep padding in sync with the viewport.
		// Unset until the first tick where the scroll box reports a non-zero size.
		TOptional<FVector2D> CachedViewportSize;

		// Reusable scratch buffers for AnimateWire to avoid per-wire heap allocation each paint.
		mutable TArray<FVector2D> AnimateWireTopPoints;
		mutable TArray<FVector2D> AnimateWireBottomPoints;

		bool IsCursorNearSendLabel(const FVector2D& CursorPosInViewport, const FVector2D& SendLabelPos) const;

		bool bNeedsRefresh = false;
		bool bAnimateZoom = false;
		bool bIsDraggingToMove = false;
	};
} // namespace UE::Audio::Insights

#undef UE_API