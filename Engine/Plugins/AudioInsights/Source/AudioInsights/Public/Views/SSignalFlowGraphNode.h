// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Messages/SignalFlowEntryKey.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOINSIGHTS_API
#define MUTE_SOLO_ENABLED (WITH_EDITOR && ENABLE_AUDIO_DEBUG)

struct FSignalFlowNodeDetailFilterSettings;

class SBorder;
class SHorizontalBox;
class SVerticalBox;

namespace UE::Audio::Insights
{
	struct FSignalFlowDashboardEntry;

	DECLARE_DELEGATE_OneParam(FOnSignalFlowNodeSelected, TSharedPtr<FSignalFlowDashboardEntry>);

#if WITH_EDITOR
	DECLARE_DELEGATE_TwoParams(FOnSignalFlowNodeContextMenuOpened, const FPointerEvent&, const TSharedPtr<FSignalFlowDashboardEntry>);
#endif // WITH_EDITOR

	class SSignalFlowGraphNode : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSignalFlowGraphNode)
			: _Entry(nullptr),
			_NodeDetailFilterSettings(nullptr),
			_DisplayName(),
			_PositionInGraph(FVector2d::ZeroVector),
			_XPos(0.0f),
			_YPos(0.0f),
			_ZoomScaleFactor(1.0f),
			_IsSelected(false),
			_ShowNodeDetails(false),
			_IsFilteredOutByText(false),
			_DisplayAmpPeakInDb(true),
#if WITH_EDITOR
			_GraphIsDraggingToMove(false),
#endif // WITH_EDITOR
			_GraphOrientation(EOrientation::Orient_Vertical)
		{
		}

		SLATE_ARGUMENT(TSharedPtr<FSignalFlowDashboardEntry>, Entry)
		SLATE_ARGUMENT(FSignalFlowNodeDetailFilterSettings*, NodeDetailFilterSettings)
		SLATE_ARGUMENT(FText, DisplayName)
		SLATE_ATTRIBUTE(FVector2d, PositionInGraph)
		SLATE_ATTRIBUTE(float, XPos)
		SLATE_ATTRIBUTE(float, YPos)
		SLATE_ATTRIBUTE(float, ZoomScaleFactor)
		SLATE_ATTRIBUTE(bool, IsSelected)
		SLATE_ATTRIBUTE(bool, IsHighlighted)
		SLATE_ATTRIBUTE(bool, ShowNodeDetails)
		SLATE_ATTRIBUTE(bool, IsFilteredOutByText)
		SLATE_ATTRIBUTE(bool, DisplayAmpPeakInDb)

#if WITH_EDITOR
		SLATE_ATTRIBUTE(bool, GraphIsDraggingToMove)
#endif // WITH_EDITOR

		SLATE_ATTRIBUTE(EOrientation, GraphOrientation)
		SLATE_EVENT(FOnSignalFlowNodeSelected, OnNodeSelected)

#if MUTE_SOLO_ENABLED
		SLATE_ATTRIBUTE(bool, IsDirectlySoloed)
#endif // MUTE_SOLO_ENABLED

#if WITH_EDITOR
		SLATE_EVENT(FOnSignalFlowNodeContextMenuOpened, OnOpenContextMenu)
#endif // WITH_EDITOR

		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

#if WITH_EDITOR
		UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
#endif // WITH_EDITOR

		UE_API virtual FVector2D ComputeDesiredSize(float InLayoutScaleMultiplier) const override;

		UE_API FVector2D GetLocalInputPos() const;
		UE_API FVector2D GetLocalCenteredOutputPos() const;
		UE_API FVector2D GetLocalSendLabelPos(const FSignalFlowEntryKey& ReceiverEntryKey, const int32 LabelIndex) const;
		UE_API int32 GetNumSendLabels() const;

		UE_API void UpdateNodeLayout();
		UE_API FVector2D GetCachedPositionInGraph() const;

		UE_API void SetSendLabelWidgets(const TSet<FSignalFlowEntryKey>& ReceiverEntryKeys);
		UE_API void RemoveSendLabelWidget(const FSignalFlowEntryKey& ReceiverEntryKey);
		UE_API bool ContainsSendLabel(const FSignalFlowEntryKey& ReceiverEntryKey) const;

		const FSignalFlowEntryKey& GetNodeEntryKey() const { return NodeEntryKey; }
		const TSharedPtr<FSignalFlowDashboardEntry> GetEntry() const { return Entry; }

#if MUTE_SOLO_ENABLED
		FName GetAssetFName() const { return AssetFName; }
#endif // MUTE_SOLO_ENABLED

		float GetXPos() const { return XPos.Get(); }
		float GetYPos() const { return YPos.Get(); }

		void SetXPos(const float Pos) { XPos = Pos; }
		void SetYPos(const float Pos) { YPos = Pos; }

	private:
		void CreateSendLabelWidget(const FSignalFlowEntryKey& ReceiverEntryKey);

		TSharedRef<SHorizontalBox> CreateNodeName(const SSignalFlowGraphNode::FArguments& InArgs);
		TSharedRef<SVerticalBox> CreateNodeDetails() const;
		TSharedRef<SHorizontalBox> CreateNodeDetailsRow(const ESignalFlowNodeDetailParam ParamType) const;
		TSharedRef<SVerticalBox> CreateSendOutputsVerticalContainer();
		TSharedRef<SHorizontalBox> CreateSendOutputsHorizontalContainer();

		bool NodeDetailsExpanderArrowIsEnabled() const;
		bool SendLabelsAreVisible() const;

#if MUTE_SOLO_ENABLED
		void ToggleMute();
		void ToggleSolo();

		struct FMuteSoloStates
		{
			bool bIsMuted = false;
			bool bIsSoloed = false;
		};

		FMuteSoloStates GetMuteSoloState() const;
		bool IsMuteSoloAvailableForNode() const;
#endif // MUTE_SOLO_ENABLED

		TSharedPtr<FSignalFlowDashboardEntry> Entry;
		TAttribute<FVector2d> PositionInGraph;
		TAttribute<float> XPos;
		TAttribute<float> YPos;
		TAttribute<float> ZoomScaleFactor;
		TAttribute<bool> IsSelected;
		TAttribute<bool> IsHighlighted;
		TAttribute<bool> ShowNodeDetails;
		TAttribute<bool> IsFilteredOutByText;
		TAttribute<bool> DisplayAmpPeakInDb;
		TAttribute<EOrientation> GraphOrientation;

		TMap<FSignalFlowEntryKey, TSharedRef<class SSignalFlowGraphSendLabel>> SendLabels;

		FOnSignalFlowNodeSelected OnNodeSelected;

#if WITH_EDITOR
		FOnSignalFlowNodeContextMenuOpened OnOpenContextMenu;
		TAttribute<bool> GraphIsDraggingToMove;
#endif // WITH_EDITOR

#if MUTE_SOLO_ENABLED
		FName AssetFName;
		FString AssetString;
		TAttribute<bool> IsDirectlySoloed;
#endif // MUTE_SOLO_ENABLED

		FVector2d CachedPositionInGraph = FVector2d::Zero();
		FSignalFlowEntryKey NodeEntryKey;

		FSignalFlowNodeDetailFilterSettings* NodeDetailFilterSettings = nullptr;

		TSharedPtr<SBorder> SelectedNodeBorder;
		TSharedPtr<SBorder> NodeBorder;
		TSharedPtr<SHorizontalBox> NodeLabelHorizontalBox;
		TSharedPtr<SVerticalBox> SendLevelsVerticalBox;
		TSharedPtr<SHorizontalBox> SendLevelsHorizontalBox;

		bool bNodeDetailsAreExpanded = true;
	};
} // namespace UE::Audio::Insights

#undef UE_API