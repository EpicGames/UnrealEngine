// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Messages/SignalFlowEntryKey.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOINSIGHTS_API

class SImage;
class SOverlay;
class STextBlock;

namespace UE::Audio::Insights
{
	class SSignalFlowGraphSendLabel : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSignalFlowGraphSendLabel)
			: _ReceiverEntryType(ESignalFlowEntryType::AudioDevice),
			_SendLevel(),
			_GraphOrientation(EOrientation::Orient_Vertical),
			_ZoomScaleFactor(1.0f)
		{
		}

		SLATE_ARGUMENT(ESignalFlowEntryType, ReceiverEntryType)

		SLATE_ATTRIBUTE(TOptional<float>, SendLevel)
		SLATE_ATTRIBUTE(EOrientation, GraphOrientation)
		SLATE_ATTRIBUTE(float, ZoomScaleFactor)

		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		UE_API virtual FVector2D ComputeDesiredSize(float InLayoutScaleMultiplier) const override;

		UE_API bool ValueIsSet() const;

	private:
		TSharedRef<STextBlock> CreateOutputLabelText(const FText& ToolTipText) const;
		TSharedRef<SOverlay> CreatePinIconWidget(const FText& ToolTipText, const ESignalFlowEntryType ReceiverEntryType);

		TAttribute<TOptional<float>> SendLevel;
		TAttribute<EOrientation> GraphOrientation;
		TAttribute<float> ZoomScaleFactor;

		TSharedPtr<SImage> OutputPinImage;
	};
} // namespace UE::Audio::Insights

#undef UE_API