// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Messages/SignalFlowEntryKey.h"
#include "Settings/SignalFlowSettings.h"
#include "Templates/SharedPointer.h"
#include "Views/SSignalFlowGraph.h"
#include "Views/TableDashboardViewFactory.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/SListView.h"

#if WITH_EDITOR
#include "Views/AssetEditorContextMenuHelper.h"
#endif // WITH_EDITOR

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FSignalFlowTraceProvider;
	class FSignalFlowEntryNode;
	class FDummyConnectionNode;
	class ISignalFlowNode;

	struct FSignalFlowDashboardEntry;
	struct TreeDepthPair;

	class SSignalFlowDashboard : public SVerticalBox
	{
	public:
		UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual bool SupportsKeyboardFocus() const override { return true; }

		void SetOnKeyDownHandler(FOnKeyDown Handler) { OnKeyDownHandler = Handler; }

	private:
		FOnKeyDown OnKeyDownHandler;
	};

	class FSignalFlowDashboardViewFactory : public FTraceDashboardViewFactoryBase, public TSharedFromThis<FSignalFlowDashboardViewFactory>
	{
	public:
		UE_API FSignalFlowDashboardViewFactory();
		UE_API virtual ~FSignalFlowDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

		UE_API FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
		UE_API void Reset();

	private:
		struct SRowWidget : public STableRow<TSharedPtr<FSignalFlowDashboardEntry>>
		{
			SLATE_BEGIN_ARGS(SRowWidget)
				: _Style(nullptr)
				{
				}
				SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FSignalFlowDashboardEntry> InData);

			TSharedPtr<FSignalFlowDashboardEntry> RowEntry;
		};

		const FTableRowStyle* GetRowStyle() const;

		void BindCommands();

		TSharedRef<SWidget> CreateToolbar();
		TSharedRef<SWidget> CreateToolbarSettingsMenu();
		TSharedRef<SWidget> CreateToolbarToggleControls();
		TSharedRef<SWidget> CreateToolbarButtonActions();
		TSharedRef<SWidget> CreateToolbarIndicators();
#if MUTE_SOLO_ENABLED
		TSharedRef<SWidget> CreateToolbarClearMuteSoloButton();
#endif // MUTE_SOLO_ENABLED

		TSharedRef<SWidget> OnGetSettingsMenuContent();
		void BuildJustificationMenuContent(FMenuBuilder& OutMenuBuilder);
		void BuildNodeDetailFiltersMenuContent(FMenuBuilder& OutMenuBuilder);
		void CreateAmpDisplayModeToggle(FMenuBuilder& OutMenuBuilder);
		void CreateNodePaddingControls(FMenuBuilder& OutMenuBuilder);
		void CreateNodePaddingSpinBox(FMenuBuilder& OutMenuBuilder, const FText& Label, const FText& Tooltip, float& OutValue);

		FReply ScrollGraphViewToSelectedNode(const bool bResetZoom);
		void ToggleGraphOrientation();
		void ToggleShowNodeDetails();
		void TogglePauseGraphEnabled();
		void ToggleEnableAllNodeDetailFilters();
		void ToggleAnimateWires();
		void SetJustification(const ESignalFlowJustification Justification);

		bool CanPauseTimestamp() const;
		bool IsTimestampPaused() const;
		void ResetPauseTimestamp();

		TSharedPtr<FSignalFlowDashboardEntry> GetFilteredAudioDeviceEntry() const;

		TSharedRef<SWidget> CreateSelectionPanel();
		TSharedRef<SWidget> CreateGraphSection();

		void OnMainSplitterFinishedResizing();
		void OnCategorySplitterFinishedResizing();

		TSharedRef<SWidget> CreateSelectableList(const FText& HeadingText, const TArray<TSharedPtr<FSignalFlowDashboardEntry>>& ListSource, const ESignalFlowEntryType EntryType, TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>>& OutListView);	
		TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> GetSelectableList(const ESignalFlowEntryType EntryType);
		bool GetSelectableListIsExpanded(const ESignalFlowEntryType EntryType) const;
		void SetSelectableListIsExpanded(const ESignalFlowEntryType EntryType, const bool bIsExpanded);
		void SetSelectableListSlotSize(const ESignalFlowEntryType EntryType, const bool bIsExpanded, float& OutSlotSize);
		void ApplyAllCategorySlotSizes();

		void SetSearchBoxFilterText(const FText& NewText);

		void OnListSelectionChanged(TSharedPtr<FSignalFlowDashboardEntry> InSelectedItem, ESelectInfo::Type SelectInfo);
		void OnSelectionChanged(TSharedPtr<FSignalFlowDashboardEntry> InSelectedItem, ESelectInfo::Type SelectInfo, const bool bCenterViewAfterSelection = false);
		void SetNewSelection(const TSharedPtr<FSignalFlowDashboardEntry>& InSelectedItem, const bool bCenterViewAfterSelection = false);

		void SetHighlightedItem(const TSharedPtr<FSignalFlowDashboardEntry>& InHighlightedItem);
		void RecomputeHighlightedPathKeys(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData);
		void ClearListHighlights();

		virtual void ProcessEntries(EProcessReason Reason) override;
		void BuildSelectionMenus(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const EProcessReason Reason);
		bool IsEntryFilteredOutByText(const FSignalFlowDashboardEntry& Entry) const;

		virtual void RefreshFilteredEntriesListView() override;
		void OnRequestGraphRefresh();
		void SendSelectedNodeDestroyedNotification();
		void CreateFilteredNodeGraph(const FSignalFlowTraceProvider& Provider, TSharedPtr<FSignalFlowDashboardEntry> FilterFromEntry, const bool bAddAllNodes);
		
		void AddAllNodesToGraph();
		void AddFilteredInputsRecursive(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, TSharedPtr<FSignalFlowDashboardEntry> ChildEntry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredOutputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredOutputsRecursive(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, TSharedPtr<FSignalFlowDashboardEntry> ParentEntry, int32 TreeDepth, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);

		void AddFilteredLinkedSourceBus(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredLinkedSources(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredLinkedBusPatchInputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredLinkedBusPatchOutputs(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, ISignalFlowNode* const OutEntryNode, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);
		void AddFilteredLinkedEntries(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSharedPtr<FSignalFlowDashboardEntry> Entry, const TSet<FSignalFlowEntryKey>& LinkedEntryKeys, TArray<FSignalFlowEntryKey>* OutFilteredLinkedEntries, TFunctionRef<void(ISignalFlowNode&)> LinkBackFunction, TSet<FSignalFlowEntryKey>& OutExistingEntryKeys, TSet<FSignalFlowEntryKey>* OutPathKeys = nullptr);

		void CleanUpFilteredEntries(const TSet<FSignalFlowEntryKey>& EntriesToRemove);

		void AddMissingNodesToGraph(const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete);
		void AddMissingNodeByOutputs(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete);
		void AddMissingNodeByInputs(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, TSet<FSignalFlowEntryKey>& OutEntriesMarkedToDelete);
		void GetNextInputsInGraphRecursive(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete, TArray<TSharedPtr<FSignalFlowDashboardEntry>>& OutNextInputs) const;
		void GetNextOutputsInGraphRecursive(const FSignalFlowDashboardEntry& Entry, const TSortedMap<FSignalFlowEntryKey, TSharedPtr<FSignalFlowDashboardEntry>>& DeviceData, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete, TArray<TSharedPtr<FSignalFlowDashboardEntry>>& OutNextOutputs) const;
		bool EntryIsInGraph(const FSignalFlowEntryKey& EntryKey, const TSet<FSignalFlowEntryKey>& EntriesMarkedToDelete) const;

		void CalculateFilteredGraphDepthStructure(TOptional<FSignalFlowEntryKey>& OutRootEntryKey, TArray<TreeDepthPair>& OutSortedTreeDepths) const;

		void CreateDummyInputs(const TArray<TreeDepthPair>& TreeDepthStructure);
		TSharedPtr<FDummyConnectionNode> CreateDummyConnectionNode(const TSharedPtr<FSignalFlowEntryNode>& InputNode, const TSharedPtr<ISignalFlowNode>& OutputNode, const TreeDepthPair& TreeDepthPair, const FSignalFlowEntryKey& ConnectionOutputNodeKey);
		
		void AssignNodeOrderRecursive(TSharedPtr<ISignalFlowNode> OutNode, const int32 Depth, int32& OutNodeProcessingOrder);
		void SortNodeInputs(TSharedPtr<ISignalFlowNode> OutNode);
		TreeDepthPair GetRealTreeDepthForNode(const TSharedPtr<ISignalFlowNode>& Node) const;
		void GetBranchMinTimestampAndTreeDepthRecursive(const TArray<FSignalFlowEntryKey>& NodeKeys, TreeDepthPair& OutMinTreeDepth, double& OutMinTimestamp) const;
		bool NodeIsOwnerOrSound(const ISignalFlowNode& Node) const;

		void FixUnassignedNodeOrders();
		void SortNodeOrder();

		void CalcXPositions(const TSharedPtr<ISignalFlowNode>& GraphRootNode);
		void CalcXPositionsForRows(const TArray<TArray<int32>>& NodeIndexRows, const EOrientation GraphOrientation, int32& OutGraphWidth);
		void CompactRealNodeXPositions();
		void MaintainFocusedNodeXPos(const TSharedPtr<ISignalFlowNode>& FallbackNode);
		void PositionRowEquidistant(const TArray<int32>& NodeIndexRow);
		
		void CalcXPositionForNode(const TSharedPtr<ISignalFlowNode>& Node, const TArray<FSignalFlowEntryKey>& NodesToPositionRelativeTo, const int32 NodeIndexInRow, const TArray<TSharedPtr<ISignalFlowNode>>& PrevPositionedNodesInRow);
		void AlignNodeTowardsEdge(const TSharedPtr<ISignalFlowNode>& Node, const int32 NodeIndexInRow);
		void AlignNodeTowardsCenter(const TSharedPtr<ISignalFlowNode>& Node, const TArray<FSignalFlowEntryKey>& NodesToPositionRelativeTo, const TArray<TSharedPtr<ISignalFlowNode>>& PrevPositionedNodesInRow);

#if WITH_EDITOR
		void OnReadEditorSettings(const FSignalFlowSettings& InSettings);
		void OnWriteEditorSettings(FSignalFlowSettings& OutSettings);
		TSharedPtr<SWidget> OnConstructContextMenu();
#endif // WITH_EDITOR

#if !WITH_EDITOR
		bool IsReadingTraceFile() const;
#endif // !WITH_EDITOR

		TSharedPtr<FSignalFlowTraceProvider> SignalFlowProvider;
		TSharedPtr<SSignalFlowDashboard> DashboardWidget;

		TSharedPtr<SSplitter> CategorySplitter;

		TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> ActiveOwnerObjectsListView;
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> ActiveOwnerObjects;

		TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> ActiveSourcesListView;
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> ActiveSources;

		TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> ActiveBusesListView;
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> ActiveBuses;

		TSharedPtr<SListView<TSharedPtr<FSignalFlowDashboardEntry>>> ActiveSubmixesListView;
		TArray<TSharedPtr<FSignalFlowDashboardEntry>> ActiveSubmixes;

		TSortedMap<FSignalFlowEntryKey, TSharedPtr<ISignalFlowNode>> FilteredEntries;
		TSet<FSignalFlowEntryKey> HighlightedPathKeys;
		TArray<TSharedPtr<ISignalFlowNode>> FilteredGraphNodes;
		TSharedPtr<SSplitter> MainSplitter;
		TSharedPtr<SSignalFlowGraph> GraphWidgetContents;

		TSharedPtr<SSignalFlowScrollBox> GraphHorizontalScrollBox;
		TSharedPtr<SSignalFlowScrollBox> GraphVerticalScrollBox;

		TSharedPtr<FUICommandList> CommandList;

		TSharedPtr<FSignalFlowDashboardEntry> SelectedItem;
		TSharedPtr<FSignalFlowDashboardEntry> HighlightedItem;
		FString SearchBoxFilterText;

		FSignalFlowNodeDetailFilterSettings NodeDetailFilters;
		FSignalFlowEntryTypeListExpansionSettings ActiveEntryMenuExpansionSettings;

		ESignalFlowJustification GraphJustification = ESignalFlowJustification::Edge;
		TOptional<FSignalFlowEntryKey> XPosFocusNodeEntryKey;

		static constexpr int32 MainSplitterSelectionPanelSlotIndex = 0;
		static constexpr int32 MainSplitterGraphSectionSlotIndex = 1;
		float GraphWidthRatio = 0.8f;
		float LargeNodePadding = 128.0f;
		float SmallNodePadding = 8.0f;

		bool bGraphRequiresRefresh = false;
		bool bAutoFocusAfterGraphRefresh = false;
		bool bShowNodeDetails = false;
		bool bPauseGraphOnSelect = false;
		bool bDisplayAmpPeakInDb = true;
		bool bAnimateWires = true;
		bool bSentSelectedNodeDestroyedNotification = false;

#if WITH_EDITOR
		FAssetEditorContextMenuHelper AssetContextMenuHelper;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API
