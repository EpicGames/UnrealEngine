// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Messages/SignalFlowEntryKey.h"
#include "Templates/SharedPointer.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	struct FSignalFlowDashboardEntry;

	// TreeDepthPair - Simple helper used to manage tree depth inside the signal flow graph
	// LayerID - the layer the node sits in
	// TreeDepth - what row inside the layer the node sits in
	struct TreeDepthPair
	{
		ESignalFlowEntryType LayerID = ESignalFlowEntryType::OwnerObject;
		int32 TreeDepth = 0;

		bool operator==(const TreeDepthPair& Other) const
		{
			return LayerID == Other.LayerID && TreeDepth == Other.TreeDepth;
		}

		bool operator<(const TreeDepthPair& Other) const
		{
			if (LayerID != Other.LayerID)
			{
				return static_cast<int32>(LayerID) < static_cast<int32>(Other.LayerID);
			}

			return TreeDepth < Other.TreeDepth;
		}

		bool operator>(const TreeDepthPair& Other) const
		{
			if (LayerID != Other.LayerID)
			{
				return static_cast<int32>(LayerID) > static_cast<int32>(Other.LayerID);
			}

			return TreeDepth > Other.TreeDepth;
		}

		bool operator<=(const TreeDepthPair& Other) const
		{
			return operator<(Other) || operator==(Other);
		}

		bool operator>=(const TreeDepthPair& Other) const
		{
			return operator>(Other) || operator==(Other);
		}

		friend int32 GetTypeHash(const TreeDepthPair& Key)
		{
			return HashCombine(GetTypeHash(static_cast<int32>(Key.LayerID)), GetTypeHash(Key.TreeDepth));
		}
	};

	// ISignalFlowNode : Class that holds data on a node's position, order, inputs, outputs etc.
	// Specifically holds data on the filtered nodes in the graph
	class ISignalFlowNode
	{
	public:
		ISignalFlowNode() = delete;
		ISignalFlowNode(const int32 InTreeDepth, const ESignalFlowEntryType InTreeDepthLayerGroup, const double InTimestamp)
			: TreeDepthPair({ InTreeDepthLayerGroup, InTreeDepth })
			, Timestamp(InTimestamp)
		{
		}

		virtual ~ISignalFlowNode() = default;

		virtual FSignalFlowEntryKey GetEntryKey() const = 0;
		virtual bool IsRealNode() const = 0;

		void SetTreeDepth(const ESignalFlowEntryType InTreeDepthLayerGroup, const int32 InTreeDepth) { TreeDepthPair = { InTreeDepthLayerGroup, InTreeDepth }; }
		const TreeDepthPair& GetTreeDepth() const { return TreeDepthPair; }
		TreeDepthPair& GetTreeDepth() { return TreeDepthPair; }

		void SetNodeOrderID(const int32 InNodeOrderID) { NodeOrderID = InNodeOrderID; }
		void ResetOrderID();
		int32 GetNodeOrderID() const { return NodeOrderID; }
		int32 GetPreviousNodeOrderID() const { return PreviousNodeOrderID; }
		bool NodeOrderIsValid() const { return NodeOrderID != INVALID_NODE_ORDER_ID; }

		double GetTimestamp() const { return Timestamp; }

		TArray<FSignalFlowEntryKey> FilteredInputs;
		TArray<FSignalFlowEntryKey> FilteredOutputs;

		TArray<FSignalFlowEntryKey> FilteredLinkedSoundSources;
		TOptional<FSignalFlowEntryKey> FilteredLinkedSourceBus;

		TArray<FSignalFlowEntryKey> FilteredLinkedBusPatchInputs;   // Entries that send bus patch connections to this node
		TArray<FSignalFlowEntryKey> FilteredLinkedBusPatchOutputs;  // Entries that this node sends bus patch connections to

		float XPos = 0.0f;

	private:
		static constexpr int32 INVALID_NODE_ORDER_ID = -1;

		TreeDepthPair TreeDepthPair;
		int32 NodeOrderID = INVALID_NODE_ORDER_ID;
		int32 PreviousNodeOrderID = INVALID_NODE_ORDER_ID;

		const double Timestamp = 0.0;
	};

	// FSignalFlowEntryNode : A node with a valid entry - directly maps to a FSignalFlowDashboardEntry created inside FSignalFlowTraceProvider
	class FSignalFlowEntryNode : public ISignalFlowNode
	{
	public:
		FSignalFlowEntryNode() = delete;
		FSignalFlowEntryNode(TSharedPtr<FSignalFlowDashboardEntry> InEntry, const int32 InTreeDepth, const ESignalFlowEntryType InTreeDepthLayerGroup, const double InTimestamp)
			: ISignalFlowNode(InTreeDepth, InTreeDepthLayerGroup, InTimestamp)
			, Entry(InEntry)
		{
		}

		UE_API virtual FSignalFlowEntryKey GetEntryKey() const override;

		virtual bool IsRealNode() const override { return true; }

		TSharedPtr<FSignalFlowDashboardEntry> Entry;
	};

	// FDummyConnectionNode : A fake invisible node placed between the input and output of two other nodes
	// Used to help position and order real nodes whilst avoiding nodes getting in each other's way and crossing connections
	class FDummyConnectionNode : public ISignalFlowNode
	{
	public:
		FDummyConnectionNode() = delete;
		FDummyConnectionNode(const FSignalFlowEntryKey& InEntryKey, const int32 InTreeDepth, const ESignalFlowEntryType InTreeDepthLayerGroup, const double InTimestamp)
			: ISignalFlowNode(InTreeDepth, InTreeDepthLayerGroup, InTimestamp)
			, EntryKey(InEntryKey)
		{
		}

		virtual FSignalFlowEntryKey GetEntryKey() const override { return EntryKey; }
		virtual bool IsRealNode() const override { return false; }
	
		FSignalFlowEntryKey ConnectionInputKey;
		FSignalFlowEntryKey ConnectionOutputKey;

	private:
		const FSignalFlowEntryKey EntryKey;
	};
} // namespace UE::Audio::Insights

#undef UE_API