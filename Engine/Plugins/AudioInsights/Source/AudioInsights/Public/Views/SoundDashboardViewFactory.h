// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Settings/SoundDashboardSettings.h"
#include "Settings/VisibleColumnsSettingsMenu.h"
#include "Views/SoundPlotsWidgetView.h"
#include "Views/TreeDashboardViewFactory.h"

#include "Filters/GenericFilter.h"

#define UE_API AUDIOINSIGHTS_API

class FUICommandList;

namespace UE::Audio::Insights
{
	class FSoundDashboardEntry;

	enum class ESoundDashboardFilterFlags : uint32
	{
		None = 0,
		MetaSound = 1 << 0,
		SoundCue  = 1 << 1,
		ProceduralSource = 1 << 2,
		SoundWave = 1 << 3,
		SoundCueTemplate = 1 << 4,
		Pinned = 1 << 5
	};

	ENUM_CLASS_FLAGS(ESoundDashboardFilterFlags);

#if WITH_EDITOR
	enum class EMuteSoloMode : uint8
	{
		Mute,
		Solo
	};
#endif // WITH_EDITOR

	class FSoundDashboardFilter : public FGenericFilter<ESoundDashboardFilterFlags>
	{
	public:
		FSoundDashboardFilter(ESoundDashboardFilterFlags InFlags, 
			const FString& InName, 
			const FText& InDisplayName, 
			const FName& InIconName,
			const FText& InToolTipText, 
			FLinearColor InColor, 
			TSharedPtr<FFilterCategory> InCategory)
			: FGenericFilter<ESoundDashboardFilterFlags>(InCategory, InName, InDisplayName, FGenericFilter<ESoundDashboardFilterFlags>::FOnItemFiltered())
			, Flags(InFlags)
		{
			ToolTip  = InToolTipText;
			Color    = InColor;
			IconName = InIconName;
		}

		bool IsActive() const {	return bIsActive; }
		ESoundDashboardFilterFlags GetFlags() const { return Flags; }

	private:
		virtual void ActiveStateChanged(bool bActive) override { bIsActive = bActive; }
		virtual bool PassesFilter(ESoundDashboardFilterFlags InItem) const override { return EnumHasAnyFlags(InItem, Flags); }

		ESoundDashboardFilterFlags Flags;
		bool bIsActive = false;
	};

	/**
	* Helper class for pinned items in the dashboard tree
	*	- Contains a weak handle to the original entry (OriginalDataEntry) which is updated from the trace provider
	*	- Copies updated params to PinnedSectionEntry for display
	*/
	class FPinnedSoundEntryWrapper
	{
	public:
		FPinnedSoundEntryWrapper() = delete;
		UE_API FPinnedSoundEntryWrapper(const TSharedPtr<IDashboardDataTreeViewEntry>& OriginalEntry);

		TSharedPtr<IDashboardDataTreeViewEntry> GetPinnedSectionEntry() const { return PinnedSectionEntry; }
		TSharedPtr<IDashboardDataTreeViewEntry> GetOriginalDataEntry() const { return OriginalDataEntry.IsValid() ? OriginalDataEntry.Pin() : nullptr; }

		UE_API TSharedPtr<FPinnedSoundEntryWrapper> AddChildEntry(const TSharedPtr<IDashboardDataTreeViewEntry> Child);

		UE_API void UpdateParams();

		UE_API void CleanUp();
		UE_API void MarkToDelete();

		UE_API bool EntryIsValid() const;

		UE_API TSharedPtr<IDashboardDataTreeViewEntry> FindOriginalEntryInChildren(const TSharedPtr<IDashboardDataTreeViewEntry>& RootPinnedEntry);

		TArray<TSharedPtr<FPinnedSoundEntryWrapper>> PinnedWrapperChildren;

	private:
		TSharedPtr<IDashboardDataTreeViewEntry> PinnedSectionEntry;
		TWeakPtr<IDashboardDataTreeViewEntry> OriginalDataEntry;
	};

	class FSoundDashboardViewFactory : public FTraceTreeDashboardViewFactory
	{
	public:
		UE_API FSoundDashboardViewFactory();
		UE_API virtual ~FSoundDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		UE_API virtual void ProcessEntries(FTraceTreeDashboardViewFactory::EProcessReason Reason) override;

		UE_API TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo> GetPlotColumnInfo();

		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnProcessPlotData, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*DataViewEntries*/, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*SelectedEntries*/, const bool /*bForceUpdate*/);
		FOnProcessPlotData OnProcessPlotData;

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdatePlotVisibility, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*DataViewEntries*/);
		FOnUpdatePlotVisibility OnUpdatePlotVisibility;

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdatePlotSelection, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*SelectedEntries*/);
		FOnUpdatePlotSelection OnUpdatePlotSelection;

	private:
		UE_API void BindCommands();

		UE_API virtual TSharedPtr<SWidget> GetFilterBarWidget() override;
		UE_API virtual TSharedPtr<SWidget> GetFilterBarButtonWidget() override;

		UE_API virtual void SetSearchBoxFilterText(const FText& NewText) override;

		UE_API bool IsRootItem(const TSharedRef<IDashboardDataTreeViewEntry>& InEntry) const;

		UE_API TSharedRef<SWidget> GenerateWidgetForRootColumn(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& InColumn, const FText& InValueText);
		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& InColumn) override;
		
		UE_API virtual TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IDashboardDataTreeViewEntry> Item, const TSharedRef<STableViewBase>& OwnerTable) override;

		UE_API virtual TSharedPtr<SWidget> OnConstructContextMenu() override;
		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataTreeViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const override;

		UE_API virtual const TMap<FName, FTraceTreeDashboardViewFactory::FHeaderRowColumnData>& GetHeaderRowColumns() const override;
		UE_API virtual void OnHiddenColumnsListChanged() override;
		UE_API virtual const TMap<FName, FTraceTreeDashboardViewFactory::FColumnData>& GetColumns() const override;

		UE_API virtual void SortTable() override;
		UE_API virtual bool IsColumnSortable(const FName& ColumnId) const override;
		virtual TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& GetTreeItemsSource() override { return FullTree; }
		UE_API virtual bool ResetTreeData() override;

		virtual int32 GetDisplayedTreeDepth() const override { return static_cast<int32>(TreeViewingMode); }
		UE_API virtual bool ShouldAutoExpand(const TSharedPtr<IDashboardDataTreeViewEntry>& Item) const override;

		UE_API void ChangeTreeViewingMode(const ESoundDashboardTreeViewingOptions SelectedMode, const bool bUpdateEditorSettings = true);
		UE_API void BuildViewOptionsMenuContent(FMenuBuilder& MenuBuilder);

		UE_API void ChangeAutoExpandMode(const ESoundDashboardAutoExpandOptions SelectedMode, const bool bUpdateEditorSettings = true);
		UE_API void RefreshExpansionChecks(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);
		UE_API void BuildAutoExpandMenuContent(FMenuBuilder& MenuBuilder);
		UE_API void BuildVisibleColumnsMenuContent(FMenuBuilder& MenuBuilder);

		UE_API void FilterText();
		UE_API void RecursiveSort(TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& OutTree, TFunctionRef<bool(const FSoundDashboardEntry&, const FSoundDashboardEntry&)> Predicate);
		UE_API void SortByPredicate(TFunctionRef<bool(const FSoundDashboardEntry&, const FSoundDashboardEntry&)> Predicate);

		UE_API TSharedRef<SWidget> OnGetSettingsMenuContent();
		UE_API TSharedRef<SWidget> MakeSettingsButtonWidget();

		void UpdateMaxStoppedSoundTimeout();

#if WITH_EDITOR
		void OnReadEditorSettings(const FSoundDashboardSettings& Settings);
		void OnWriteEditorSettings(FSoundDashboardSettings& Settings);

		UE_API TSharedRef<SWidget> MakeMuteSoloWidget();

		UE_API TSharedRef<SWidget> CreateMuteSoloButton(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, 
			const TSharedRef<IDashboardDataTreeViewEntry>& InRowData,
			const FName& InColumn,
			TFunction<void(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*InEntries*/)> MuteSoloToggleFunc,
			TFunctionRef<bool(const IDashboardDataTreeViewEntry& /*InEntry*/, const bool /*bInCheckChildren*/)> IsMuteSoloFunc);

		UE_API void ToggleMuteSoloEntries(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries, const EMuteSoloMode InMuteSoloMode);
		UE_API void MuteSoloFilteredEntries();

		UE_API TArray<TObjectPtr<UObject>> GetSelectedEditableAssets() const;
#endif // WITH_EDITOR

		void OnTimingViewTimeMarkerChanged(double InTimeMarker);
		void HandleOnTimeControlMethodReset();

		UE_API TSharedRef<SWidget> MakeShowPlotWidget();

		UE_API TSharedRef<SWidget> CreateShowPlotColumnButton(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget,
			const TSharedRef<IDashboardDataTreeViewEntry>& InRowData,
			const FName& InColumn,
			TFunction<void(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& /*InEntries*/, const bool /*bActivatePlot*/)> ShowPlotToggleFunc,
			TFunctionRef<bool(const IDashboardDataTreeViewEntry& /*InEntry*/)> IsPlotActiveFunc);

		UE_API void ToggleShowPlotEntries(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InEntries, const bool bShowPlot, const bool bIgnoreCategories = false);
		UE_API void SetShowPlotRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const bool bShowPlot);
		UE_API void SetShowPlot(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const bool bShowPlot);
		void CleanUpParentPlotEnabledStates(const TSet<uint32>& ActiveSoundsToggledOff);
		
		void SetShowPlotVisibleRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);
		void PlotFilteredEntries();
		void ResetPlots();
		void BroadcastPlotUpdate();
		void ProcessPlotData();

		UE_API bool SelectedItemsIncludesAnAsset() const;
		UE_API bool SelectionIncludesUnpinnedItem() const;
		UE_API void ClearSelection();

		UE_API void PinSound();
		UE_API void UnpinSound();

		UE_API void PinSelectedItems(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedItems);
		UE_API void UnpinSelectedItems(const TSharedPtr<FPinnedSoundEntryWrapper>& PinnedWrapperEntry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedItems, const bool bSelectionContainsAssets);

		UE_API void RecreatePinnedEntries(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);

		UE_API void MarkBranchAsPinned(const TSharedPtr<IDashboardDataTreeViewEntry> Entry, const bool bIsPinned);
		UE_API void InitPinnedItemEntries();
		UE_API void CreatePinnedEntry(TSharedPtr<IDashboardDataTreeViewEntry> Entry);
		UE_API void UpdatePinnedSection();
		UE_API void RebuildPinnedSection();

#if WITH_EDITOR
		UE_API void BrowseSoundAsset() const;
		UE_API void OpenSoundAsset() const;
#endif // !WITH_EDITOR

		static constexpr ESoundDashboardFilterFlags AllFilterFlags = 
			ESoundDashboardFilterFlags::MetaSound        |
			ESoundDashboardFilterFlags::SoundCue         |
			ESoundDashboardFilterFlags::ProceduralSource |
			ESoundDashboardFilterFlags::SoundWave        |
			ESoundDashboardFilterFlags::SoundCueTemplate |
			ESoundDashboardFilterFlags::Pinned;

		TSharedPtr<FPinnedSoundEntryWrapper> PinnedItemEntries;
		TArray<TSharedPtr<IDashboardDataTreeViewEntry>> FullTree;

		TSharedPtr<FUICommandList> CommandList;
		TSharedPtr<SWidget> SoundsFilterBar;
		TSharedPtr<SWidget> SoundsFilterBarButton;

		ESoundDashboardTreeViewingOptions TreeViewingMode = ESoundDashboardTreeViewingOptions::FullTree;
		ESoundDashboardAutoExpandOptions AutoExpandMode = ESoundDashboardAutoExpandOptions::Categories;
		ESoundDashboardFilterFlags SelectedFilterFlags = AllFilterFlags;

		float RecentlyStoppedSoundTimeout = 3.0f;

		bool bIsPinnedCategoryFilterEnabled = true;
		bool bPinnedSectionNeedsRebuild = false;
		bool bShowRecentlyStoppedSounds = false;
		bool bTreeViewModeChanged = false;
		bool bDisplayAmpPeakInDb = true;
		bool bIsMuteFilteredMode = false;
		bool bIsSoloFilteredMode = false;
		bool bIsPlotFilteredMode = false;

		TSharedPtr<FVisibleColumnsSettingsMenu<FSoundDashboardVisibleColumns>> VisibleColumnsSettingsMenu;
		TOptional<FSoundDashboardVisibleColumns> InitVisibleColumnSettings;
		FSoundPlotsDashboardPlotRanges DefaultPlotRanges;
	};
} // namespace UE::Audio::Insights

#undef UE_API
