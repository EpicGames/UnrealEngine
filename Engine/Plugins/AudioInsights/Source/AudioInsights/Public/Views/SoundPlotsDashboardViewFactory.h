// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Views/SoundPlotsWidgetView.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FSoundDashboardViewFactory;
	class IDashboardDataTreeViewEntry;

	class FSoundPlotsDashboardViewFactory : public IDashboardViewFactory, public TSharedFromThis<FSoundPlotsDashboardViewFactory>
	{
	public:
		virtual ~FSoundPlotsDashboardViewFactory() = default;

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

		UE_API void InitPlots(const TSharedRef<FSoundDashboardViewFactory> SoundsDashboard);

	private:
		TSharedPtr<FSoundPlotsWidgetView> PlotsWidget;

		void ProcessPlotData(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, const bool bForceUpdate = false);

		void UpdatePlotVisibility(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries);
		void UpdatePlotSelection(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries);

		FDelegateHandle OnUpdatePlotVisibility;
		FDelegateHandle OnUpdatePlotSelection;

		FDelegateHandle OnProcessPlotData;
	};
} // namespace UE::Audio::Insights

#undef UE_API