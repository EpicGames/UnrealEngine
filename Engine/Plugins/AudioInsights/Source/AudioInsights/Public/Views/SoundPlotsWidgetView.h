// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsDataSource.h"
#include "DSP/Dsp.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/Function.h"
#include "Views/SAudioCurveView.h"

#define UE_API AUDIOINSIGHTS_API

template <typename ItemType> class SComboBox;

namespace UE::Audio::Insights
{
	class FSoundPlotsWidgetView : public TSharedFromThis<FSoundPlotsWidgetView>
	{
	public:
		// Main information about what data is being plotted in this widget, along with functions to gather information.
		// This data is passed into the constructor of this widget in a TMap of Column name -> Plot Column Info
		struct FPlotColumnInfo
		{
			const FText ColumnDisplayName;
			const TFunction<const ::Audio::TCircularAudioBuffer<FDataPoint>& (const IDashboardDataTreeViewEntry& InData)> DataFunc;
			const TFunction<float(const float)> UnitConversionFunc;
			const TFunction<const FNumberFormattingOptions*()> GetFormatOptionsFunc;
			const TFunction<const FProperty*()> GetYRangePropertyFunc;
			const TFunction<bool()> GetUseCustomYRangeFunc;
			const TFunction<void(bool)> SetUseCustomYRangeFunc;
			const TFunction<FFloatInterval()> GetCustomYRangeFunc;
			const TFunction<void(const FFloatInterval&, TOptional<ETextCommit::Type>)> SetCustomYRangeFunc;
		};

		UE_API FSoundPlotsWidgetView(TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo>&& InColumnInfo, const TFunction<bool(const IDashboardDataTreeViewEntry& InEntry)> InIsPlotEnabledForEntryFunc);
		UE_API virtual ~FSoundPlotsWidgetView();

		UE_API TSharedRef<SWidget> MakeWidget();
		UE_API void ProcessPlotData(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, const bool bForceUpdate = false);
		UE_API void UpdatePlotVisibility(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& DataViewEntries);

		UE_API void UpdatePlotSelection(const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries);

	private:
		using FPlotCurvePoint = SAudioCurveView::FCurvePoint;
		using FPointDataPerCurveMap = TMap<uint64, TArray<FPlotCurvePoint>>; // Map of source id to data point array 
		using FLatestTimestampPerCurveMap = TMap<uint64, double>; // Map of source id to most recent timestamp associated with any data
		using FPlotCurveMetadata = SAudioCurveView::FCurveMetadata;

		void OnAnalysisStarting(const double Timestamp);
#if WITH_EDITOR
		void OnPIEStarted(bool bSimulating);
		void OnPIEStopped(bool bSimulating);
		void OnPIEPaused(bool bSimulating);
		void OnPIEResumed(bool bSimulating);

		void OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination);
#else
		void OnAudioInsightsComponentTabSpawn();
		void OnSessionAnalysisCompleted();
#endif // WITH_EDITOR

		void OnTimingViewTimeMarkerChanged(double InTimeMarker);
		void OnTimeControlMethodReset();

		FReply OnHandleKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent);

		TSharedRef<SWidget> MakePlotsWidget();
		void ConstructPlotsWidgetContents();
		void BindCommands();
		void ResetInspectTimestamp();

		void CollectDataRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& SelectedEntries, bool& bOutHasNewMetadata);
		void ResetPlots();

		void UpdatePlotVisibilityRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& Entry);

		void UpdatePlot();

		// Column information used by plot widgets, keyed by column name. These keys should be a subset of the keys in GetColumns(). 
		const TFunction<const ::Audio::TCircularAudioBuffer<FDataPoint>& (const IDashboardDataTreeViewEntry& InEntry)> GetPlotColumnDataFunc(const FName& ColumnName);
		const TFunction<float (const float)> GetPlotColumnUnitConversionFunc(const FName& ColumnName);
		const FNumberFormattingOptions* GetPlotColumnNumberFormat(const FName& ColumnName);
		FFloatInterval GetPlotColumnCustomizedOrDefaultYRange(const FName& ColumnName);
		const FText GetPlotColumnDisplayName(const FName& ColumnName);
		FText GetAdditionalTooltipText() const;

		TOptional<float> GetPlotColumnYRangeFloatMetaData(const FName& ColumnName, const FName& MetaDataKey);
		TOptional<FFloatInterval> GetPlotColumnCustomizedYRange(const FName& ColumnName);
		bool GetPlotColumnUseCustomYRange(const FName& ColumnName);
		void SetPlotColumnUseCustomYRange(const FName& ColumnName, bool bUseCustomYRange);
		void SetPlotColumnCustomYRangeMin(const FName& ColumnName, float Value, TOptional<ETextCommit::Type> CommitType = NullOpt);
		void SetPlotColumnCustomYRangeMax(const FName& ColumnName, float Value, TOptional<ETextCommit::Type> CommitType = NullOpt);
		void SetTimestampFromCurveView(const double Timestamp);

		TRange<double> GetViewRange();

		bool GetIsPlotEnabledForEntry(const IDashboardDataTreeViewEntry& InEntry) const;

		// Curve points per timestamp per source id per column name 
		TMap<FName, TSharedPtr<FPointDataPerCurveMap>> PlotWidgetCurveIdToPointDataMapPerColumn;

		// Map to track the latest timestamp for each column's data points.
		// Used to avoid adding duplicate data to the plot if this widget view is updated at a higher rate than the data is received
		TMap<FName, TSharedPtr<FLatestTimestampPerCurveMap>> ColumnNameToLatestTimestamp;

		// SourceId to metadata for the corresponding curve
		TSharedPtr<TMap<uint64, FPlotCurveMetadata>> PlotWidgetMetadataPerCurve;

		// Column names for plot selector widget 
		TArray<FName> ColumnNames;

		// Func to access column data from the dashboard using this Plots view
		const TMap<FName, FSoundPlotsWidgetView::FPlotColumnInfo> ColumnInfo;

		// Func to enquire whether plotting is enabled for an entry
		const TFunction<bool(const IDashboardDataTreeViewEntry& InEntry)> IsPlotEnabledForEntryFunc;

		double BeginTimestamp = TNumericLimits<double>::Max();
		double CurrentTimestamp = TNumericLimits<double>::Lowest();

		TRange<double> CurrentRange { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };

		enum class EGameState : uint8
		{
			Running,
			Stopped,
			Paused,
			LoadingTraceFile,
			ControlledByTimemarker
		};

		EGameState GameState = EGameState::Stopped;

		TSharedPtr<SAudioCurveView> PlotWidget;
		TSharedPtr<SComboBox<FName>> ComboBox;
		FName SelectedPlotColumnName;

		TSharedPtr<FUICommandList> CommandList;

#if !WITH_EDITOR
		double PreviousTime = 0.0;
		double CurrentRangeUpperBound = 0.0;
#endif // !WITH_EDITOR

		bool bTimeMarkerMoved = false;

#if WITH_EDITOR
		bool bIsTraceActive = false;
		bool bIsPIEPaused = false;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API
