// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Views/TableDashboardViewFactory.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	class FSubmixTraceProvider;
	struct FSubmixDashboardEntry;

	class FSubmixDashboardViewFactory : public FTraceObjectTableDashboardViewFactory
	{
	public:
		UE_API FSubmixDashboardViewFactory();
		UE_API virtual ~FSubmixDashboardViewFactory();

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

		UE_API static void SendSubmixEnvelopeFollowerCVar(const bool bEnableEnvelopeFollower, const FSubmixDashboardEntry& SubmixDashboardEntry);

	protected:
		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& InColumnName) override;
		UE_API virtual void ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason) override;
		UE_API virtual const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& GetColumns() const override;

		UE_API void RebuildAudioMeters() const;
		UE_API bool ShouldUpdateAudioMeter(const FSubmixDashboardEntry& InSubmixDashboardEntry) const;
		UE_API void UpdateAudioMeters() const;
		UE_API void ReactivateAudioMeters();

		UE_API void SetSubmixEnvelopeFollowerEnabled(const FSubmixDashboardEntry& InSubmixDashboardEntry, const bool bInEnabled);

		UE_API virtual void SortTable() override;

		UE_API void HandleOnSubmixAdded(const uint32 InSubmixId);
		UE_API void HandleOnSubmixRemoved(const uint32 InSubmixId);
		UE_API void HandleOnSubmixLoaded(const uint32 InSubmixId);
		UE_API void HandleOnTimeMarkerUpdated();
		UE_API void HandleOnTimeControlMethodReset();

		UE_API void RequestListRefresh();

		TSharedPtr<FSubmixTraceProvider> SubmixProvider;

		TSet<uint32> CheckedSubmixes;
		TSet<uint32> PendingToReactivateSubmixIds;

		bool bShouldRebuildAudioMeters = false;
		bool bListRefreshed = false;
		bool bShouldReactivateAudioMeters = false;
	};
} // namespace UE::Audio::Insights

#undef UE_API
