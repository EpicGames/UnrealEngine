// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsDataSource.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define UE_API AUDIOINSIGHTS_API

namespace UE::Audio::Insights
{
	
	template <typename ItemType>
	class SScrollableListView : public SListView<ItemType>
	{
	public:
		void Construct(const typename SListView<ItemType>::FArguments& InArgs)
		{
			SListView<ItemType>::Construct(InArgs);
		}

		virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			OnMouseWheelDetected.Broadcast();

			return SListView<ItemType>::OnMouseWheel(MyGeometry, MouseEvent);
		}

		DECLARE_MULTICAST_DELEGATE(FOnMouseWheel);
		FOnMouseWheel OnMouseWheelDetected;
	};

	class IObjectDashboardEntry : public IDashboardDataViewEntry
	{
	public:
		virtual ~IObjectDashboardEntry() = default;

		virtual FText GetDisplayName() const = 0;
		virtual FString GetObjectPath() const { return FString(); }
		virtual const UObject* GetObject() const = 0;
		virtual UObject* GetObject() = 0;
	};

	struct FSoundAssetDashboardEntry : public IObjectDashboardEntry
	{
		virtual ~FSoundAssetDashboardEntry() = default;

		UE_API virtual FText GetDisplayName() const override;
		virtual FString GetObjectPath() const override { return Name; }
		UE_API virtual const UObject* GetObject() const override;
		UE_API virtual UObject* GetObject() override;
		UE_API virtual bool IsValid() const override;

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 PlayOrder = INDEX_NONE;
		uint64 ComponentId = TNumericLimits<uint64>::Max();
		double Timestamp = 0.0;
		FString Name;
	};

	class FTraceTableDashboardViewFactory : public FTraceDashboardViewFactoryBase, public TSharedFromThis<FTraceTableDashboardViewFactory>
	{
	public:
		UE_API FTraceTableDashboardViewFactory();
		virtual ~FTraceTableDashboardViewFactory() = default;

	protected:
		struct SRowWidget : public SMultiColumnTableRow<TSharedPtr<IDashboardDataViewEntry>>
		{
			SLATE_BEGIN_ARGS(SRowWidget)
				: _Style(nullptr)
				{}
				SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<IDashboardDataViewEntry> InData, TSharedRef<FTraceTableDashboardViewFactory> InFactory);
			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& Column) override;

			TSharedPtr<IDashboardDataViewEntry> Data;
			TSharedPtr<FTraceTableDashboardViewFactory> Factory;
		};

	public:
		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column);
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		UE_API virtual void RefreshFilteredEntriesListView() override;

	protected:
		template<typename TableProviderType>
		bool FilterEntries(TFunctionRef<bool(const IDashboardDataViewEntry&)> IsFiltered)
		{
			const TSharedPtr<const TableProviderType> Provider = FindProvider<const TableProviderType>();
			if (Provider.IsValid())
			{
				if (const typename TableProviderType::FDeviceData* DeviceData = Provider->FindFilteredDeviceData())
				{
					DataViewEntries.Reset();

					auto TransformEntry = [](const typename TableProviderType::FEntryPair& Pair)
					{
						return StaticCastSharedPtr<IDashboardDataViewEntry>(Pair.Value);
					};

					if (SearchBoxFilterText.IsEmpty() && !FilterBarButton.IsValid())
					{
						Algo::Transform(*DeviceData, DataViewEntries, TransformEntry);
					}
					else
					{
						auto FilterEntry = [this, &IsFiltered](const typename TableProviderType::FEntryPair& Pair)
						{
							return IsFiltered((const IDashboardDataViewEntry&)(*Pair.Value));
						};
						Algo::TransformIf(*DeviceData, DataViewEntries, FilterEntry, TransformEntry);
					}
					
					// Sort list
					RequestSort();

					return true;
				}
				else
				{
					if (!DataViewEntries.IsEmpty())
					{
						DataViewEntries.Empty();
						return true;
					}
				}
			}

			return false;
		}

		template <typename TDataViewEntry>
		void SortByPredicate(TFunctionRef<bool(const TDataViewEntry&, const TDataViewEntry&)> Predicate)
		{
			auto SortDashboardEntries = [this](const TSharedPtr<IDashboardDataViewEntry>& First, const TSharedPtr<IDashboardDataViewEntry>& Second, TFunctionRef<bool(const TDataViewEntry&, const TDataViewEntry&)> Predicate_Internal)
				{
					return Predicate_Internal(CastEntry<TDataViewEntry>(*First), CastEntry<TDataViewEntry>(*Second));
				};

			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					return SortDashboardEntries(A, B, Predicate);
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([&SortDashboardEntries, &Predicate](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					return SortDashboardEntries(B, A, Predicate);
				});
			}
		}

#if WITH_EDITOR
		virtual bool IsDebugDrawEnabled() const { return false; }
		virtual void DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const { };
#endif // WITH_EDITOR

		struct FColumnData
		{
			const FText DisplayName;
			const TFunction<FText(const IDashboardDataViewEntry&)> GetDisplayValue;
			const TFunction<FName(const IDashboardDataViewEntry&)> GetIconName;
			bool bDefaultHidden = false;
			const float FillWidth = 1.0f;
			const EHorizontalAlignment Alignment = HAlign_Left;
			const TFunction<FLinearColor(const IDashboardDataViewEntry&)> GetIconColor = nullptr;
			const TFunction<FText(const IDashboardDataViewEntry&)> GetIconTooltip = nullptr;
			const FName HeaderRowIconName = NAME_None;
			const FText HeaderRowTooltip = FText::GetEmpty();
			const bool bShowDisplayName = true;
		};

		virtual const TMap<FName, FColumnData>& GetColumns() const = 0;

		UE_API const FText& GetSearchFilterText() const;

		virtual void SortTable() = 0;
		virtual bool IsColumnSortable(const FName& ColumnId) const { return true; }

		virtual void OnHiddenColumnsListChanged() { }

		virtual TSharedPtr<SWidget> CreateFilterBarButtonWidget() { return nullptr; }
		virtual TSharedRef<SWidget> CreateFilterBarWidget() { return SNullWidget::NullWidget; }

		UE_API virtual TSharedRef<SWidget> CreateSettingsButtonWidget();
		virtual TSharedRef<SWidget> OnGetSettingsMenuContent() { return SNullWidget::NullWidget; }
		virtual FText OnGetSettingsMenuToolTip() { return FText::GetEmpty(); }

		UE_API virtual TSharedPtr<SWidget> OnConstructContextMenu();
		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo);
		virtual bool ClearSelectionOnClick() const { return false; }

		virtual bool EnableHorizontalScrollBox() const { return true; }
		virtual void OnListViewScrolled(double InScrollOffset) { }
		virtual void OnFinishedScrolling() {}

		virtual const FTableRowStyle* GetRowStyle() const { return nullptr; }
		UE_API virtual FSlateColor GetRowColor(const TSharedPtr<IDashboardDataViewEntry>& InRowDataPtr);

		FDelegateHandle OnEntriesUpdatedHandle;

		TArray<TSharedPtr<IDashboardDataViewEntry>> DataViewEntries;

		TSharedPtr<SWidget> DashboardWidget;
		TSharedPtr<SHeaderRow> HeaderRowWidget;
		TSharedPtr<SScrollableListView<TSharedPtr<IDashboardDataViewEntry>>> FilteredEntriesListView;
		TSharedPtr<SWidget> FilterBarButton;
		TSharedPtr<SSearchBox> SearchBoxWidget;

		FName SortByColumn;
		EColumnSortMode::Type SortMode = EColumnSortMode::None;

	private:
#if WITH_EDITOR
		UE_API virtual void UpdateDebugDraw(const float DeltaTime) override;
#endif // WITH_EDITOR

		UE_API TSharedRef<SHeaderRow> MakeHeaderRowWidget();
		UE_API void SetSearchBoxFilterText(const FText& NewText);

		UE_API EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;
		UE_API void RequestSort();
		UE_API void OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

		TSharedRef<SWidget> CreateHorizontalScrollBox();

		FText SearchBoxFilterText;
	};

	class FTraceObjectTableDashboardViewFactory : public FTraceTableDashboardViewFactory
	{
	public:
		virtual ~FTraceObjectTableDashboardViewFactory() = default;

		UE_API virtual TSharedRef<SWidget> GenerateWidgetForColumn(TSharedRef<IDashboardDataViewEntry> InRowData, const FName& Column) override;
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const override;
		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;

	protected:
#if WITH_EDITOR
		UE_API virtual TSharedRef<SWidget> MakeAssetMenuBar() const;

		UE_API TArray<UObject*> GetSelectedEditableAssets() const;

		UE_API bool OpenAsset() const;
		UE_API bool BrowseToAsset() const;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API