// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsDataSource.h"
#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API AUDIOINSIGHTS_API

class SHeaderRow;
class SSearchBox;
class STableViewBase;

namespace UE::Audio::Insights
{
	/**
	 * Tree view entries can inherit from this class to implement extra UObject functionality (ex: open, browse, edit, etc.) 
	 */
	class IObjectTreeDashboardEntry : public IDashboardDataTreeViewEntry
	{
	public:
		virtual ~IObjectTreeDashboardEntry() = default;

		virtual TObjectPtr<UObject> GetObject() = 0;
		virtual const TObjectPtr<UObject> GetObject() const = 0;
	};

	/** 
	 * Inherit from this class to create a tree view dashboard for Audio Insights.
	 * It contains a search textbox, filters can be optionally be implemented via GetFilterBarWidget and GetFilterBarButtonWidget.
	 * Item actions can be done via OnSelectionChanged, OnDataRowKeyInput, OnConstructContextMenu (for right click)
	 */
	class FTraceTreeDashboardViewFactory : public FTraceDashboardViewFactoryBase, public TSharedFromThis<FTraceTreeDashboardViewFactory>
	{
	public:
		UE_API FTraceTreeDashboardViewFactory();
		virtual ~FTraceTreeDashboardViewFactory() = default;

	protected:
		struct SRowWidget : public SMultiColumnTableRow<TSharedPtr<IDashboardDataTreeViewEntry>>
		{
			SLATE_BEGIN_ARGS(SRowWidget) 
				: _RenderOpacity(1.0f)
			{}
				SLATE_ATTRIBUTE(float, RenderOpacity)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<IDashboardDataTreeViewEntry> InData, TSharedRef<FTraceTreeDashboardViewFactory> InFactory);
			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& Column) override;
			virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

			TSharedPtr<IDashboardDataTreeViewEntry> Data;
			TSharedPtr<FTraceTreeDashboardViewFactory> Factory;
			TAttribute<float> RenderOpacity;
		};

		struct FHeaderRowColumnData
		{
			const FText DisplayName;
			const FName IconName;
			const FText TooltipText = FText::GetEmpty();
			const bool bShowDisplayName = true;
			const bool bDefaultHidden = false;
			const float FillWidth = 1.0f;
			const TOptional<float> FixedWidth;
			const EHorizontalAlignment Alignment = HAlign_Left;
		};

		struct FColumnData
		{
			const TFunction<FText(const IDashboardDataTreeViewEntry&)> GetDisplayValue;
			const TFunction<FName(const IDashboardDataTreeViewEntry&)> GetIconName;
			const TFunction<FSlateColor(const IDashboardDataTreeViewEntry&)> GetTextColorValue;
		};

		UE_API virtual TSharedPtr<SWidget> GetFilterBarWidget();
		UE_API virtual TSharedPtr<SWidget> GetFilterBarButtonWidget();

		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const TSharedRef<FTraceTreeDashboardViewFactory::SRowWidget>& InRowWidget, const TSharedRef<IDashboardDataTreeViewEntry>& InRowData, const FName& Column) = 0;
		
		UE_API virtual TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IDashboardDataTreeViewEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);

		UE_API virtual TSharedPtr<SWidget> OnConstructContextMenu();

		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataTreeViewEntry> SelectedItem, ESelectInfo::Type SelectInfo);
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const;

		UE_API const FText& GetSearchFilterText() const;
		UE_API virtual void SetSearchBoxFilterText(const FText& NewText);

		UE_API virtual void RefreshFilteredEntriesListView() override;

		virtual const TMap<FName, FHeaderRowColumnData>& GetHeaderRowColumns() const = 0;
		virtual void OnHiddenColumnsListChanged() {};
		virtual const TMap<FName, FColumnData>& GetColumns() const = 0;

		virtual void SortTable() = 0;
		virtual bool IsColumnSortable(const FName& ColumnId) const { return true; }

		virtual TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& GetTreeItemsSource() { return DataViewEntries; }

		virtual int32 GetDisplayedTreeDepth() const { return 0; }

		virtual bool ShouldAutoExpand(const TSharedPtr<IDashboardDataTreeViewEntry>& Item) const { return true; }

		virtual bool ResetTreeData()
		{
			if (!DataViewEntries.IsEmpty())
			{
				DataViewEntries.Empty();
				return true;
			}

			return false;
		}

#if WITH_EDITOR
		virtual bool IsDebugDrawEnabled() const { return false; }
		virtual void DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataTreeViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const {};
#endif // WITH_EDITOR

		template<typename TableProviderType>
		bool FilterEntries(TFunctionRef<bool(IDashboardDataTreeViewEntry&)> InPredicate)
		{
			const TSharedPtr<const TableProviderType> Provider = FindProvider<const TableProviderType>();
			if (Provider.IsValid())
			{
				if (const typename TableProviderType::FDeviceData* DeviceData = Provider->FindFilteredDeviceData())
				{
					DataViewEntries.Reset();

					// Filter Entries
					const auto TransformEntry = [](const typename TableProviderType::FEntryPair& Pair)
					{
						return StaticCastSharedPtr<IDashboardDataTreeViewEntry>(Pair.Value);
					};

					const auto FilterEntry = [this, &InPredicate](const typename TableProviderType::FEntryPair& Pair)
					{
						return InPredicate(*Pair.Value);
					};

					const int32 DepthToDisplay = GetDisplayedTreeDepth();
					if (DepthToDisplay == 0)
					{
						Algo::TransformIf(*DeviceData, DataViewEntries, FilterEntry, TransformEntry);
					}
					else
					{
						for (const typename TableProviderType::FEntryPair& Pair : *DeviceData)
						{
							TransformDataRecursive(TransformEntry(Pair), InPredicate, DepthToDisplay);
						}
					}

					// Sort list
					RequestSort();

					return true;
				}
				else
				{
					return ResetTreeData();
				}
			}

			return false;
		}

		TArray<TSharedPtr<IDashboardDataTreeViewEntry>> DataViewEntries;

		TSharedPtr<SWidget> DashboardWidget;
		TSharedPtr<SHeaderRow> HeaderRowWidget;
		TSharedPtr<STreeView<TSharedPtr<IDashboardDataTreeViewEntry>>> FilteredEntriesListView;

		FName SortByColumn;
		EColumnSortMode::Type SortMode = EColumnSortMode::None;

	private:
#if WITH_EDITOR
		virtual void UpdateDebugDraw(const float DeltaTime) override;
#endif // WITH_EDITOR

		UE_API TSharedRef<SHeaderRow> MakeHeaderRowWidget();

		UE_API EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;
		UE_API void RequestSort();
		UE_API void OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);
		UE_API void HandleRecursiveExpansion(TSharedPtr<IDashboardDataTreeViewEntry> Item, bool bIsItemExpanded);

		void TransformDataRecursive(const TSharedPtr<IDashboardDataTreeViewEntry>& OriginalEntry, TFunctionRef<bool(IDashboardDataTreeViewEntry&)> FilterPredicate, const int32 DepthToDisplayFrom, const bool bParentPassedFilter = true);

		TSharedPtr<SSearchBox> SearchBoxWidget;
		FText SearchBoxFilterText;
	};
} // namespace UE::Audio::Insights

#undef UE_API