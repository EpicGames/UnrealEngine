// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Views/TableDashboardViewFactory.h"

#if WITH_EDITOR
#include "Views/AssetEditorContextMenuHelper.h"
#endif // WITH_EDITOR

#define UE_API AUDIOINSIGHTS_API


namespace UE::Audio::Insights
{
	class FVirtualLoopDashboardViewFactory : public FTraceObjectTableDashboardViewFactory
	{
	public:
		UE_API FVirtualLoopDashboardViewFactory();
		virtual ~FVirtualLoopDashboardViewFactory() = default;

		UE_API virtual FName GetName() const override;
		UE_API virtual FText GetDisplayName() const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;

		UE_API virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> InOwnerTab, const FSpawnTabArgs& InSpawnTabArgs) override;

#if WITH_EDITOR
		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnDebugDrawEntries, float /*InElapsed*/, const TArray<TSharedPtr<IDashboardDataViewEntry>>& /*InSelectedItems*/, ::Audio::FDeviceId /*InAudioDeviceId*/);
		UE_API inline static FOnDebugDrawEntries OnDebugDrawEntries;

		UE_API virtual TSharedPtr<SWidget> OnConstructContextMenu() override;
		UE_API virtual void OnSelectionChanged(TSharedPtr<IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;
		UE_API virtual FReply OnDataRowKeyInput(const FGeometry& Geometry, const FKeyEvent& KeyEvent) const override;
#endif // WITH_EDITOR

	protected:
		UE_API virtual void ProcessEntries(FTraceTableDashboardViewFactory::EProcessReason Reason) override;
		UE_API virtual const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& GetColumns() const override;

		UE_API virtual void SortTable() override;

#if WITH_EDITOR
		UE_API virtual bool IsDebugDrawEnabled() const override;
		UE_API virtual void DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const;
#endif // WITH_EDITOR

	private:
#if WITH_EDITOR
		FAssetEditorContextMenuHelper AssetContextMenuHelper;
#endif // WITH_EDITOR
	};
} // namespace UE::Audio::Insights

#undef UE_API
