// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioInsightsEditor : ModuleRules
{
	public AudioInsightsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioMixerCore",
				"Core",
			}
		);	
		
		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioInsights",
				"AudioMixer",
				"AudioWidgets",
				"AudioWidgetsCore",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"InputCore",
				"OutputLog",
				"Projects",
				"SignalProcessing",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
				"TraceLog",
				"UnrealEd",
				"WorkspaceMenuStructure",
			}
		);
	}
}
