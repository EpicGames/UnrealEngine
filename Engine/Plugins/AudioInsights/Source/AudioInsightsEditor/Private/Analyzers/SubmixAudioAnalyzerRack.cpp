// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubmixAudioAnalyzerRack.h"

#include "AudioAnalyzerRack.h"
#include "AudioDeviceManager.h"
#include "AudioInsightsEditorModule.h"
#include "AudioInsightsEditorSettings.h"
#include "AudioInsightsStyle.h"
#include "AudioLoudnessMeter.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMeter.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioVectorscopePanelStyle.h"
#include "AudioWidgetsStyle.h"

namespace UE::Audio::Insights
{
	namespace FSubmixAudioAnalyzerRackPrivate
	{
		/**
		 * This style set is given to the AudioWidgets::FAudioAnalyzerRack to override the parent AudioWidgetsStyle.
		 */
		class FAnalyzerRackStyleSet final : public FSlateStyleSet
		{
		public:
			static FAnalyzerRackStyleSet& Get()
			{
				static FAnalyzerRackStyleSet Instance;
				return Instance;
			}

			FAnalyzerRackStyleSet()
				: FSlateStyleSet("AudioInsightsAnalyzerRackStyleSet")
			{
				SetParentStyleName(FAudioWidgetsStyle::Get().GetStyleSetName());

				const FLinearColor AnalyzerForegroundColor(0.025719f, 0.208333f, 0.069907f, 1.0f); // "Audio" Green

				// Override colors for these widget styles:

				FAudioMeterDefaultColorStyle MeterStyle;
				MeterStyle.MeterValueColor = AnalyzerForegroundColor;
				Set("AudioMeter.DefaultColorStyle", MeterStyle);

				Set("AudioOscilloscope.PanelStyle", FAudioOscilloscopePanelStyle()
					.SetWaveViewerStyle(FSampledSequenceViewerStyle()
						.SetSequenceColor(AnalyzerForegroundColor)));

				Set("AudioSpectrumPlot.Style", FAudioSpectrumPlotStyle()
					.SetCrosshairColor(FSlateColor(AnalyzerForegroundColor).UseSubduedForeground())
					.SetSpectrumColor(AnalyzerForegroundColor));

				Set("AudioVectorscope.PanelStyle", FAudioVectorscopePanelStyle()
					.SetVectorViewerStyle(FSampledSequenceVectorViewerStyle()
						.SetLineColor(AnalyzerForegroundColor)));
			}

		protected:
			virtual const FSlateWidgetStyle* GetWidgetStyleInternal(const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const override
			{
				if (DesiredTypeName == FAudioMaterialMeterStyle::TypeName)
				{
					// Return null for this type to disable use of the audio material meter.
					ensure(!bWarnIfNotFound);
					return nullptr;
				}

				return FSlateStyleSet::GetWidgetStyleInternal(DesiredTypeName, StyleName, DefaultStyle, bWarnIfNotFound);
			}
		};

		TSharedRef<AudioWidgets::FAudioAnalyzerRack> CreateAudioAnalyzerRack()
		{
			using namespace AudioWidgets;

			// Set params so that rack layout is stored specific to Audio Insights and the custom style set is used for analyzer widgets:
			const FAudioAnalyzerRack::FRackConstructParams Params
			{
				.TabManagerLayoutName = TEXT("AudioInsights_FAudioAnalyzerRack_v0"),
				.StyleSet = &FAnalyzerRackStyleSet::Get(),
				.EditorSettingsClass = UAudioInsightsEditorSettings::StaticClass(),
				.ExcludedRackUnitTypes = { AudioWidgets::FAudioLoudnessMeter::RackUnitTypeInfo.TypeName, AudioWidgets::FAudioMeter::RackUnitTypeInfo.TypeName },
			};

			return MakeShared<FAudioAnalyzerRack>(Params);
		}
	} // namespace FSubmixAudioAnalyzerRackPrivate

	FSubmixAudioAnalyzerRack::FSubmixAudioAnalyzerRack(TWeakObjectPtr<USoundSubmix> InSoundSubmix)
		: AudioAnalyzerRack(FSubmixAudioAnalyzerRackPrivate::CreateAudioAnalyzerRack())
	{
		RebuildAudioAnalyzerRack(InSoundSubmix);
	}

	FSubmixAudioAnalyzerRack::~FSubmixAudioAnalyzerRack()
	{
		CleanupAudioAnalyzerRack();
	}

	TSharedRef<SWidget> FSubmixAudioAnalyzerRack::MakeWidget(TSharedRef<SDockTab> InOwnerTab, const FSpawnTabArgs& InSpawnTabArgs)
	{
		return AudioAnalyzerRack->CreateWidget(InOwnerTab, InSpawnTabArgs);
	}

	void FSubmixAudioAnalyzerRack::RebuildAudioAnalyzerRack(TWeakObjectPtr<USoundSubmix> InSoundSubmix)
	{
		using namespace ::Audio;

		if (SoundSubmix.IsValid())
		{
			CleanupAudioAnalyzerRack();
		}

		SoundSubmix = InSoundSubmix;

		const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (!AudioDeviceManager)
		{
			return;
		}

		const FDeviceId AudioDeviceId = FAudioInsightsEditorModule::GetChecked().GetDeviceId();

		const FMixerDevice* MixerDevice = static_cast<const FMixerDevice*>(AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId));
		if (!MixerDevice)
		{
			return;
		}

		if (!SoundSubmix.IsValid())
		{
			return;
		}

		FMixerSubmixWeakPtr MixerSubmixWeakPtr = MixerDevice->GetSubmixInstance(SoundSubmix.Get());
		if (!MixerSubmixWeakPtr.IsValid())
		{
			return;
		}

		AudioAnalyzerRack->Init(MixerDevice->GetNumDeviceChannels(), AudioDeviceId);

		// Start processing
		AudioAnalyzerRack->StartProcessing();

		// Register audio bus in submix
		const TObjectPtr<UAudioBus> AudioBus = AudioAnalyzerRack->GetAudioBus();
		if (!AudioBus)
		{
			return;
		}

		const FAudioBusKey AudioBusKey(AudioBus->GetUniqueID());
		const int32 AudioBusNumChannels = AudioBus->GetNumChannels();

		FAudioThread::RunCommandOnAudioThread([MixerDevice, MixerSubmixWeakPtr, AudioBusKey, AudioBusNumChannels]()
		{
			TObjectPtr<UAudioBusSubsystem> AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
			check(AudioBusSubsystem);

			if (FMixerSubmixPtr MixerSubmix = MixerSubmixWeakPtr.Pin();
				MixerSubmix.IsValid())
			{
				MixerSubmix->RegisterAudioBus(AudioBusKey, AudioBusSubsystem->AddPatchInputForAudioBus(AudioBusKey, MixerDevice->GetNumOutputFrames(), AudioBusNumChannels));
			}
		});
	}

	void FSubmixAudioAnalyzerRack::CleanupAudioAnalyzerRack()
	{
		using namespace ::Audio;

		const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (!AudioDeviceManager)
		{
			return;
		}

		const FDeviceId AudioDeviceId = FAudioInsightsEditorModule::GetChecked().GetDeviceId();

		const FMixerDevice* MixerDevice = static_cast<const FMixerDevice*>(AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId));
		if (!MixerDevice)
		{
			return;
		}

		if (!SoundSubmix.IsValid())
		{
			return;
		}

		FMixerSubmixWeakPtr MixerSubmixWeakPtr = MixerDevice->GetSubmixInstance(SoundSubmix.Get());
		if (!MixerSubmixWeakPtr.IsValid())
		{
			return;
		}

		// Unregister audio bus from submix
		const TObjectPtr<UAudioBus> AudioBus = AudioAnalyzerRack->GetAudioBus();
		if (!AudioBus)
		{
			return;
		}

		const FAudioBusKey AudioBusKey(AudioBus->GetUniqueID());

		FAudioThread::RunCommandOnAudioThread([MixerSubmixWeakPtr, AudioBusKey]()
		{
			if (FMixerSubmixPtr MixerSubmix = MixerSubmixWeakPtr.Pin();
				MixerSubmix.IsValid())
			{
				MixerSubmix->UnregisterAudioBus(AudioBusKey);
			}
		});

		// Stop processing
		AudioAnalyzerRack->StopProcessing();

		SoundSubmix.Reset();
	}
} // namespace UE::Audio::Insights
