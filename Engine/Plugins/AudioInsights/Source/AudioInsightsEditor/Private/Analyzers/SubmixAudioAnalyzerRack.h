// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

namespace AudioWidgets { class FAudioAnalyzerRack; }

class FSpawnTabArgs;
class SDockTab;
class SWidget;
class USoundSubmix;

namespace UE::Audio::Insights
{
	class FSubmixAudioAnalyzerRack : public TSharedFromThis<FSubmixAudioAnalyzerRack>
	{
	public:
		FSubmixAudioAnalyzerRack(TWeakObjectPtr<USoundSubmix> InSoundSubmix);
		virtual ~FSubmixAudioAnalyzerRack();

		TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> InOwnerTab, const FSpawnTabArgs& InSpawnTabArgs);

		void RebuildAudioAnalyzerRack(TWeakObjectPtr<USoundSubmix> InSoundSubmix);

	private:
		void CleanupAudioAnalyzerRack();

		TSharedRef<AudioWidgets::FAudioAnalyzerRack> AudioAnalyzerRack;
		TWeakObjectPtr<USoundSubmix> SoundSubmix;
	};
} // namespace UE::Audio::Insights
