// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsEditorDashboardFactory.h"

#include "Async/Async.h"
#include "Audio/AudioTraceUtil.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioInsightsDashboardAssetCommands.h"
#include "AudioInsightsDashboardFactory.h"
#include "AudioInsightsEditorModule.h"
#include "AudioInsightsEditorSettings.h"
#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceModule.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAudioInsightsModuleInterface.h"
#include "Internationalization/Text.h"
#include "IPropertyTypeCustomization.h"
#include "ISettingsModule.h"
#include "Messages/BookmarkTraceMessages.h"
#include "Templates/SharedPointer.h"
#include "Trace/Trace.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	namespace EditorDashboardFactoryPrivate
	{
		static const FText ToolNameText = LOCTEXT("AudioDashboardEditor_ToolName_Text", "Audio Insights");

		static const FText PreviewDeviceText = LOCTEXT("AudioDashboardEditor_PreviewDevice_Text", "[Preview Audio]");

		static const FText MainMenuFileText = LOCTEXT("AudioDashboardEditor_MainMenu_File_Text", "File");
		static const FText MainMenuViewText = LOCTEXT("AudioDashboardEditor_MainMenu_View_Text", "View");

		static const FText FileCloseText = LOCTEXT("AudioDashboardEditor_File_Close_Text", "Close");
		static const FText FileCloseTooltipText = LOCTEXT("AudioDashboardEditor_File_Close_TooltipText", "Closes the Audio Insights dashboard.");

		static const FText ViewResetLayoutText = LOCTEXT("AudioDashboardEditor_View_ResetLayout_Text", "Reset Layout");

		static const FText OnlyTraceAudioChannelsText = LOCTEXT("AudioDashboardEditor_OnlyTraceAudioChannels_Text", "Audio Only Tracing");
		static const FText OnlyTraceAudioChannelsTooltipText = LOCTEXT("AudioDashboardEditor_OnlyTraceAudioChannels_TooltipText", "When trace monitoring or recording is started from Audio Insights, only the Audio, Audio Mixer and Bookmark channels are enabled, reducing RAM usage and trace file size.\nIf a trace session is already active, Audio Insights channels are simply added to the existing session.");

		static const FText StartMonitoringText = LOCTEXT("AudioDashboardEditor_StartMonitoring_Text", "Start Monitoring");
		static const FText StartMonitoringTooltipText = LOCTEXT("AudioDashboardEditor_StartMonitoring_TooltipText", "Starts monitoring for audio trace data without saving the data to a .utrace file.");

		static const FText StopMonitoringText = LOCTEXT("AudioDashboardEditor_StopMonitoring_Text", "Monitoring...");
		static const FText StopMonitoringTooltipText = LOCTEXT("AudioDashboardEditor_StopMonitoring_TooltipText", "Stops the active monitoring session.");

		static const FText StartRecordingText = LOCTEXT("AudioDashboardEditor_StartRecording_Text", "Start Recording");
		static const FText StartRecordingTooltipText = LOCTEXT("AudioDashboardEditor_StartRecording_TooltipText", "Starts to record audio trace data saving it to a .utrace file.");

		static const FText StopRecordingText = LOCTEXT("AudioDashboardEditor_StopRecording_Text", "Recording...");
		static const FText StopRecordingTooltipText = LOCTEXT("AudioDashboardEditor_StopRecording_TooltipText", "Stops the currently active trace recording.");

		static const FText RecordingNotificationText = LOCTEXT("AudioDashboardEditor_RecordingButton_NotificationText", "Trace Started");
		static const FText RecordingNotificationSubText = LOCTEXT("AudioDashboardEditor_RecordingButton_NotificationSubText", "Trace is now active and saving to the following location (file or tracestore):\n");

		static const FText TraceFailedToStartText = LOCTEXT("AudioDashboardEditor_TraceFailedToStart_NotificationText", "Trace failed to start.");
		static const FText TraceFailedToStartSubText = LOCTEXT("AudioDashboardEditor_TraceFailedToStart_NotificationSubText", "The Unreal Trace Server may not be running. Restart it from the Trace status bar menu > Unreal Trace Server > Start.");

		static const FText SaveCacheSnapshotTooltipText = LOCTEXT("AudioDashboardEditor_SaveCacheSnapshotButton_TooltipText", "Saves a snapshot of the Audio Insights cached data to a .utrace file containing only audio trace events.");
		static const FText SaveCacheSnapshotNotificationText = LOCTEXT("AudioDashboardEditor_SaveCacheSnapshotButton_NotificationText", "Audio Insights Cache saved.");
		static const FText SaveCacheSnapshotNotificationSubText = LOCTEXT("AudioDashboardEditor_SaveCacheSnapshotButton_NotificationSubText", "A snapshot .utrace with Audio Insights cache data with audio trace events has been saved to your trace server.");

		static const FText SaveBookmarkTooltipText = LOCTEXT("AudioDashboardEditor_BookmarkButton_TooltipText", "Ctrl+M - Creates an 'Audio Insights' bookmark in the .utrace file generated by Save Snapshot or Trace at the timestamp the button was pressed.");

		static const FText BookmarkLabelText = LOCTEXT("AudioDashboardEditor_BookmarkLabelTitle_Text", "Audio Insights Bookmark");
		static const FText SaveBookmarkNotificationText = LOCTEXT("AudioDashboardEditor_BookmarkButton_NotificationText", "Audio Insights Bookmark created.");
		static const FText SaveBookmarkNotificationDirectTraceSubText = LOCTEXT("AudioDashboardEditor_BookmarkButton_NotificationSubText", "bookmark has been added to the Audio Insights Snapshot.");
		static const FText SaveBookmarkNotificationLiveTraceSubText = LOCTEXT("AudioDashboardEditor_BookmarkButton_LiveTrace_NotificationSubText", "bookmark has been added to the live .utrace file on your trace server.");

		static const FText WorldFilterText = LOCTEXT("AudioDashboardEditor_WorldFilter_Text", "World Filter:");
		static const FText WorldFilterTooltipText = LOCTEXT("AudioDashboardEditor_WorldFilter_TooltipText", "Select world(s) to monitor (worlds may share audio output).");

		static const FText EditorPreferencesTooltipText = LOCTEXT("AudioDashboardEditor_EditorPreferences_TooltipText", "Opens Audio Insights Editor Preferences.");

		static const FText EnableTracesButtonText = LOCTEXT("AudioDashboardEditor_EnableTracesButton_Text", "Enable Audio Insights trace channels");
		static const FText EnableTracesDescriptionText = LOCTEXT("AudioDashboardEditor_EnableTracesButton_TooltipText", "Enables the Audio, Audio Mixer and Bookmark trace channels. Audio Insights will not function without these channels enabled.");
		static const FText NoTracesEnabledWarningText = LOCTEXT("AudioDashboardEditor_NoTracesEnabledWarning_Text", "Audio Insights requires the Audio, Audio Mixer and Bookmark trace channels to be enabled to function.");
		static const FText EnableThemNowText = LOCTEXT("AudioDashboardEditor_EnableNow_Text", "Enable them now?");

		void ShowNotification(const FText& InText, const FText& InSubText)
		{
			FNotificationInfo Info(InText);
			Info.SubText = InSubText;
			Info.bFireAndForget = true;
			Info.FadeOutDuration = 1.0f;
			Info.ExpireDuration = 4.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
		}

		void ShowErrorNotification(const FText& InText, const FText& InSubText)
		{
			FNotificationInfo Info(InText);
			Info.SubText = InSubText;
			Info.bFireAndForget = false;
			Info.bUseSuccessFailIcons = true;
			Info.FadeOutDuration = 2.0f;
			Info.ExpireDuration = 6.0f;

			TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
			if (NotificationItem.IsValid())
			{
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
				NotificationItem->ExpireAndFadeout();
			}
		}

		void ShowRecordingNotification()
		{
			FString TraceDestinationStr = FTraceAuxiliary::GetTraceDestinationString();
			if (TraceDestinationStr.IsEmpty())
			{
				TraceDestinationStr = TEXT("External Target");
			}
			
			ShowNotification(RecordingNotificationText, FText::Format(FText::FromString(TEXT("{0}{1}")), RecordingNotificationSubText, FText::FromString(TraceDestinationStr)));
		}

		FText GetDebugNameFromDeviceId(::Audio::FDeviceId InDeviceId)
		{
			FString WorldName;
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				TArray<UWorld*> DeviceWorlds = DeviceManager->GetWorldsUsingAudioDevice(InDeviceId);
				for (const UWorld* World : DeviceWorlds)
				{
					if (!WorldName.IsEmpty())
					{
						WorldName += TEXT(", ");
					}
					WorldName += World->GetDebugDisplayName();
				}
			}

			if (WorldName.IsEmpty())
			{
				return PreviewDeviceText;
			}

			return FText::FromString(WorldName);
		}

		// Helper class required to intercept OnKeyDown events
		class SAudioInsightsTabContents : public SVerticalBox
		{
		public:
			SLATE_BEGIN_ARGS(SAudioInsightsTabContents)
			{}

			SLATE_EVENT(FOnKeyDown, OnKeyDown)

			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs)
			{
				OnKeyDownHandler = InArgs._OnKeyDown;

				SVerticalBox::Construct(SVerticalBox::FArguments());
			}

			virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
			{
				if (OnKeyDownHandler.IsBound())
				{
					return OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
				}

				return SVerticalBox::OnKeyDown(MyGeometry, InKeyEvent);
			}

			virtual bool SupportsKeyboardFocus() const override { return true; }

		private:
			FOnKeyDown OnKeyDownHandler;
		};
	} // namespace EditorDashboardFactoryPrivate

	void FEditorDashboardFactory::OnWorldRegisteredToAudioDevice(const UWorld* InWorld, ::Audio::FDeviceId InDeviceId)
	{
		if (InDeviceId != INDEX_NONE)
		{
			SetActiveDeviceId(InDeviceId);
		}

		RefreshDeviceSelector();
	}

	void FEditorDashboardFactory::OnWorldUnregisteredFromAudioDevice(const UWorld* InWorld, ::Audio::FDeviceId InDeviceId)
	{
		RefreshDeviceSelector();
	}

	void FEditorDashboardFactory::OnTraceStarted(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
	{
		IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

		// If we haven't started a trace, start the analysis with the currently running trace
		if (!TraceModule.IsAudioInsightsTrace())
		{
			TraceModule.StartTraceAnalysis(ShouldOnlyTraceAudioChannels(), ETraceMode::Recording);
			BookmarkIndex = 0;
		}
	}

	void FEditorDashboardFactory::OnPIEStarted(bool bSimulating)
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		::Audio::Trace::EventLog::SendEvent(ActiveDeviceId, ::Audio::Trace::EventLog::ID::PIEStarted);
#endif
	}

	void FEditorDashboardFactory::OnPIEStopped(bool bSimulating)
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		::Audio::Trace::EventLog::SendEvent(ActiveDeviceId, ::Audio::Trace::EventLog::ID::PIEStopped);
#endif
	}

	void FEditorDashboardFactory::OnOnlyTraceAudioChannelsPropertyChanged(const bool bInOnlyTraceAudioChannels)
	{
		IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();
		TraceModule.OnOnlyTraceAudioChannelsStateChanged(bInOnlyTraceAudioChannels);
	}

	void FEditorDashboardFactory::CreateTraceBookmark()
	{
		using namespace EditorDashboardFactoryPrivate;

		if (IAudioInsightsModule::IsLiveSession())
		{
			const FString BookmarkLabel = FString::Printf(TEXT("%s %d"), *BookmarkLabelText.ToString(), BookmarkIndex);

			TRACE_BOOKMARK(TEXT("%s"), *BookmarkLabel);

			FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

			const double CurrentTimestamp = CacheManager.GetCacheEndTimeStamp();
			CacheManager.AddMessageToCache(MakeShared<FBookmarkTraceMessage>(CurrentTimestamp, BookmarkLabel));

			BookmarkIndex++;

			// Show floating notification
			FNotificationInfo Info(SaveBookmarkNotificationText);

			const IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

			const FText SubText = (TraceModule.GetTraceMode() == ETraceMode::Recording) 
				? SaveBookmarkNotificationLiveTraceSubText 
				: SaveBookmarkNotificationDirectTraceSubText;

			static const FTextFormat SubTextFormat = FTextFormat::FromString(TEXT("{0} {1} {2}"));
			Info.SubText = FText::Format(SubTextFormat, BookmarkLabelText, BookmarkIndex, SubText);

			Info.bFireAndForget = true;
			Info.FadeOutDuration = 1.0f;
			Info.ExpireDuration = 4.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	void FEditorDashboardFactory::OnDeviceCreated(::Audio::FDeviceId InDeviceId)
	{
		OnActiveAudioDeviceChanged.Broadcast();
	}

	void FEditorDashboardFactory::OnDeviceDestroyed(::Audio::FDeviceId InDeviceId)
	{
		if (ActiveDeviceId == InDeviceId)
		{
			if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				ActiveDeviceId = DeviceManager->GetMainAudioDeviceID();
			}
		}

		AudioDeviceIds.RemoveAll([InDeviceId](const TSharedPtr<::Audio::FDeviceId>& DeviceIdPtr)
		{
			return *DeviceIdPtr.Get() == InDeviceId;
		});

		if (AudioDeviceComboBox.IsValid())
		{
			AudioDeviceComboBox->RefreshOptions();
		}

		OnActiveAudioDeviceChanged.Broadcast();
	}

	void FEditorDashboardFactory::RefreshDeviceSelector()
	{
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			if (!DeviceManager->IsValidAudioDevice(ActiveDeviceId))
			{
				ActiveDeviceId = DeviceManager->GetMainAudioDeviceID();
			}
		}

		AudioDeviceIds.Empty();
		if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			DeviceManager->IterateOverAllDevices([this, &DeviceManager](::Audio::FDeviceId DeviceId, const FAudioDevice* AudioDevice)
			{
				AudioDeviceIds.Add(MakeShared<::Audio::FDeviceId>(DeviceId));
			});
		}

		if (AudioDeviceComboBox.IsValid())
		{
			AudioDeviceComboBox->RefreshOptions();
		}
	}

	void FEditorDashboardFactory::ResetDelegates()
	{
		if (OnWorldRegisteredToAudioDeviceHandle.IsValid())
		{
			FAudioDeviceWorldDelegates::OnWorldRegisteredToAudioDevice.Remove(OnWorldRegisteredToAudioDeviceHandle);
			OnWorldRegisteredToAudioDeviceHandle.Reset();
		}

		if (OnWorldUnregisteredFromAudioDeviceHandle.IsValid())
		{
			FAudioDeviceWorldDelegates::OnWorldUnregisteredWithAudioDevice.Remove(OnWorldUnregisteredFromAudioDeviceHandle);
			OnWorldUnregisteredFromAudioDeviceHandle.Reset();
		}

		if (OnDeviceCreatedHandle.IsValid())
		{
			FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(OnDeviceCreatedHandle);
			OnDeviceCreatedHandle.Reset();
		}

		if (OnDeviceDestroyedHandle.IsValid())
		{
			FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(OnDeviceDestroyedHandle);
			OnDeviceDestroyedHandle.Reset();
		}

		if (OnTraceStartedHandle.IsValid())
		{
			FTraceAuxiliary::OnTraceStarted.Remove(OnTraceStartedHandle);
			OnTraceStartedHandle.Reset();
		}

		if (OnPIEStartedHandle.IsValid())
		{
			FEditorDelegates::PostPIEStarted.Remove(OnPIEStartedHandle);
			OnPIEStartedHandle.Reset();
		}

		if (OnPIEStoppedHandle.IsValid())
		{
			FEditorDelegates::EndPIE.Remove(OnPIEStoppedHandle);
			OnPIEStoppedHandle.Reset();
		}

		if (OnOnlyTraceAudioChannelsPropertyChangedHandle.IsValid())
		{
			TObjectPtr<UAudioInsightsEditorSettings> AudioInsightsEditorSettings = GetMutableDefault<UAudioInsightsEditorSettings>();

			if (AudioInsightsEditorSettings)
			{
				AudioInsightsEditorSettings->OnOnlyTraceAudioChannelsPropertyChanged.Remove(OnOnlyTraceAudioChannelsPropertyChangedHandle);
				OnOnlyTraceAudioChannelsPropertyChangedHandle.Reset();
			}
		}

	}

	void FEditorDashboardFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FDashboardAssetCommands& Commands = FDashboardAssetCommands::Get();
		CommandList->MapAction(Commands.GetTraceBookmarkCommand(), FExecuteAction::CreateRaw(this, &FEditorDashboardFactory::CreateTraceBookmark));
	}

	::Audio::FDeviceId FEditorDashboardFactory::GetDeviceId() const
	{
		return ActiveDeviceId;
	}

	FEditorDashboardFactory::FEditorDashboardFactory()
	{
		FDashboardAssetCommands::Register();

		BindCommands();
	}

	FEditorDashboardFactory::~FEditorDashboardFactory()
	{
		FDashboardAssetCommands::Unregister();
	}

	TSharedRef<SDockTab> FEditorDashboardFactory::MakeDockTabWidget(const FSpawnTabArgs& Args)
	{
		using namespace EditorDashboardFactoryPrivate;

		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(ToolNameText)
			.Clipping(EWidgetClipping::ClipToBounds)
			.TabRole(ETabRole::NomadTab);

		DashboardTabManager = FGlobalTabmanager::Get()->NewTabManager(DockTab);

		DashboardTabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateStatic([](const TSharedRef<FTabManager::FLayout>& InLayout)
		{
			if (InLayout->GetPrimaryArea().Pin().IsValid())
			{
				FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
			}
		}));

		ToolbarWidgets = MakeShared<FToolbarWidgets>();

		InitDelegates();

		RegisterTabSpawners();
		RefreshDeviceSelector();

		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			SetActiveDeviceId(DeviceManager->GetActiveAudioDevice()->DeviceID);
		}

		if (!bIsLayoutReseting)
		{
			IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

			// If we haven't started a trace, start the analysis with the currently running trace
			if (FTraceAuxiliary::IsConnected() && !TraceModule.IsAudioInsightsTrace())
			{
				TraceModule.StartTraceAnalysis(ShouldOnlyTraceAudioChannels(), ETraceMode::Recording);
			}
			else
			{
				TraceModule.StartTraceAnalysis(ShouldOnlyTraceAudioChannels(), ETraceMode::Monitoring);
			}
		}

		const TSharedRef<FTabManager::FLayout> TabLayout = LoadLayoutFromConfig();

		TSharedRef<SVerticalBox> TabContent = SNew(SAudioInsightsTabContents)
			.OnKeyDown_Lambda([this](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
			{
				return (CommandList && CommandList->ProcessCommandBindings(InKeyEvent)) ? FReply::Handled() : FReply::Unhandled();
			});

		TabContent->AddSlot().AutoHeight()[ MakeMenuBarWidget() ];
		TabContent->AddSlot().AutoHeight()[ MakeMainToolbarWidget() ];
		TabContent->AddSlot().AutoHeight()[ SNew(SBox).HeightOverride(4.0f) ];
		TabContent->AddSlot()[ DashboardTabManager->RestoreFrom(TabLayout, Args.GetOwnerWindow()).ToSharedRef() ];

		DockTab->SetContent(
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				TabContent
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				MakeEnableTracesOverlay()
			]
		);

		DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab> TabClosed)
		{
			// Stop trace when closing Audio Insights (only if we have started it, we don't want to stop external traces)
			IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

			if (TraceModule.IsTraceAnalysisActive() && TraceModule.IsAudioInsightsTrace() && !bIsLayoutReseting)
			{
				TraceModule.StopTraceAnalysis();
			}

			ResetDelegates();
			ToolbarWidgets.Reset();
			UnregisterTabSpawners();
			SaveLayoutToConfig();

			for (const auto& KVP : DashboardViewFactories)
			{
				if (TSharedPtr<SDockTab> DashboardTab = DashboardTabManager->FindExistingLiveTab(KVP.Key))
				{
					// Explicitly close each dashboard tab. This will give a chance for it to close any undocked sub-managed tabs of its own:
					DashboardTab->RequestCloseTab();
				}
			}

			DashboardTabManager->CloseAllAreas();

			DashboardTabManager.Reset();
			DashboardWorkspace.Reset();
		}));

		return DockTab;
	}

	TSharedRef<SWidget> FEditorDashboardFactory::MakeMenuBarWidget()
	{
		using namespace EditorDashboardFactoryPrivate;

		FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

		MenuBarBuilder.AddPullDownMenu(
			MainMenuFileText,
			FText::GetEmpty(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(
					FileCloseText,
					FileCloseTooltipText,
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this]()
					{
						if (DashboardTabManager.IsValid())
						{
							if (TSharedPtr<SDockTab> OwnerTab = DashboardTabManager->GetOwnerTab())
							{
								OwnerTab->RequestCloseTab();
							}
						}
					}))
				);
			}),
			"File"
		);

		MenuBarBuilder.AddPullDownMenu(
			MainMenuViewText,
			FText::GetEmpty(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
			{
				for (const auto& KVP : DashboardViewFactories)
				{
					const FName& FactoryName = KVP.Key;
					const TSharedPtr<IDashboardViewFactory>& Factory = KVP.Value;

					MenuBuilder.AddMenuEntry(
						Factory->GetDisplayName(),
						FText::GetEmpty(),
						Factory->GetIcon(),
						FUIAction(FExecuteAction::CreateLambda([this, FactoryName]()
						{
							if (DashboardTabManager.IsValid())
							{
								const TSharedPtr<SDockTab> ViewportTab = DashboardTabManager->FindExistingLiveTab(FactoryName);

								if (!ViewportTab.IsValid())
								{
									const TSharedPtr<SDockTab> InvokedTab = DashboardTabManager->TryInvokeTab(FactoryName);

									if (InvokedTab.IsValid() && DashboardViewFactories[FactoryName].IsValid())
									{
										const EDefaultDashboardTabStack DefaultTabStack = DashboardViewFactories[FactoryName]->GetDefaultTabStack();

										if (DefaultTabStack == EDefaultDashboardTabStack::AudioAnalyzerRack || DefaultTabStack == EDefaultDashboardTabStack::OutputMetering)
										{
											InvokedTab->SetParentDockTabStackTabWellHidden(true);
										}
									}
								}
								else
								{
									ViewportTab->RequestCloseTab();
								}
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([&DashboardTabManager = DashboardTabManager, FactoryName]()
						{
							return DashboardTabManager.IsValid() ? DashboardTabManager->FindExistingLiveTab(FactoryName).IsValid() : false;
						})),
						NAME_None,
						EUserInterfaceActionType::Check
					);

					if (const EDefaultDashboardTabStack DefaultTabStack = DashboardViewFactories[FactoryName]->GetDefaultTabStack();
						DefaultTabStack == EDefaultDashboardTabStack::Log 
						|| DefaultTabStack == EDefaultDashboardTabStack::AudioMeters
						|| DefaultTabStack == EDefaultDashboardTabStack::Plots)
					{
						MenuBuilder.AddMenuSeparator();
					}
				}

				MenuBuilder.AddMenuSeparator();

				MenuBuilder.AddMenuEntry(
					ViewResetLayoutText,
					FText::GetEmpty(), 
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this]()
					{
						if (DashboardTabManager)
						{
							for (const auto& KVP : DashboardViewFactories)
							{
								// Try and get the dashboard tab:
								TSharedPtr<SDockTab> DashboardTab = DashboardTabManager->FindExistingLiveTab(KVP.Key);
								if (!DashboardTab.IsValid())
								{
									DashboardTab = DashboardTabManager->TryInvokeTab(KVP.Key);
								}

								if (DashboardTab.IsValid())
								{
									if (TSharedPtr<FTabManager> SubTabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(DashboardTab))
									{
										// There is a sub tab manager for this dashbaord tab; clear its persisted areas:
										SubTabManager->CloseAllAreas();
										SubTabManager->SavePersistentLayout();
									}
								}
							}

							if (TSharedPtr<SDockTab> OwnerTab = DashboardTabManager->GetOwnerTab())
							{
								// Wipe all the persisted areas and close tab
								DashboardTabManager->CloseAllAreas();

								bIsLayoutReseting = true;

								OwnerTab->RequestCloseTab();

								// Can't invoke the tab immediately (it won't show up), needs to be done a bit later
								AsyncTask(ENamedThreads::GameThread, [AudioInsightsTabId = OwnerTab->GetLayoutIdentifier(), &bIsLayoutReseting = this->bIsLayoutReseting]()
								{
									FGlobalTabmanager::Get()->TryInvokeTab(AudioInsightsTabId);
									bIsLayoutReseting = false;
								});
							}
						}
					}),
					FCanExecuteAction()));
			}),
			"View"
		);

		return MenuBarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FEditorDashboardFactory::MakeMainToolbarWidget()
	{
		using namespace EditorDashboardFactoryPrivate;

		static FSlateBrush BackgroundColorBrush;
		BackgroundColorBrush.TintColor = FSlateColor(FLinearColor(0.018f, 0.018f, 0.018f, 1.0f));
		BackgroundColorBrush.DrawAs = ESlateBrushDrawType::Box;

		return SNew(SBorder)
			.BorderImage(&BackgroundColorBrush)
			[
				SNew(SVerticalBox)
				// Top row: Trace controls, world filter, settings
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					// Monitoring button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(3.0f, 0.0f, 5.0f, 0.0f)
					[
						MakeTraceModeButton(ETraceMode::Monitoring)
					]
					// Separator
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBox)
						.HeightOverride(38.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Vertical)
							.Thickness(2.0f)
						]
					]
					// Recording button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(5.0f, 0.0f, 5.0f, 0.0f)
					[
						MakeTraceModeButton(ETraceMode::Recording)
					]
					// Separator
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBox)
						.HeightOverride(38.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Vertical)
							.Thickness(2.0f)
						]
					]
					// Save cache snapshot button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(12.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.ToolTipText(SaveCacheSnapshotTooltipText)
						.IsEnabled_Lambda([this]()
						{
							return IAudioInsightsModule::IsLiveSession();
						})
						.OnClicked_Lambda([]()
						{
							IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();
							TraceModule.SaveCacheSnapshot();

							ShowNotification(SaveCacheSnapshotNotificationText, SaveCacheSnapshotNotificationSubText);

							return FReply::Handled();
						})
						[
							SNew(SBox)
							.HeightOverride(16.0f)
							[
								SNew(SHorizontalBox)
								// Save cache snapshot icon
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Trace.CacheSnapshot"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
					]
					// Trace Bookmark Button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(5.0f, 0.0f, 10.0f, 0.0f)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.ToolTipText(SaveBookmarkTooltipText)
						.IsEnabled_Lambda([this]()
						{
							return IAudioInsightsModule::IsLiveSession();
						})
						.OnClicked_Lambda([this]()
						{
							CreateTraceBookmark();

							return FReply::Handled();
						})
						[
							SNew(SBox)
							.HeightOverride(16.0f)
							[
								SNew(SHorizontalBox)
								// Save Bookmark icon
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Trace.Bookmark"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
					]
					// Separator
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBox)
						.HeightOverride(38.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Vertical)
							.Thickness(2.0f)
						]
					]
					// Only trace audio channels toggle button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(20.0f, 0.0f, 18.0f, 0.0f)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.ToolTipText(OnlyTraceAudioChannelsTooltipText)
						.OnClicked_Lambda([this]()
						{
							TObjectPtr<UAudioInsightsEditorSettings> const AudioInsightsEditorSettings = GetMutableDefault<UAudioInsightsEditorSettings>();
							if (AudioInsightsEditorSettings)
							{
								AudioInsightsEditorSettings->bOnlyTraceAudioChannels = !AudioInsightsEditorSettings->bOnlyTraceAudioChannels;
								AudioInsightsEditorSettings->SaveConfig();

								IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();
								TraceModule.OnOnlyTraceAudioChannelsStateChanged(AudioInsightsEditorSettings->bOnlyTraceAudioChannels);
							}

							return FReply::Handled();
						})
						[
							SNew(SBox)
							.HeightOverride(16.0f)
							[
								SNew(SHorizontalBox)
								// Only trace audio channels icon
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.OnlyTraceAudioChannels"))
									.ColorAndOpacity_Lambda([this]()
									{
										return ShouldOnlyTraceAudioChannels() ? FSlateColor(FLinearColor::Green) : FSlateColor::UseForeground();
									})
								]
								// Only trace audio channels text
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								.Padding(6.0f, 1.0f, 0.0f, 0.0f)
								[
									SNew(STextBlock)
									.Text(OnlyTraceAudioChannelsText)
									.Justification(ETextJustify::Center)
									.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
								]
							]
						]
					]

					// World filter label
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(WorldFilterText)
						.ToolTipText(WorldFilterTooltipText)
						.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
					]
					// World filter combobox
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.MaxWidth(200.0f)
					.Padding(2.0f, 0.0f)
					[
						SAssignNew(AudioDeviceComboBox, SComboBox<TSharedPtr<::Audio::FDeviceId>>)
						.ToolTipText(WorldFilterTooltipText)
						.OptionsSource(&AudioDeviceIds)
						.OnGenerateWidget_Lambda([](const TSharedPtr<::Audio::FDeviceId>& WidgetDeviceId)
						{
							FText NameText = GetDebugNameFromDeviceId(*WidgetDeviceId);
							return SNew(STextBlock)
								.Text(NameText)
								.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
						})
						.OnSelectionChanged_Lambda([this](TSharedPtr<::Audio::FDeviceId> NewDeviceId, ESelectInfo::Type)
						{
							if (NewDeviceId.IsValid())
							{
								ActiveDeviceId = *NewDeviceId;
								RefreshDeviceSelector();

								OnActiveAudioDeviceChanged.Broadcast();
							}
						})
						[
							SNew(STextBlock)
							.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
							.Text_Lambda([this]()
							{
								return GetDebugNameFromDeviceId(ActiveDeviceId);
							})
						]
					]

					// Editor Preferences
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(12.0f, 0.0f, 0.0f, 0.0f)
					.HAlign(HAlign_Right)
					[
						CreateEditorPreferencesButtonWidget()
					]

					// Documentation
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1.0f, 0.0f, 0.0f, 0.0f)
					.HAlign(HAlign_Right)
					[
						FDashboardFactory::CreateDocumentationButtonWidget()
					]
				]
				// Bottom row: Cache controls
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					// Cache label
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(28.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheLabelWidget()
					]
					// Cache control: Pause button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(30.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCachePauseButtonWidget()
					]
					// Stop cache button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(1.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheStopButtonWidget()
					]
					// Resume cache button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(1.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheResumeButtonWidget()
					]
					// Current timestamp
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheCurrentTimestampWidget()
					]
					// Follow button
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheFollowButtonWidget()
					]
					// Begin timestamp
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheBeginTimestampWidget()
					]
					// Nudge back
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheNudgeBackButtonWidget()
					]
					// Timeline ruler
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					.Padding(4.0f, 0.0f, 4.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheTimelineRulerWidget()
					]
					// Nudge forward
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheNudgeForwardButtonWidget()
					]
					// End timestamp
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheEndTimestampWidget()
					]
					// Separator
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(8.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SBox)
						.HeightOverride(22.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Vertical)
							.Thickness(3.0f)
						]
					]
					// Cache size and duration
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						ToolbarWidgets->MakeCacheSizeAndDurationWidget()
					]
					// Cache settings
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						ToolbarWidgets->MakeCacheSettingsButtonWidget()
					]
				]
			];
	}

	TSharedRef<SWidget> FEditorDashboardFactory::MakeTraceModeButton(const ETraceMode InTraceButtonMode)
	{
		using namespace EditorDashboardFactoryPrivate;

		return SNew(SBox)
			.WidthOverride(130.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText_Lambda([InTraceButtonMode]()
				{
					const IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

					const ETraceMode CurrentTraceMode = TraceModule.GetTraceMode();
					const bool bIsButtonActive = InTraceButtonMode == ETraceMode::Monitoring ? CurrentTraceMode != ETraceMode::None : CurrentTraceMode == InTraceButtonMode;
					
					if (TraceModule.IsTraceAnalysisActive() && bIsButtonActive)
					{
						return InTraceButtonMode == ETraceMode::Monitoring ? StopMonitoringTooltipText : StopRecordingTooltipText;
					}

					return InTraceButtonMode == ETraceMode::Monitoring ? StartMonitoringTooltipText : StartRecordingTooltipText;
				})
				.OnClicked_Lambda([this, InTraceButtonMode]()
				{
					IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

					if (FTraceAuxiliary::IsConnected())
					{
						const ETraceMode TraceMode = TraceModule.GetTraceMode();

						TraceModule.StopTraceAnalysis();

						// Go to Recording mode when Monitoring is active and Record button is pressed
						if (InTraceButtonMode == ETraceMode::Recording)
						{
							// Stopping a trace doesn't occur immediately, we need to check until it actually
							// stops in order to call StartTraceAnalysis again (and disable the ticker)
							auto StartTraceAnalysisDeferred = [](const bool bInOnlyTraceAudioChannels, const ETraceMode InTraceMode)
							{
								FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
									[bInOnlyTraceAudioChannels, InTraceMode](float DeltaTime)
									{
										if (FTraceAuxiliary::IsConnected())
										{
											return true;
										}

										IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();
										TraceModule.StartTraceAnalysis(bInOnlyTraceAudioChannels, InTraceMode);

										if (TraceModule.IsTraceAnalysisActive())
										{
											ShowRecordingNotification();
										}
										else
										{
											ShowErrorNotification(TraceFailedToStartText, TraceFailedToStartSubText);
										}

										return false;
									}));
							};

							if (TraceMode == ETraceMode::Monitoring)
							{
								StartTraceAnalysisDeferred(ShouldOnlyTraceAudioChannels(), ETraceMode::Recording);
							}
						}
					}
					else
					{
						TraceModule.StartTraceAnalysis(ShouldOnlyTraceAudioChannels(), InTraceButtonMode);

						if (TraceModule.IsTraceAnalysisActive())
						{
							if (InTraceButtonMode == ETraceMode::Recording)
							{
								ShowRecordingNotification();
							}
						}
						else
						{
							ShowErrorNotification(TraceFailedToStartText, TraceFailedToStartSubText);
						}
					}

					return FReply::Handled();
				})
				[
					SNew(SBox)
					.HeightOverride(16.0f)
					[
						SNew(SHorizontalBox)
						// Trace button icon
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FSlateStyle::Get().GetBrush("AudioInsights.Icon.Trace"))
							.ColorAndOpacity_Lambda([InTraceButtonMode]()
				 			{
								const IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

								const ETraceMode CurrentTraceMode = TraceModule.GetTraceMode();
								const bool bIsButtonActive = InTraceButtonMode == ETraceMode::Monitoring ? CurrentTraceMode != ETraceMode::None : CurrentTraceMode == InTraceButtonMode;

								if (TraceModule.IsTraceAnalysisActive() && bIsButtonActive)
								{
									return FSlateColor(InTraceButtonMode == ETraceMode::Monitoring ? FLinearColor::Green : FLinearColor::Red);
								}

								return FSlateColor::UseForeground();
				 			})
						]
						// Trace button text
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(6.0f, 1.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text_Lambda([InTraceButtonMode]()
				 			{
								const IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

								const ETraceMode CurrentTraceMode = TraceModule.GetTraceMode();
								const bool bIsButtonActive = InTraceButtonMode == ETraceMode::Monitoring ? CurrentTraceMode != ETraceMode::None	: CurrentTraceMode == InTraceButtonMode;

								if (TraceModule.IsTraceAnalysisActive() && bIsButtonActive)
								{
									return InTraceButtonMode == ETraceMode::Monitoring ? StopMonitoringText : StopRecordingText;
								}

								return InTraceButtonMode == ETraceMode::Monitoring ? StartMonitoringText : StartRecordingText;
				 			})
							.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
						]
					]
				]
			];
	}

	TSharedRef<SWidget> FEditorDashboardFactory::MakeEnableTracesOverlay()
	{
		using namespace EditorDashboardFactoryPrivate;

		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.75f))
			.Visibility_Lambda([]() -> EVisibility
			{
				const IAudioInsightsTraceModule& TraceModule = FAudioInsightsEditorModule::GetChecked().GetTraceModule();

				const bool bIsActive = TraceModule.IsTraceAnalysisActive() && TraceModule.GetTraceMode() != ETraceMode::None;
				if (!bIsActive)
				{
					return EVisibility::Hidden;
				}

				struct FChannelStates
				{
					bool bAudioOn = false;
					bool bAudioMixerOn = false;
					bool bBookmarkOn = false;
				};

				FChannelStates States;

				UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& ChannelInfo, void* User)
				{
					FChannelStates* const ChannelStates = static_cast<FChannelStates*>(User);

					if (!ChannelInfo.bIsEnabled)
					{
						return true;
					}

					if (FCStringAnsi::Stricmp(ChannelInfo.Name, "AudioChannel") == 0)
					{
						ChannelStates->bAudioOn = true;
					}
					else if (FCStringAnsi::Stricmp(ChannelInfo.Name, "AudioMixerChannel") == 0)
					{
						ChannelStates->bAudioMixerOn = true;
					}
					else if (FCStringAnsi::Stricmp(ChannelInfo.Name, "BookmarkChannel") == 0)
					{
						ChannelStates->bBookmarkOn = true;
					}

					return true;
				}, &States);

				const bool bChannelsOk = States.bAudioOn && States.bAudioMixerOn && States.bBookmarkOn;
				return bChannelsOk ? EVisibility::Hidden : EVisibility::Visible;
			})
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				[
					SNullWidget::NullWidget
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(NoTracesEnabledWarningText)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(EnableThemNowText)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 10.0f)
				[
					MakeEnableTracesButton()
				]

				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				[
					SNullWidget::NullWidget
				]
			];
	}

	TSharedRef<SWidget> FEditorDashboardFactory::MakeEnableTracesButton()
	{
		using namespace EditorDashboardFactoryPrivate;

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.OnClicked(this, &FEditorDashboardFactory::OnEnableTracesClicked)
			.ToolTipText(EnableTracesDescriptionText)
			.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
			.Content()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
					.Justification(ETextJustify::Center)
					.Text(EnableTracesButtonText)
				]
			];
	}

	FReply FEditorDashboardFactory::OnEnableTracesClicked()
	{
		UE::Trace::ToggleChannel(TEXT("Audio"), true);
		UE::Trace::ToggleChannel(TEXT("AudioMixer"), true);
		UE::Trace::ToggleChannel(TEXT("Bookmark"), true);

		return FReply::Handled();
	}

	TSharedRef<SWidget> FEditorDashboardFactory::CreateEditorPreferencesButtonWidget()
	{
		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(EditorDashboardFactoryPrivate::EditorPreferencesTooltipText)
			.OnClicked_Lambda([]()
			{
				if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
				{
					SettingsModule->ShowViewer("Editor", "Plugins", "AudioInsightsEditorSettings");
				}
				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("EditorPreferences.TabIcon"))
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				]
			];
	}

	void FEditorDashboardFactory::SetActiveDeviceId(const ::Audio::FDeviceId InDeviceId)
	{
		const TObjectPtr<const UAudioInsightsEditorSettings> AudioInsightsEditorSettings = GetDefault<UAudioInsightsEditorSettings>();

		// We don't want to set ActiveDeviceId if bWorldFilterDefaultsToFirstClient is true and more than 2 PIE clients are running
		if (AudioInsightsEditorSettings == nullptr ||
			(AudioInsightsEditorSettings && !AudioInsightsEditorSettings->bWorldFilterDefaultsToFirstClient) ||
			AudioDeviceIds.Num() < 2)
		{
			ActiveDeviceId = InDeviceId;
		}
	}

	void FEditorDashboardFactory::InitDelegates()
	{
		if (!OnWorldRegisteredToAudioDeviceHandle.IsValid())
		{
			OnWorldRegisteredToAudioDeviceHandle = FAudioDeviceWorldDelegates::OnWorldRegisteredToAudioDevice.AddSP(this, &FEditorDashboardFactory::OnWorldRegisteredToAudioDevice);
		}

		if (!OnWorldUnregisteredFromAudioDeviceHandle.IsValid())
		{
			OnWorldUnregisteredFromAudioDeviceHandle = FAudioDeviceWorldDelegates::OnWorldUnregisteredWithAudioDevice.AddSP(this, &FEditorDashboardFactory::OnWorldUnregisteredFromAudioDevice);
		}

		if (!OnDeviceCreatedHandle.IsValid())
		{
			OnDeviceCreatedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddSP(this, &FEditorDashboardFactory::OnDeviceCreated);
		}

		if (!OnDeviceDestroyedHandle.IsValid())
		{
			OnDeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddSP(this, &FEditorDashboardFactory::OnDeviceDestroyed);
		}

		if (!OnTraceStartedHandle.IsValid())
		{
			OnTraceStartedHandle = FTraceAuxiliary::OnTraceStarted.AddSP(this, &FEditorDashboardFactory::OnTraceStarted);
		}

		if (!OnPIEStartedHandle.IsValid())
		{
			OnPIEStartedHandle = FEditorDelegates::PostPIEStarted.AddSP(this, &FEditorDashboardFactory::OnPIEStarted);
		}

		if (!OnPIEStoppedHandle.IsValid())
		{
			OnPIEStoppedHandle = FEditorDelegates::EndPIE.AddSP(this, &FEditorDashboardFactory::OnPIEStopped);
		}

		if (!OnOnlyTraceAudioChannelsPropertyChangedHandle.IsValid())
		{
			TObjectPtr<UAudioInsightsEditorSettings> AudioInsightsEditorSettings = GetMutableDefault<UAudioInsightsEditorSettings>();

			if (AudioInsightsEditorSettings)
			{
				OnOnlyTraceAudioChannelsPropertyChangedHandle = AudioInsightsEditorSettings->OnOnlyTraceAudioChannelsPropertyChanged.AddSP(this, &FEditorDashboardFactory::OnOnlyTraceAudioChannelsPropertyChanged);
			}
		}

	}

	TSharedRef<FTabManager::FLayout> FEditorDashboardFactory::GetDefaultTabLayout()
	{
		using namespace EditorDashboardFactoryPrivate;

		TSharedRef<FTabManager::FStack> ViewportTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> LogTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> AnalysisTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> AudioMetersTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> PlotsTabStack = FTabManager::NewStack();
		TSharedRef<FTabManager::FStack> OutputMeteringTabStack = FTabManager::NewStack()->SetHideTabWell(true);
		TSharedRef<FTabManager::FStack> AudioAnalyzerRackTabStack = FTabManager::NewStack()->SetHideTabWell(true);

		for (const auto& KVP : DashboardViewFactories)
		{
			const FName& FactoryName = KVP.Key;
			const TSharedPtr<IDashboardViewFactory>& Factory = KVP.Value;

			const EDefaultDashboardTabStack DefaultTabStack = Factory->GetDefaultTabStack();
			switch (DefaultTabStack)
			{
				case EDefaultDashboardTabStack::Viewport:
				{
					ViewportTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::Log:
				{
					LogTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::Analysis:
				{
					AnalysisTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::AudioMeters:
				{
					AudioMetersTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::Plots:
				{
					PlotsTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::OutputMetering:
				{
					OutputMeteringTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				case EDefaultDashboardTabStack::AudioAnalyzerRack:
				{
					AudioAnalyzerRackTabStack->AddTab(FactoryName, ETabState::OpenedTab);
				}
				break;

				default:
					break;
			}
		}

		AnalysisTabStack->SetForegroundTab(FName("Sounds"));

		return FTabManager::NewLayout("AudioDashboard_Editor_Layout_v3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				// Left column
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f) // Column width
					// Top
					->Split
					(
						ViewportTabStack
						->SetSizeCoefficient(0.5f)
					)
					// Bottom
					->Split
					(
						LogTabStack
						->SetSizeCoefficient(0.5f)
					)
				)
				
				// Middle column
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.6f) // Column width
					// Top
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Horizontal)
						->Split
						(
							AnalysisTabStack
							->SetSizeCoefficient(0.58f)
						)
					)
					// Bottom
					->Split
					(
						// Top
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(0.42f) // Row width
						->Split
						(
							PlotsTabStack
							->SetSizeCoefficient(0.5f)
						)
						// Bottom
						->Split
						(
							AudioMetersTabStack
							->SetSizeCoefficient(0.5f)
						)
					)
				)

				// Right column
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f) // Column width
					->Split
					(
						OutputMeteringTabStack
						->SetSizeCoefficient(0.8f)
					)
					->Split
					(
						AudioAnalyzerRackTabStack
						->SetSizeCoefficient(0.2f)
					)
				)
			)
		);
	}

	void FEditorDashboardFactory::RegisterTabSpawners()
	{
		using namespace EditorDashboardFactoryPrivate;

		DashboardWorkspace = DashboardTabManager->AddLocalWorkspaceMenuCategory(ToolNameText);

		for (const auto& KVP : DashboardViewFactories)
		{
			const FName& FactoryName = KVP.Key;
			const TSharedPtr<IDashboardViewFactory> Factory = KVP.Value;
			DashboardTabManager->RegisterTabSpawner(FactoryName, FOnSpawnTab::CreateLambda([this, FactoryWeak = Factory.ToWeakPtr()](const FSpawnTabArgs& Args)
			{
				const TSharedPtr<IDashboardViewFactory> FactoryShared = FactoryWeak.Pin();

				TSharedRef<SDockTab> DockTab = SNew(SDockTab)
					.Clipping(EWidgetClipping::ClipToBounds)
					.Label(FactoryShared.IsValid() ? FactoryShared->GetDisplayName() : FText::GetEmpty());

				TSharedRef<SWidget> DashboardView = FactoryShared.IsValid() ? FactoryShared->MakeWidget(DockTab, Args) : SNullWidget::NullWidget;
				DockTab->SetContent(DashboardView);

				return DockTab;
			}))
			.SetDisplayName(Factory->GetDisplayName())
			.SetGroup(DashboardWorkspace->AsShared())
			.SetIcon(Factory->GetIcon());
		}
	}

	void FEditorDashboardFactory::UnregisterTabSpawners()
	{
		if (DashboardTabManager.IsValid())
		{
			for (const auto& KVP : DashboardViewFactories)
			{
				const FName& FactoryName = KVP.Key;
				DashboardTabManager->UnregisterTabSpawner(FactoryName);
			}
		}
	}

	void FEditorDashboardFactory::RegisterViewFactory(TSharedRef<IDashboardViewFactory> InFactory)
	{
		if (const FName Name = InFactory->GetName(); 
			ensureAlwaysMsgf(!DashboardViewFactories.Contains(Name), TEXT("Failed to register Audio Insights Dashboard '%s': Dashboard with name already registered"), *Name.ToString()))
		{
			DashboardViewFactories.Add(Name, InFactory);
		}
	}

	void FEditorDashboardFactory::UnregisterViewFactory(FName InName)
	{
		DashboardViewFactories.Remove(InName);
	}

	TSharedRef<FTabManager::FLayout> FEditorDashboardFactory::LoadLayoutFromConfig()
	{
		return FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, GetDefaultTabLayout());
	}

	void FEditorDashboardFactory::SaveLayoutToConfig()
	{
		if (DashboardTabManager.IsValid())
		{
			FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, DashboardTabManager->PersistLayout());
		}
	}

	bool FEditorDashboardFactory::ShouldOnlyTraceAudioChannels() const
	{
		const TObjectPtr<const UAudioInsightsEditorSettings> AudioInsightsEditorSettings = GetDefault<UAudioInsightsEditorSettings>();
		return AudioInsightsEditorSettings && AudioInsightsEditorSettings->bOnlyTraceAudioChannels;
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
