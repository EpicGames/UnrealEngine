// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioInsightsToolbarWidgets.h"
#include "Containers/Map.h"
#include "Engine/World.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsDashboardFactory.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboBox.h"

class FUICommandList;

namespace UE::Audio::Insights
{
	class FEditorDashboardFactory : public IDashboardFactory, public TSharedFromThis<FEditorDashboardFactory>
	{
	public:
		FEditorDashboardFactory();
		virtual ~FEditorDashboardFactory();

		TSharedRef<SDockTab> MakeDockTabWidget(const FSpawnTabArgs& Args);

		virtual void RegisterViewFactory(TSharedRef<IDashboardViewFactory> InFactory) override;
		virtual void UnregisterViewFactory(FName InDashboardName) override;
		virtual ::Audio::FDeviceId GetDeviceId() const override;

		DECLARE_MULTICAST_DELEGATE(FOnActiveAudioDeviceChanged);
		FOnActiveAudioDeviceChanged OnActiveAudioDeviceChanged;

	private:
		void OnWorldRegisteredToAudioDevice(const UWorld* InWorld, ::Audio::FDeviceId InDeviceId);
		void OnWorldUnregisteredFromAudioDevice(const UWorld* InWorld, ::Audio::FDeviceId InDeviceId);

		void OnDeviceCreated(::Audio::FDeviceId InDeviceId);
		void OnDeviceDestroyed(::Audio::FDeviceId InDeviceId);

		void OnTraceStarted(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination);

		void OnOnlyTraceAudioChannelsPropertyChanged(const bool bInOnlyTraceAudioChannels);

		void OnPIEStarted(bool bSimulating);
		void OnPIEStopped(bool bSimulating);

		void CreateTraceBookmark();

		void RefreshDeviceSelector();
		void ResetDelegates();

		void BindCommands();

		TSharedRef<SWidget> MakeMenuBarWidget();
		TSharedRef<SWidget> MakeMainToolbarWidget();

		TSharedRef<SWidget> MakeTraceModeButton(const ETraceMode InTraceButtonMode);

		TSharedRef<SWidget> MakeEnableTracesOverlay();
		TSharedRef<SWidget> MakeEnableTracesButton();
		FReply OnEnableTracesClicked();

		TSharedRef<SWidget> CreateEditorPreferencesButtonWidget();

		void SetActiveDeviceId(const ::Audio::FDeviceId InDeviceId);
		void InitDelegates();
		TSharedRef<FTabManager::FLayout> GetDefaultTabLayout();

		void RegisterTabSpawners();
		void UnregisterTabSpawners();

		TSharedRef<FTabManager::FLayout> LoadLayoutFromConfig();
		void SaveLayoutToConfig();

		bool ShouldOnlyTraceAudioChannels() const;

		FDelegateHandle OnWorldRegisteredToAudioDeviceHandle;
		FDelegateHandle OnWorldUnregisteredFromAudioDeviceHandle;

		FDelegateHandle OnDeviceCreatedHandle;
		FDelegateHandle OnDeviceDestroyedHandle;

		FDelegateHandle OnTraceStartedHandle;

		FDelegateHandle OnPIEStartedHandle;
		FDelegateHandle OnPIEStoppedHandle;

		FDelegateHandle OnOnlyTraceAudioChannelsPropertyChangedHandle;

		bool bIsLayoutReseting = false;

		TSharedPtr<FToolbarWidgets> ToolbarWidgets;

		TSharedPtr<FTabManager> DashboardTabManager;
		TSharedPtr<FWorkspaceItem> DashboardWorkspace;

		TSharedPtr<FUICommandList> CommandList;

		TArray<TSharedPtr<::Audio::FDeviceId>> AudioDeviceIds;
		TSharedPtr<SComboBox<TSharedPtr<::Audio::FDeviceId>>> AudioDeviceComboBox;

		::Audio::FDeviceId ActiveDeviceId = INDEX_NONE;
		int32 BookmarkIndex = 0;

		TMap<FName, TSharedPtr<IDashboardViewFactory>> DashboardViewFactories;
	};
} // namespace UE::Audio::Insights
