// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsEditorModule.h"

#include "AudioInsightsEditorLog.h"
#include "AudioInsightsPluginTemplate.h"
#include "AudioInsightsStyle.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Features/IPluginsEditorFeature.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsModule.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Views/AudioAnalyzerRackDashboardViewFactory.h"
#include "Views/AudioBusDashboardViewFactory.h"
#include "Views/AudioEventLogDashboardViewFactory.h"
#include "Views/AudioMetersPanelDashboardViewFactory.h"
#include "Views/DetailsDashboardViewFactory.h"
#include "Views/LogDashboardViewFactory.h"
#include "Views/OutputMeterDashboardViewFactory.h"
#include "Views/SignalFlowDashboardViewFactory.h"
#include "Views/SoundDashboardViewFactory.h"
#include "Views/SoundPlotsDashboardViewFactory.h"
#include "Views/SubmixDashboardViewFactory.h"
#include "Views/ViewportDashboardViewFactory.h"
#include "Views/VirtualLoopDashboardViewFactory.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

DEFINE_LOG_CATEGORY(LogAudioInsightsEditor);

namespace UE::Audio::Insights
{
	void FAudioInsightsEditorModule::StartupModule()
	{
		RegisterMenus();

		DashboardFactory = MakeShared<FEditorDashboardFactory>();
		
		TSharedRef<FSoundDashboardViewFactory> SoundsDashboard = MakeShared<FSoundDashboardViewFactory>();
		TSharedRef<FSoundPlotsDashboardViewFactory> PlotsDashboard = MakeShared<FSoundPlotsDashboardViewFactory>();
		TSharedRef<FSubmixDashboardViewFactory> SubmixDashboard = MakeShared<FSubmixDashboardViewFactory>();
		EventLogViewFactory = MakeShared<FAudioEventLogDashboardViewFactory>();

		// @TODO UE-274216: Decide what to do with the Viewport dashboard
		//DashboardFactory->RegisterViewFactory(MakeShared<FViewportDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(MakeShared<FLogDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(MakeShared<FDetailsDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(EventLogViewFactory.ToSharedRef());
		DashboardFactory->RegisterViewFactory(SoundsDashboard);
		DashboardFactory->RegisterViewFactory(MakeShared<FVirtualLoopDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(SubmixDashboard);
		DashboardFactory->RegisterViewFactory(MakeShared<FAudioBusDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(MakeShared<FAudioMetersPanelDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(MakeShared<FSignalFlowDashboardViewFactory>());
		DashboardFactory->RegisterViewFactory(PlotsDashboard);
		DashboardFactory->RegisterViewFactory(MakeShared<FOutputMeterDashboardViewFactory>(SubmixDashboard));
		DashboardFactory->RegisterViewFactory(MakeShared<FAudioAnalyzerRackDashboardViewFactory>());

		PlotsDashboard->InitPlots(SoundsDashboard);

		// Create a template for adding new dashboards to Audio Insights
		IPluginManager& PluginManager = IPluginManager::Get();
		const TSharedPtr<const IPlugin> Plugin = PluginManager.FindPlugin(UE_PLUGIN_NAME);
		if (Plugin.IsValid())
		{
			const FText TemplateName = LOCTEXT("TemplateLabel", "Audio Insights Template");
			const FText TemplateDescription = LOCTEXT("TemplateDesc", "Create a plugin that has all the C++ boilerplate needed to create your own Audio Insights dashboard.");
			const FString PluginBaseDir = Plugin->GetBaseDir();

			AudioInsightsPluginTemplateDescription = MakeShared<FAudioInsightsPluginTemplateDescription>(TemplateName, TemplateDescription, PluginBaseDir / TEXT("Templates") / TEXT("Basic"));

			// Register Audio Insights Plugin Template
			IModularFeatures& ModularFeatures = IModularFeatures::Get();

			ModularFeatures.OnModularFeatureRegistered().AddRaw(this, &FAudioInsightsEditorModule::OnModularFeatureRegistered);
			ModularFeatures.OnModularFeatureUnregistered().AddRaw(this, &FAudioInsightsEditorModule::OnModularFeatureUnregistered);

			if (ModularFeatures.IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				OnModularFeatureRegistered(EditorFeatures::PluginsEditor, &ModularFeatures.GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor));
			}
		}
		else
		{
			UE_LOG(LogAudioInsightsEditor, Warning, TEXT("AudioInsights plugin not found, plugin template for new dashboards will not be registered."));
		}
	}

	void FAudioInsightsEditorModule::ShutdownModule()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		ModularFeatures.OnModularFeatureRegistered().RemoveAll(this);
		ModularFeatures.OnModularFeatureUnregistered().RemoveAll(this);

		if (ModularFeatures.IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
		{
			OnModularFeatureUnregistered(EditorFeatures::PluginsEditor, &ModularFeatures.GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor));
		}

		AudioInsightsPluginTemplateDescription.Reset();
		EventLogViewFactory.Reset();
		DashboardFactory.Reset();
	}

	void FAudioInsightsEditorModule::RegisterDashboardViewFactory(TSharedRef<IDashboardViewFactory> InDashboardFactory)
	{
		DashboardFactory->RegisterViewFactory(InDashboardFactory);
	}

	void FAudioInsightsEditorModule::UnregisterDashboardViewFactory(FName InName)
	{
		DashboardFactory->UnregisterViewFactory(InName);
	}

	void FAudioInsightsEditorModule::RegisterEventLogCategories(const TMap<FString, TSet<FString>>& InCategoriesToEvents)
	{
		if (EventLogViewFactory.IsValid())
		{
			EventLogViewFactory->RegisterExternalEventTypes(InCategoriesToEvents);
		}
	}

	void FAudioInsightsEditorModule::RegisterEventLogDisplayNames(const TMap<FString, FText>& InDisplayNames)
	{
		if (EventLogViewFactory.IsValid())
		{
			EventLogViewFactory->RegisterExternalDisplayNames(InDisplayNames);
		}
	}

	::Audio::FDeviceId FAudioInsightsEditorModule::GetDeviceId() const
	{
		return DashboardFactory->GetDeviceId();
	}

	bool FAudioInsightsEditorModule::IsModuleLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(AudioInsightsEditorModuleName);
	}

	FAudioInsightsEditorModule& FAudioInsightsEditorModule::GetChecked()
	{
		return static_cast<FAudioInsightsEditorModule&>(FModuleManager::LoadModuleChecked<IAudioInsightsEditorModule>(AudioInsightsEditorModuleName));
	}

	IAudioInsightsTraceModule& FAudioInsightsEditorModule::GetTraceModule()
	{
		IAudioInsightsModule& InsightsModule = IAudioInsightsModule::GetChecked();
		return InsightsModule.GetTraceModule();
	}

	FAudioInsightsCacheManager& FAudioInsightsEditorModule::GetCacheManager()
	{
		IAudioInsightsModule& InsightsModule = IAudioInsightsModule::GetChecked();
		return InsightsModule.GetCacheManager();
	}

	FAudioInsightsTimingViewExtender& FAudioInsightsEditorModule::GetTimingViewExtender()
	{
		IAudioInsightsModule& InsightsModule = IAudioInsightsModule::GetChecked();
		return InsightsModule.GetTimingViewExtender();
	}

	FAudioInsightsDetailsSelectionManager& FAudioInsightsEditorModule::GetDetailsSelectionManager()
	{
		IAudioInsightsModule& InsightsModule = IAudioInsightsModule::GetChecked();
		return InsightsModule.GetDetailsSelectionManager();
	}

	TSharedPtr<FEditorDashboardFactory> FAudioInsightsEditorModule::GetDashboardFactory()
	{
		return DashboardFactory;
	}

	const TSharedPtr<FEditorDashboardFactory> FAudioInsightsEditorModule::GetDashboardFactory() const
	{
		return DashboardFactory;
	}

	TSharedRef<SDockTab> FAudioInsightsEditorModule::CreateDashboardTabWidget(const FSpawnTabArgs& Args)
	{
		return DashboardFactory->MakeDockTabWidget(Args);
	}

	void FAudioInsightsEditorModule::RegisterMenus()
	{
		const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner("AudioInsights", FOnSpawnTab::CreateRaw(this, &FAudioInsightsEditorModule::CreateDashboardTabWidget))
			.SetDisplayName(LOCTEXT("OpenDashboard_TabDisplayName", "Audio Insights"))
			.SetTooltipText(LOCTEXT("OpenDashboard_TabTooltip", "Opens Audio Insights, an extensible suite of tools and visualizers which enable monitoring and debugging audio in the Unreal Engine."))
			.SetGroup(MenuStructure.GetToolsCategory())
			.SetIcon(FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Dashboard"));
	}

	void FAudioInsightsEditorModule::OnModularFeatureRegistered(const FName& Name, IModularFeature* ModularFeature)
	{
		if (Name == EditorFeatures::PluginsEditor && ModularFeature != nullptr && AudioInsightsPluginTemplateDescription.IsValid())
		{
			IPluginsEditorFeature* PluginEditor = static_cast<IPluginsEditorFeature*>(ModularFeature);
			PluginEditor->RegisterPluginTemplate(AudioInsightsPluginTemplateDescription.ToSharedRef());
		}
	}

	void FAudioInsightsEditorModule::OnModularFeatureUnregistered(const FName& Name, IModularFeature* ModularFeature)
	{
		if (Name == EditorFeatures::PluginsEditor && ModularFeature != nullptr && AudioInsightsPluginTemplateDescription.IsValid())
		{
			IPluginsEditorFeature* PluginEditor = static_cast<IPluginsEditorFeature*>(ModularFeature);
			PluginEditor->UnregisterPluginTemplate(AudioInsightsPluginTemplateDescription.ToSharedRef());
		}
	}
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE // AudioInsights

IMPLEMENT_MODULE(UE::Audio::Insights::FAudioInsightsEditorModule, AudioInsightsEditor)
