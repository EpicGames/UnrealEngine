// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "AudioInsightsEditorDashboardFactory.h"
#include "Features/IPluginsEditorFeature.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsEditorModule.h"
#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"
#include "Views/VirtualLoopsDebugDraw.h"
#include "Widgets/Docking/SDockTab.h"

namespace UE::Audio::Insights
{
	class FAudioEventLogDashboardViewFactory;

	class FAudioInsightsEditorModule final : public IAudioInsightsEditorModule
	{
	public:
		FAudioInsightsEditorModule() = default;
		virtual ~FAudioInsightsEditorModule() = default;

		FAudioInsightsEditorModule(const FAudioInsightsEditorModule&) = delete;
		FAudioInsightsEditorModule& operator=(const FAudioInsightsEditorModule&) = delete;
		FAudioInsightsEditorModule(FAudioInsightsEditorModule&&) = delete;
		FAudioInsightsEditorModule& operator=(FAudioInsightsEditorModule&&) = delete;

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		virtual void RegisterDashboardViewFactory(TSharedRef<IDashboardViewFactory> InDashboardFactory) override;
		virtual void UnregisterDashboardViewFactory(FName InName) override;
		virtual void RegisterEventLogCategories(const TMap<FString, TSet<FString>>& InCategoriesToEvents) override;
		virtual void RegisterEventLogDisplayNames(const TMap<FString, FText>& InDisplayNames) override;

		virtual ::Audio::FDeviceId GetDeviceId() const override;

		TSharedPtr<FEditorDashboardFactory> GetDashboardFactory();
		const TSharedPtr<FEditorDashboardFactory> GetDashboardFactory() const;

		static bool IsModuleLoaded();
		static FAudioInsightsEditorModule& GetChecked();
		virtual IAudioInsightsTraceModule& GetTraceModule() override;
		virtual class FAudioInsightsCacheManager& GetCacheManager() override;
		virtual class FAudioInsightsTimingViewExtender& GetTimingViewExtender() override;
		virtual class FAudioInsightsDetailsSelectionManager& GetDetailsSelectionManager() override;

	private:
		void RegisterMenus();

		TSharedRef<SDockTab> CreateDashboardTabWidget(const FSpawnTabArgs& Args);

		void OnModularFeatureRegistered(const FName& Name, IModularFeature* ModularFeature);
		void OnModularFeatureUnregistered(const FName& Name, IModularFeature* ModularFeature);
		
		inline static const FName AudioInsightsEditorModuleName = "AudioInsightsEditor";

		TSharedPtr<FEditorDashboardFactory> DashboardFactory;
		TSharedPtr<FAudioEventLogDashboardViewFactory> EventLogViewFactory;

		FVirtualLoopsDebugDraw VirtualLoopsDebugDraw;

		TSharedPtr<FPluginTemplateDescription> AudioInsightsPluginTemplateDescription;
	};
} // namespace UE::Audio::Insights
