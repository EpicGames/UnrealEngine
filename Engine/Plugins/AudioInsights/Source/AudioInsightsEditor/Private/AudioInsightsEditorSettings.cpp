// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInsightsEditorSettings.h"

#include "AudioInsightsLog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Views/AudioEventLogDashboardViewFactory.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioInsightsEditorSettings)

#define LOCTEXT_NAMESPACE "UAudioInsightsEditorSettings"

namespace AudioInsightsEditorSettingsPrivate
{
	FString CreateUniqueName(const FString& DefaultName, TFunctionRef<bool(const FString&)> IsUniqueNameCheck)
	{
		FString NewName = DefaultName;

		for (int NumTries = 1; NumTries < TNumericLimits<int32>::Max(); ++NumTries)
		{
			if (IsUniqueNameCheck(NewName))
			{
				return NewName;
			}

			NewName = DefaultName + TEXT(" ") + FString::FromInt(NumTries);
		}

		// unable to create new name
		return FString();
	}
}

UAudioInsightsEditorSettings::UAudioInsightsEditorSettings()
	: InBuiltAudioLogEventTypes(UE::Audio::Insights::FAudioEventLogDashboardViewFactory::GetInitEventTypeFilters())
{
}

FName UAudioInsightsEditorSettings::GetCategoryName() const
{
	return "Plugins";
}

FText UAudioInsightsEditorSettings::GetSectionText() const
{
	return LOCTEXT("AudioInsightsEditorSettings_SectionText", "Audio Insights (Editor)");
}

FText UAudioInsightsEditorSettings::GetSectionDescription() const
{
	return LOCTEXT("AudioInsightsEditorSettings_SectionDesc", "Configure Audio Insights editor settings.");
}

void UAudioInsightsEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();

	OnRequestReadEventLogSettingsHandle = FAudioEventLogSettings::OnRequestReadSettings.AddLambda([this](){ FAudioEventLogSettings::OnReadSettings.Broadcast(EventLogSettings); });
	OnRequestWriteEventLogSettingsHandle = FAudioEventLogSettings::OnRequestWriteSettings.AddLambda([this]()
	{
		FAudioEventLogSettings::OnWriteSettings.Broadcast(EventLogSettings);
		SaveConfig();
	});

	OnRequestReadSoundDashboardSettingsHandle = FSoundDashboardSettings::OnRequestReadSettings.AddLambda([this](){ FSoundDashboardSettings::OnReadSettings.Broadcast(SoundDashboardSettings); });
	OnRequestWriteSoundDashboardSettingsHandle = FSoundDashboardSettings::OnRequestWriteSettings.AddLambda([this]()
	{
		FSoundDashboardSettings::OnWriteSettings.Broadcast(SoundDashboardSettings);
		SaveConfig();
	});

	OnRequestReadSignalFlowSettingsHandle = FSignalFlowSettings::OnRequestReadSettings.AddLambda([this]()
	{
		FSignalFlowSettings::OnReadSettings.Broadcast(SignalFlowSettings);
	});

	OnRequestWriteSignalFlowSettingsHandle = FSignalFlowSettings::OnRequestWriteSettings.AddLambda([this]()
	{
		FSignalFlowSettings::OnWriteSettings.Broadcast(SignalFlowSettings);
		SaveConfig();
	});
}

void UAudioInsightsEditorSettings::BeginDestroy()
{
	Super::BeginDestroy();

	FAudioEventLogSettings::OnRequestReadSettings.Remove(OnRequestReadEventLogSettingsHandle);
	FAudioEventLogSettings::OnRequestWriteSettings.Remove(OnRequestWriteEventLogSettingsHandle);

	FSoundDashboardSettings::OnRequestReadSettings.Remove(OnRequestReadSoundDashboardSettingsHandle);
	FSoundDashboardSettings::OnRequestWriteSettings.Remove(OnRequestWriteSoundDashboardSettingsHandle);

	FSignalFlowSettings::OnRequestReadSettings.Remove(OnRequestReadSignalFlowSettingsHandle);
	FSignalFlowSettings::OnRequestWriteSettings.Remove(OnRequestWriteSignalFlowSettingsHandle);
}

void UAudioInsightsEditorSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActiveMemberNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();

	if (ActiveMemberNode == nullptr || ActiveMemberNode->GetValue() == nullptr)
	{
		return;
	}

	if (ActiveMemberNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioInsightsEditorSettings, bOnlyTraceAudioChannels))
	{
		OnOnlyTraceAudioChannelsPropertyChanged.Broadcast(bOnlyTraceAudioChannels);
	}

	if (ActiveMemberNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioInsightsEditorSettings, EventLogSettings))
	{
		if (VerifyEventLogInput(*ActiveMemberNode, PropertyChangedEvent))
		{
			FAudioEventLogSettings::OnReadSettings.Broadcast(EventLogSettings);
		}
	}

	if (ActiveMemberNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioInsightsEditorSettings, SoundDashboardSettings))
	{
		FSoundDashboardSettings::OnReadSettings.Broadcast(SoundDashboardSettings);
	}

	if (ActiveMemberNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioInsightsEditorSettings, SignalFlowSettings))
	{
		FSignalFlowSettings::OnReadSettings.Broadcast(SignalFlowSettings);
	}
}

void UAudioInsightsEditorSettings::PostEditUndo()
{
	Super::PostEditUndo();

	FAudioEventLogSettings::OnReadSettings.Broadcast(EventLogSettings);
	FSoundDashboardSettings::OnReadSettings.Broadcast(SoundDashboardSettings);
	FSignalFlowSettings::OnReadSettings.Broadcast(SignalFlowSettings);
}

bool UAudioInsightsEditorSettings::VerifyEventLogInput(const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode& ActiveMemberNode, FPropertyChangedChainEvent& PropertyChangedEvent)
{
	check(ActiveMemberNode.GetValue() != nullptr);

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* CustomCategoryNode = ActiveMemberNode.GetNextNode();

	if (CustomCategoryNode == nullptr 
		|| CustomCategoryNode->GetValue() == nullptr 
		|| CustomCategoryNode->GetValue()->GetFName() != GET_MEMBER_NAME_CHECKED(FAudioEventLogSettings, CustomCategoriesToEvents))
	{
		// This change is not related to custom events - no need to verify anything
		return true;
	}

	if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayRemove) 
		|| (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayClear))
	{
		// If the operation is a delete, no need to verify any new event names
		// We will want to auto-remove any events still cached in the event filter
		RefreshEventFilter();
		return true;
	}

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* CustomEventsNode = CustomCategoryNode->GetNextNode();
	if (CustomEventsNode
		&& CustomEventsNode->GetValue()
		&& CustomEventsNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(FAudioEventLogCustomEvents, EventNames))
	{
		return VerifyCustomEventNameEdit(PropertyChangedEvent, CastField<FMapProperty>(CustomCategoryNode->GetValue()), CastField<FSetProperty>(CustomEventsNode->GetValue()));
	}
	else
	{
		return VerifyCategoryEdit();
	}
}

void UAudioInsightsEditorSettings::RefreshEventFilter()
{
	// If custom events have been removed, we want to remove them from the event filter too
	TArray<FString> FiltersToRemove;

	for (TSet<FString>::TConstIterator It = EventLogSettings.EventFilters.CreateConstIterator(); It; ++It)
	{
		bool bFoundMatch = false;

		// Check custom categories
		for (const auto& [FilterCategory, CustomEvents] : EventLogSettings.CustomCategoriesToEvents)
		{
			if (CustomEvents.EventNames.Contains(*It))
			{
				bFoundMatch = true;
				break;
			}
		}

		if (!bFoundMatch)
		{
			// Now check the in-built Event Types in Audio Insights
			for (const auto& [FilterCategory, InBuiltEvents] : InBuiltAudioLogEventTypes)
			{
				if (InBuiltEvents.Contains(*It))
				{
					bFoundMatch = true;
					break;
				}
			}
		}
		
		if (!bFoundMatch)
		{
			// This event filter no longer exists - mark it as being removed
			FiltersToRemove.Add(*It);
		}
	}

	for (const FString& FilterToRemove : FiltersToRemove)
	{
		EventLogSettings.EventFilters.Remove(FilterToRemove);
	}
}

bool UAudioInsightsEditorSettings::VerifyCategoryEdit()
{
	using namespace AudioInsightsEditorSettingsPrivate;

	// TMap will already check to ensure there are no duplicate Keys
	// If we detect an empty field, try to create a unique default name to replace them
	const FString EmptyCategoryName = FString();
	if (FAudioEventLogCustomEvents* Events = EventLogSettings.CustomCategoriesToEvents.Find(EmptyCategoryName))
	{
		// Create a unique category name if the category is empty
		const FString NewCategoryName = CreateUniqueName(TEXT("New Category"), [this](const FString& NameToCheck)
		{
			return !EventLogSettings.CustomCategoriesToEvents.Contains(NameToCheck);
		});

		// CreateUniqueName() should be able to create a name for the new category - warn and return out if this has failed
		ensureMsgf(!NewCategoryName.IsEmpty(), TEXT("Failed to create a unique name for edited custom Audio Event Log category"));
		if (NewCategoryName.IsEmpty())
		{
			return false;
		}

		// Move the events into the newly named category - delete the empty one
		EventLogSettings.CustomCategoriesToEvents.Add(NewCategoryName, MoveTemp(*Events));
		EventLogSettings.CustomCategoriesToEvents.Remove(EmptyCategoryName);
	}

	return true;
}

bool UAudioInsightsEditorSettings::VerifyCustomEventNameEdit(const FPropertyChangedChainEvent& PropertyChangedEvent, const FMapProperty* CategoryToEventMapProperty, const FSetProperty* EventNamesSetProperty)
{
	using namespace AudioInsightsEditorSettingsPrivate;

	// All events need to have unique names - including the in-built event names in the event log
	// Check both the custom event names and the in built event names for any duplicates, and update the name if found
	// Also - do not allow empty event names, auto-generate a unique name in this case

	FString* EditedCategoryName = GetEditedCategoryKey(PropertyChangedEvent, CategoryToEventMapProperty);
	if (EditedCategoryName == nullptr)
	{
		return false;
	}

	FAudioEventLogCustomEvents* Events = EventLogSettings.CustomCategoriesToEvents.Find(*EditedCategoryName);
	if (Events == nullptr)
	{
		UE_LOGF(LogAudioInsights, Warning, "Could not find category with name %ls in EventLogSettings.CustomCategoriesToEvents", **EditedCategoryName);
		return false;
	}

	const EditedEvent EditedEvent = GetEditedEvent(PropertyChangedEvent, EventNamesSetProperty, *Events);
	if (EditedEvent.EventName == nullptr)
	{
		return false;
	}

	// We have found the event - now validate that it is unique and not empty - edit if neccessary
	if (EditedEvent.EventName->IsEmpty() || !IsEventNameUnique(*EditedEvent.EventName, *EditedCategoryName, EditedEvent.LogicalIndex))
	{
		const FString NewEventName = CreateUniqueName(EditedEvent.EventName->IsEmpty() ? TEXT("New Event") : *EditedEvent.EventName,
		[this, &EditedCategoryName, &EditedEvent](const FString& NameToCheck)
		{
			return IsEventNameUnique(NameToCheck, *EditedCategoryName, EditedEvent.LogicalIndex);
		});

		// CreateUniqueName() should be able to create a name for the new event - warn and return out if this has failed
		ensureMsgf(!NewEventName.IsEmpty(), TEXT("Failed to create a unique name for edited custom Audio Event Log event"));
		if (NewEventName.IsEmpty())
		{
			return false;
		}

		if (!EditedEvent.EventName->IsEmpty())
		{
			static const FTextFormat SubTextFormat = FTextFormat::FromString(TEXT("{0} {1}"));
			const FText SubText = FText::Format(SubTextFormat, LOCTEXT("AudioInsightsEditorSettings_DuplicateCustomNameSubtext", "Duplicate names are not allowed. The event name has been changed to : "), FText::FromString(NewEventName));

			// This was a event name duplication fail - notify the user that the event name is being changed
			ShowNotification(LOCTEXT("AudioInsightsEditorSettings_DuplicateCustomName", "Event name already exists in the Event Log."), SubText);
		}

		// Update the set with the new name and delete the old one
		Events->EventNames.Remove(*EditedEvent.EventName);
		Events->EventNames.Add(NewEventName);
	}

	return true;
}

FString* UAudioInsightsEditorSettings::GetEditedCategoryKey(const FPropertyChangedChainEvent& PropertyChangedEvent, const FMapProperty* CategoryToEventMapProperty) const
{
	ensureMsgf(CategoryToEventMapProperty, TEXT("Invalid FMapProperty* CategoryToEventMapProperty passed in"));
	if (CategoryToEventMapProperty == nullptr)
	{
		return nullptr;
	}

	// First find the specific event that has been edited using the CategoryToEventMapProperty
	FScriptMapHelper MapHelper(CategoryToEventMapProperty, CategoryToEventMapProperty->ContainerPtrToValuePtr<void>(&EventLogSettings));
	const int32 ContainerIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FAudioEventLogSettings, CustomCategoriesToEvents).ToString());
	const int32 MapIndex = MapHelper.FindInternalIndex(ContainerIndex);

	if (!MapHelper.IsValidIndex(MapIndex))
	{
		UE_LOGF(LogAudioInsights, Warning, "Detected invalid index in CustomCategoriesToEvents TMap when trying to validate custom event log edit");
		return nullptr;
	}

	FString* EditedCategoryName = reinterpret_cast<FString*>(MapHelper.GetKeyPtr(MapIndex));
	ensure(EditedCategoryName);
	return EditedCategoryName;
}

UAudioInsightsEditorSettings::EditedEvent UAudioInsightsEditorSettings::GetEditedEvent(const FPropertyChangedChainEvent& PropertyChangedEvent, const FSetProperty* EditedEventsSetProperty, const FAudioEventLogCustomEvents& Events) const
{
	ensureMsgf(EditedEventsSetProperty, TEXT("Invalid FSetProperty* EditedEventsSetProperty passed in"));
	if (EditedEventsSetProperty == nullptr)
	{
		return { nullptr };
	}

	FScriptSetHelper SetHelper(EditedEventsSetProperty, EditedEventsSetProperty->ContainerPtrToValuePtr<void>(&Events));

	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayAdd)
	{
		// If the change is an add action, we cannot rely on GetArrayIndex to return the correct index
		// Newly added elements will be empty - locate the index via this method
		for (FScriptSetHelper::FIterator Iterator = SetHelper.CreateIterator(); Iterator; ++Iterator)
		{
			FString* EventName = reinterpret_cast<FString*>(SetHelper.GetElementPtr(Iterator));
			ensure(EventName);

			if (EventName && EventName->IsEmpty())
			{
				return { EventName, Iterator.GetLogicalIndex() };
			}
		}

		UE_LOGF(LogAudioInsights, Warning, "Could not locate newly added event name element in EventNames set");
		return { nullptr };
	}
	else
	{
		const int32 EditedEventIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FAudioEventLogCustomEvents, EventNames).ToString());
		const int32 SetIndex = SetHelper.FindInternalIndex(EditedEventIndex);
		if (!SetHelper.IsValidIndex(SetIndex))
		{
			UE_LOGF(LogAudioInsights, Warning, "Detected invalid index in EventNames TSet when trying to validate custom event log edit");
			return { nullptr };
		}

		FString* EventName = reinterpret_cast<FString*>(SetHelper.GetElementPtr(SetIndex));
		ensure(EventName);

		if (EventName)
		{
			return { EventName, EditedEventIndex };
		}
	}

	return { nullptr };
}

bool UAudioInsightsEditorSettings::IsEventNameUnique(const FString& EditedEventName, const FString& EditedFilterCategory, const int32 EditedLogicalIndex) const
{
	if (EditedEventName.IsEmpty() || EditedFilterCategory.IsEmpty() || EditedLogicalIndex == INDEX_NONE)
	{
		return false;
	}
	
	// Check custom categories
	for (const auto& [FilterCategory, CustomEvents] : EventLogSettings.CustomCategoriesToEvents)
	{
		if (CustomEvents.EventNames.Contains(EditedEventName))
		{
			if (EditedFilterCategory == FilterCategory)
			{
				const int32 FoundEventIndex = CustomEvents.EventNames.Array().IndexOfByKey(EditedEventName);
				if (FoundEventIndex != INDEX_NONE && FoundEventIndex == EditedLogicalIndex)
				{
					// This is the value currently being edited, skip
					continue;
				}
			}

			// This is a duplicate
			return false;
		}
	}

	// Now check the in-built Event Types in Audio Insights
	for (const auto& [FilterCategory, InBuiltEvents] : InBuiltAudioLogEventTypes)
	{
		if (InBuiltEvents.Contains(EditedEventName))
		{
			// No need to check if this is the value edited - users cannot edit in-built categories
			// This is a duplicate
			return false;
		}
	}

	return true;
}

void UAudioInsightsEditorSettings::ShowNotification(const FText& TitleText, const FText& SubText) const
{
	// Show floating notification
	FNotificationInfo Info(TitleText);

	Info.SubText = SubText;
	Info.bFireAndForget = true;
	Info.FadeOutDuration = 1.0f;
	Info.ExpireDuration = 4.0f;

	FSlateNotificationManager::Get().AddNotification(Info);
}

#undef LOCTEXT_NAMESPACE
