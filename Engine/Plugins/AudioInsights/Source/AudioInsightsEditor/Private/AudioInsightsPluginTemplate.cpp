// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInsightsPluginTemplate.h"

#include "PluginDescriptor.h"

FAudioInsightsPluginTemplateDescription::FAudioInsightsPluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath)
	: FPluginTemplateDescription(InName, InDescription, InOnDiskPath, /*bCanContainContent=*/ true, EHostType::EditorAndProgram)
{
}

void FAudioInsightsPluginTemplateDescription::UpdatePluginNameTextWhenTemplateSelected(FText& OutPluginNameText)
{
	OutPluginNameText = FText::FromString("AudioInsightsTemplate");
}

void FAudioInsightsPluginTemplateDescription::UpdatePluginNameTextWhenTemplateUnselected(FText& OutPluginNameText)
{
	OutPluginNameText = FText::GetEmpty();
}

void FAudioInsightsPluginTemplateDescription::CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor)
{
	FPluginReferenceDescriptor& NewPluginRefDesc = Descriptor.Plugins.AddDefaulted_GetRef();
	
	NewPluginRefDesc.Name = "AudioInsights";
	NewPluginRefDesc.bEnabled = true;
	NewPluginRefDesc.bOptional = false;
}
