// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IPluginsEditorFeature.h"

#define UE_API AUDIOINSIGHTSEDITOR_API

/**
 * Used to create custom templates for Audio Insights.
 */
struct FAudioInsightsPluginTemplateDescription : FPluginTemplateDescription
{
	UE_API FAudioInsightsPluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath);
	
	// FPluginTemplateDescription overrides
	UE_API virtual void UpdatePluginNameTextWhenTemplateSelected(FText& OutPluginNameText) override;
	UE_API virtual void UpdatePluginNameTextWhenTemplateUnselected(FText& OutPluginNameText) override;
	UE_API virtual void CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor) override;
};

#undef UE_API
