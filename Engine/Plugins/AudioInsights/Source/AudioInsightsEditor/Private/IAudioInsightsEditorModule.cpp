// Copyright Epic Games, Inc. All Rights Reserved.
#include "IAudioInsightsEditorModule.h"

#include "AudioInsightsEditorModule.h"
#include "Modules/ModuleManager.h"

IAudioInsightsTraceModule& IAudioInsightsEditorModule::GetTraceModule()
{
	IAudioInsightsEditorModule& AudioInsightsModule = static_cast<IAudioInsightsEditorModule&>(UE::Audio::Insights::FAudioInsightsEditorModule::GetChecked());
	return AudioInsightsModule.GetTraceModule();
}

bool IAudioInsightsEditorModule::IsModuleLoaded()
{
	return UE::Audio::Insights::FAudioInsightsEditorModule::IsModuleLoaded();
}

IAudioInsightsEditorModule& IAudioInsightsEditorModule::GetChecked()
{
	return static_cast<IAudioInsightsEditorModule&>(UE::Audio::Insights::FAudioInsightsEditorModule::GetChecked());
}
