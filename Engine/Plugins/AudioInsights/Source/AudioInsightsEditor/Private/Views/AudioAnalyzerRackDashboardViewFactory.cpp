// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/AudioAnalyzerRackDashboardViewFactory.h"

#include "Analyzers/SubmixAudioAnalyzerRack.h"
#include "AudioInsightsEditorDashboardFactory.h"
#include "AudioInsightsEditorModule.h"
#include "AudioInsightsStyle.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundSubmix.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	FName FAudioAnalyzerRackDashboardViewFactory::GetName() const
	{
		return "AudioAnalyzerRack";
	}

	FText FAudioAnalyzerRackDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_DashboardsAnalyzerRackTab_DisplayName", "Analyzers");
	}

	EDefaultDashboardTabStack FAudioAnalyzerRackDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::AudioAnalyzerRack;
	}

	FSlateIcon FAudioAnalyzerRackDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon");
	}

	TSharedRef<SWidget> FAudioAnalyzerRackDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		// Create SubmixAudioAnalyzerRack (use MainSubmix as default)
		if (TWeakObjectPtr<UAudioSettings> AudioSettings = GetMutableDefault<UAudioSettings>();
			AudioSettings.IsValid())
		{
			MainSubmix = Cast<USoundSubmix>(AudioSettings->MasterSubmix.ResolveObject());

			if (MainSubmix)
			{
				SubmixAudioAnalyzerRack = MakeShared<FSubmixAudioAnalyzerRack>(MainSubmix);

				TSharedPtr<FEditorDashboardFactory> DashboardFactory = FAudioInsightsEditorModule::GetChecked().GetDashboardFactory();
				if (DashboardFactory.IsValid())
				{
					DashboardFactory->OnActiveAudioDeviceChanged.RemoveAll(this);
					DashboardFactory->OnActiveAudioDeviceChanged.AddSP(this, &FAudioAnalyzerRackDashboardViewFactory::HandleOnActiveAudioDeviceChanged);
				}
			}
		}
		
		if (!SubmixAudioAnalyzerRack.IsValid())
		{
			return SNew(SColorBlock)
				.Color(FSlateStyle::Get().GetColor("AudioInsights.Analyzers.BackgroundColor"));
		}

		return SubmixAudioAnalyzerRack->MakeWidget(OwnerTab, SpawnTabArgs);
	}

	void FAudioAnalyzerRackDashboardViewFactory::HandleOnActiveAudioDeviceChanged()
	{
		if (SubmixAudioAnalyzerRack.IsValid())
		{
			SubmixAudioAnalyzerRack->RebuildAudioAnalyzerRack(MainSubmix);
		}
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
