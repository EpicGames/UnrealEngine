// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Views/DashboardViewFactory.h"

class USoundSubmix;

namespace UE::Audio::Insights
{
	class FSubmixAudioAnalyzerRack;

	class FAudioAnalyzerRackDashboardViewFactory : public IDashboardViewFactory, public TSharedFromThis<FAudioAnalyzerRackDashboardViewFactory>
	{
	public:
		virtual FName GetName() const override;
		virtual FText GetDisplayName() const override;
		virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		virtual FSlateIcon GetIcon() const override;
		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	private:
		void HandleOnActiveAudioDeviceChanged();

		TSharedPtr<FSubmixAudioAnalyzerRack> SubmixAudioAnalyzerRack;
		TObjectPtr<USoundSubmix> MainSubmix;
	};
} // namespace UE::Audio::Insights
