// Copyright Epic Games, Inc. All Rights Reserved.
#include "Views/DetailsDashboardViewFactory.h"

#include "AudioInsightsDashboardAssetCommands.h"
#include "AudioInsightsDetailsSelectionManager.h"
#include "AudioInsightsStyle.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "IAudioInsightsModule.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AudioInsights"

namespace UE::Audio::Insights
{
	void SDetailsDashboardWindow::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
	{
		CommandList = InCommandList;
		SOverlay::Construct(SOverlay::FArguments());
	}

	FReply SDetailsDashboardWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return SOverlay::OnKeyDown(MyGeometry, InKeyEvent);
	}

	FName FDetailsDashboardViewFactory::GetName() const
	{
		return "Details";
	}

	FText FDetailsDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioDashboard_DetailsTab_DisplayName", "Details");
	}

	EDefaultDashboardTabStack FDetailsDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Log;
	}

	FSlateIcon FDetailsDashboardViewFactory::GetIcon() const
	{
		return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Details");
	}

	TSharedRef<SWidget> FDetailsDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (!CommandList.IsValid())
		{
			BindCommands();
		}

		if (!DashboardWidget.IsValid())
		{
			DashboardWidget = CreateDetailsViewWidget();
		}

		if (!AssetSelectionChangedHandle.IsValid())
		{
			BindOnAssetChangedDelegate();
		}

		return DashboardWidget.ToSharedRef();
	}

	TSharedRef<SWidget> FDetailsDashboardViewFactory::CreateAssetHeaderWidget()
	{
		return SNew(SBox)
			.Padding(8.0f, 4.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SAssignNew(AssetIconImage, SImage)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(AssetNameText, STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.ConstantTextBlockStyle"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					MakeAssetMenuBar()
				]
			];
	}

	TSharedRef<SDetailsDashboardWindow> FDetailsDashboardViewFactory::CreateDetailsViewWidget()
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bShowObjectLabel = false;

		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		DetailsViewWidget = SNew(SVerticalBox)
			.Visibility(EVisibility::Collapsed)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				CreateAssetHeaderWidget()
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				DetailsView->AsShared()
			];

		NoSelectionMessageWidget = SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AudioDashboard_Details_NoSelection", "Select an audio asset to view its properties."))
			];

		TSharedRef<SDetailsDashboardWindow> Window = SNew(SDetailsDashboardWindow, CommandList.ToSharedRef());

		Window->AddSlot()
		[
			NoSelectionMessageWidget.ToSharedRef()
		];

		Window->AddSlot()
		[
			DetailsViewWidget.ToSharedRef()
		];

		return Window;
	}

	void FDetailsDashboardViewFactory::BindOnAssetChangedDelegate()
	{
		FAudioInsightsDetailsSelectionManager& SelectionManager = IAudioInsightsModule::GetChecked().GetDetailsSelectionManager();
		AssetSelectionChangedHandle = SelectionManager.OnAssetSelectionChanged.AddSP(this, &FDetailsDashboardViewFactory::OnAssetSelectionChanged);

		const TWeakObjectPtr<UObject> CurrentAsset = SelectionManager.GetSelectedAsset();
		if (CurrentAsset.IsValid())
		{
			OnAssetSelectionChanged(CurrentAsset.Get());
		}
	}

	void FDetailsDashboardViewFactory::OnAssetSelectionChanged(const TObjectPtr<UObject> InAsset)
	{
		if (!DetailsView.IsValid() || !DetailsViewWidget.IsValid() || !NoSelectionMessageWidget.IsValid() || !DashboardWidget.IsValid() || !AssetIconImage.IsValid() || !AssetNameText.IsValid())
		{
			return;
		}

		if (InAsset)
		{
			DetailsView->SetObject(InAsset);

			AssetIconImage->SetImage(FSlateIconFinder::FindIconForClass(InAsset->GetClass()).GetIcon());
			AssetNameText->SetText(FText::FromName(InAsset->GetFName()));
			AssetNameText->SetToolTipText(FText::FromName(InAsset->GetFName()));

			DetailsViewWidget->SetVisibility(EVisibility::Visible);
			NoSelectionMessageWidget->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			DetailsView->SetObject(nullptr);

			AssetIconImage->SetImage(nullptr);
			AssetNameText->SetText(FText::GetEmpty());
			AssetNameText->SetToolTipText(FText::GetEmpty());

			DetailsViewWidget->SetVisibility(EVisibility::Collapsed);
			NoSelectionMessageWidget->SetVisibility(EVisibility::Visible);
		}
	}

	void FDetailsDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();
		const FDashboardAssetCommands& Commands = FDashboardAssetCommands::Get();

		CommandList->MapAction(Commands.GetOpenCommand(), FExecuteAction::CreateSP(this, &FDetailsDashboardViewFactory::OpenAsset));
		CommandList->MapAction(Commands.GetBrowserSyncCommand(), FExecuteAction::CreateSP(this, &FDetailsDashboardViewFactory::BrowseToAsset));
		CommandList->MapAction(Commands.GetSaveCommand(), FExecuteAction::CreateSP(this, &FDetailsDashboardViewFactory::SaveAsset), FCanExecuteAction::CreateSP(this, &FDetailsDashboardViewFactory::CanSaveAsset));
		CommandList->MapAction(Commands.GetSaveAllCommand(), FExecuteAction::CreateSP(this, &FDetailsDashboardViewFactory::SaveAllAssets));
	}

	TSharedRef<SWidget> FDetailsDashboardViewFactory::MakeAssetMenuBar() const
	{
		const FDashboardAssetCommands& Commands = FDashboardAssetCommands::Get();

		FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);
		Commands.AddAssetCommands(ToolbarBuilder);

		ToolbarBuilder.AddToolBarButton(
			Commands.GetSaveCommand(),
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			TAttribute<FSlateIcon>::Create([]() { return FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"); }),
			"Save"
		);

		ToolbarBuilder.AddToolBarButton(
			Commands.GetSaveAllCommand(),
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			TAttribute<FSlateIcon>::Create([]() { return FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.SaveAll"); }),
			"SaveAll"
		);

		return ToolbarBuilder.MakeWidget();
	}

	TObjectPtr<UObject> FDetailsDashboardViewFactory::GetSelectedEditableAsset() const
	{
		const FAudioInsightsDetailsSelectionManager& SelectionManager = IAudioInsightsModule::GetChecked().GetDetailsSelectionManager();
		const TWeakObjectPtr<UObject> Asset = SelectionManager.GetSelectedAsset();

		if (Asset.IsValid() && Asset->IsAsset())
		{
			return Asset.Get();
		}

		return nullptr;
	}

	void FDetailsDashboardViewFactory::OpenAsset() const
	{
		if (GEditor == nullptr)
		{
			return;
		}

		const TObjectPtr<UObject> Asset = GetSelectedEditableAsset();
		if (Asset == nullptr)
		{
			return;
		}

		const TObjectPtr<UAssetEditorSubsystem> AssetSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (AssetSubsystem)
		{
			AssetSubsystem->OpenEditorForAsset(Asset);
		}
	}

	void FDetailsDashboardViewFactory::BrowseToAsset() const
	{
		if (GEditor == nullptr)
		{
			return;
		}

		const TObjectPtr<UObject> Asset = GetSelectedEditableAsset();
		if (Asset == nullptr)
		{
			return;
		}

		GEditor->SyncBrowserToObject(Asset);
	}

	void FDetailsDashboardViewFactory::SaveAsset() const
	{
		UPackage* const Package = GetAssetPackage();
		if (Package == nullptr || !Package->IsDirty())
		{
			return;
		}

		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(Package);

		constexpr bool bCheckDirty = true;
		constexpr bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
	}

	void FDetailsDashboardViewFactory::SaveAllAssets() const
	{
		constexpr bool bPromptUserToSave = true;
		constexpr bool bSaveMapPackages = false;
		constexpr bool bSaveContentPackages = true;
		constexpr bool bFastSave = false;

		FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave);
	}

	bool FDetailsDashboardViewFactory::CanSaveAsset() const
	{
		const UPackage* const Package = GetAssetPackage();
		return Package != nullptr && Package->IsDirty();
	}

	UPackage* FDetailsDashboardViewFactory::GetAssetPackage() const
	{
		const TObjectPtr<UObject> Asset = GetSelectedEditableAsset();
		if (Asset == nullptr)
		{
			return nullptr;
		}

		return Asset->GetOutermost();
	}
} // namespace UE::Audio::Insights

#undef LOCTEXT_NAMESPACE
