// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"

class IDetailsView;
class SImage;
class STextBlock;

namespace UE::Audio::Insights
{
	class SDetailsDashboardWindow final : public SOverlay
	{
	public:
		SLATE_BEGIN_ARGS(SDetailsDashboardWindow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);

		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual bool SupportsKeyboardFocus() const override { return true; }

	private:
		TSharedPtr<FUICommandList> CommandList;
	};

	class FDetailsDashboardViewFactory final : public IDashboardViewFactory, public TSharedFromThis<FDetailsDashboardViewFactory>
	{
	public:
		FDetailsDashboardViewFactory() = default;
		~FDetailsDashboardViewFactory() = default;

		virtual FName GetName() const override;
		virtual FText GetDisplayName() const override;
		virtual EDefaultDashboardTabStack GetDefaultTabStack() const override;
		virtual FSlateIcon GetIcon() const override;
		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

	private:
		TSharedRef<SWidget> CreateAssetHeaderWidget();
		TSharedRef<SDetailsDashboardWindow> CreateDetailsViewWidget();
		void BindCommands();
		void BindOnAssetChangedDelegate();

		void OnAssetSelectionChanged(const TObjectPtr<UObject> InAsset);

		TSharedRef<SWidget> MakeAssetMenuBar() const;
		TObjectPtr<UObject> GetSelectedEditableAsset() const;

		void OpenAsset() const;
		void BrowseToAsset() const;
		void SaveAsset() const;
		void SaveAllAssets() const;
		bool CanSaveAsset() const;
		UPackage* GetAssetPackage() const;

		TSharedPtr<IDetailsView> DetailsView;
		TSharedPtr<SWidget> DetailsViewWidget;
		TSharedPtr<SWidget> NoSelectionMessageWidget;
		TSharedPtr<SDetailsDashboardWindow> DashboardWidget;

		TSharedPtr<FUICommandList> CommandList;
		TSharedPtr<SImage> AssetIconImage;
		TSharedPtr<STextBlock> AssetNameText;

		FDelegateHandle AssetSelectionChangedHandle;
	};
} // namespace UE::Audio::Insights
