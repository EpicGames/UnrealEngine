// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundAttenuationVisualizer.h"

#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeAttenuation.h"

namespace UE::Audio::Insights
{
	FSoundAttenuationVisualizer::FSoundAttenuationVisualizer(const FColor& InColor)
		: Color(InColor)
	{
		
	}
	
	void FSoundAttenuationVisualizer::Draw(float InDeltaTime, const FTransform& InTransform, const UObject& InObject, const UWorld& InWorld) const
	{
		if (LastObjectId != InObject.GetUniqueID())
		{
			ShapeDetailsMap.Reset();
			
			if (const USoundCue* SoundCue = Cast<const USoundCue>(&InObject))
			{
				TArray<const USoundNodeAttenuation*> AttenuationNodes;
				SoundCue->RecursiveFindAttenuation(SoundCue->FirstNode, AttenuationNodes);
				
				for (const USoundNodeAttenuation* Node : AttenuationNodes)
				{
					if (Node)
					{
						if (const FSoundAttenuationSettings* AttenuationSettingsToApply = Node->GetAttenuationSettingsToApply())
						{
							AttenuationSettingsToApply->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
						}
					}
				}
			}
			else if (const USoundBase* SoundBase = Cast<USoundBase>(&InObject))
			{
				if (const FSoundAttenuationSettings* Settings = SoundBase->GetAttenuationSettingsToApply())
				{
					Settings->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
				}
			}
			else
			{
				return;
			}
		}
		
		LastObjectId = InObject.GetUniqueID();

		for (const auto& [AttenuationShape, ShapeDetails] : ShapeDetailsMap)
		{
			switch (AttenuationShape)
			{
				case EAttenuationShape::Sphere:
				{
					if (ShapeDetails.Falloff > 0.f)
					{
						DrawDebugSphere(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X + ShapeDetails.Falloff, 10, Color);
						DrawDebugSphere(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X, 10, Color);
					}
					else
					{
						DrawDebugSphere(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X, 10, Color);
					}
					break;
				}

				case EAttenuationShape::Box:
				{
					if (ShapeDetails.Falloff > 0.f)
					{
						DrawDebugBox(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents + FVector(ShapeDetails.Falloff), InTransform.GetRotation(), Color);
						DrawDebugBox(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents, InTransform.GetRotation(), Color);
					}
					else
					{
						DrawDebugBox(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents, InTransform.GetRotation(), Color);
					}
					break;
				}

				case EAttenuationShape::Capsule:
				{
					if (ShapeDetails.Falloff > 0.f)
					{
						DrawDebugCapsule(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X + ShapeDetails.Falloff, ShapeDetails.Extents.Y + ShapeDetails.Falloff, InTransform.GetRotation(), Color);
						DrawDebugCapsule(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X, ShapeDetails.Extents.Y, InTransform.GetRotation(), Color);
					}
					else
					{
						DrawDebugCapsule(&InWorld, InTransform.GetTranslation(), ShapeDetails.Extents.X, ShapeDetails.Extents.Y, InTransform.GetRotation(), Color);
					}
					break;
				}

				case EAttenuationShape::Cone:
				{
					const FVector Origin = InTransform.GetTranslation() - (InTransform.GetUnitAxis(EAxis::X) * ShapeDetails.ConeOffset);

					if (ShapeDetails.Falloff > 0.f || ShapeDetails.Extents.Z > 0.f)
					{
						const float OuterAngle = FMath::DegreesToRadians(ShapeDetails.Extents.Y + ShapeDetails.Extents.Z);
						const float InnerAngle = FMath::DegreesToRadians(ShapeDetails.Extents.Y);
						DrawDebugCone(&InWorld, Origin, InTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.Falloff + ShapeDetails.ConeOffset, OuterAngle, OuterAngle, 10, Color);
						DrawDebugCone(&InWorld, Origin, InTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.ConeOffset, InnerAngle, InnerAngle, 10, Color);
					}
					else
					{
						const float Angle = FMath::DegreesToRadians(ShapeDetails.Extents.Y);
						DrawDebugCone(&InWorld, Origin, InTransform.GetUnitAxis(EAxis::X), ShapeDetails.Extents.X + ShapeDetails.ConeOffset, Angle, Angle, 10, Color);
					}

					if (!FMath::IsNearlyZero(ShapeDetails.ConeSphereRadius, UE_KINDA_SMALL_NUMBER))
					{
						if (ShapeDetails.ConeSphereFalloff > 0.f)
						{

							DrawDebugSphere(&InWorld, Origin, ShapeDetails.ConeSphereRadius + ShapeDetails.ConeSphereFalloff, 10, Color);
							DrawDebugSphere(&InWorld, Origin, ShapeDetails.ConeSphereRadius, 10, Color);
						}
						else
						{
							DrawDebugSphere(&InWorld, Origin, ShapeDetails.ConeSphereRadius, 10, Color);
						}
					}

					break;
				}

				default:
				{
					break;
				}
			}
		}
	}
} // namespace UE::Audio::Insights
