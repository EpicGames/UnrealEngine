// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "Engine/Attenuation.h"
#include "Math/Color.h"

namespace UE::Audio::Insights
{
	class IDashboardDataViewEntry;

	class FSoundAttenuationVisualizer
	{
	public:
		explicit FSoundAttenuationVisualizer(const FColor& InColor);
		void Draw(float InDeltaTime, const FTransform& InTransform, const UObject& InObject, const UWorld& InWorld) const;
		
		const FColor& GetColor() const { return Color; }

	private:
		const FColor Color { 155, 155, 255 };
		
		mutable TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails> ShapeDetailsMap;
		mutable uint32 LastObjectId = INDEX_NONE;
	};
} // namespace UE::Audio::Insights
