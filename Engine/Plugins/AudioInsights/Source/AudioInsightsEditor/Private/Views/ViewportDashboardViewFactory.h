// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsStyle.h"
#include "Views/DashboardViewFactory.h"
#include "Widgets/SNullWidget.h"
// #include "LevelEditor.h"
// #include "SLevelViewport.h"

#define LOCTEXT_NAMESPACE "AudioInsights"


namespace UE::Audio::Insights
{
	class FViewportDashboardViewFactory : public IDashboardViewFactory
	{
	public:
		virtual FName GetName() const override
		{
			return "Viewport";
		}

		virtual FText GetDisplayName() const override
		{
			return LOCTEXT("AudioDashboard_ViewportTab_DisplayName", "Viewport");
		}

		virtual EDefaultDashboardTabStack GetDefaultTabStack() const override
		{
			return EDefaultDashboardTabStack::Viewport;
		}

		virtual FSlateIcon GetIcon() const override
		{
			return FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Viewport");
		}

		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override
		{
			// TODO: This test currently hijacks the main level viewport which is not ideal.
			// Find way to make our own instance and bind to tab here.
			// FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			// TSharedPtr<SLevelViewport> LevelViewport = LevelEditor.GetFirstActiveLevelViewport();
			// ViewportContent = LevelViewport->AsShared();

			return SNullWidget::NullWidget;
		}
	};
} // namespace UE::Audio::Insights
#undef LOCTEXT_NAMESPACE
