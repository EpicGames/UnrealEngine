// Copyright Epic Games, Inc. All Rights Reserved.
#include "VirtualLoopsDebugDraw.h"

#include "AudioInsightsDataSource.h"
#include "AudioDeviceManager.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "Messages/VirtualLoopTraceMessages.h"
#include "Views/VirtualLoopDashboardViewFactory.h"

namespace UE::Audio::Insights
{
	namespace VirtualLoopsDebugDrawPrivate
	{
		const FVirtualLoopDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FVirtualLoopDashboardEntry&>(InData);
		};
	}

	FVirtualLoopsDebugDraw::FVirtualLoopsDebugDraw()
		: AttenuationVisualizer(FColor::Blue)
	{
		FVirtualLoopDashboardViewFactory::OnDebugDrawEntries.AddRaw(this, &FVirtualLoopsDebugDraw::DebugDraw);
	}

	FVirtualLoopsDebugDraw::~FVirtualLoopsDebugDraw()
	{
		FVirtualLoopDashboardViewFactory::OnDebugDrawEntries.RemoveAll(this);
	}

	void FVirtualLoopsDebugDraw::DebugDrawEntries(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const
	{
		for (const TSharedPtr<IDashboardDataViewEntry>& SelectedEntry : InSelectedItems)
		{
			if (!SelectedEntry.IsValid())
			{
				continue;
			}
			
			const FVirtualLoopDashboardEntry& LoopData = VirtualLoopsDebugDrawPrivate::CastEntry(*SelectedEntry);
			
			const FRotator& Rotator   = LoopData.Rotator;
			const FVector& Location   = LoopData.Location;
			const FString Description = FString::Printf(TEXT("%s [Virt: %.2fs]"), *LoopData.Name, LoopData.TimeVirtualized);

			const TArray<UWorld*> Worlds = FAudioDeviceManager::Get()->GetWorldsUsingAudioDevice(InAudioDeviceId);

			for (UWorld* World : Worlds)
			{
				DrawDebugSphere(World, Location, 30.0f, 8, AttenuationVisualizer.GetColor(), false, InElapsed, SDPG_Foreground);
				DrawDebugString(World, Location + FVector(0, 0, 32), *Description, nullptr, AttenuationVisualizer.GetColor(), InElapsed, false, 1.0f);

				if (const TObjectPtr<const UObject> Object = LoopData.GetObject())
				{
					FTransform Transform;
					Transform.SetLocation(Location);
					Transform.SetRotation(FQuat(Rotator));

					AttenuationVisualizer.Draw(InElapsed, Transform, *Object, *World);
				}
			}
		}
	}

	void FVirtualLoopsDebugDraw::DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId)
	{
		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (AudioDeviceManager)
		{
			AudioDeviceManager->IterateOverAllDevices([this, &InSelectedItems, InElapsed](::Audio::FDeviceId DeviceId, FAudioDevice* Device)
			{
				DebugDrawEntries(InElapsed, InSelectedItems, DeviceId);
			});
		}
	}

} // namespace UE::Audio::Insights
