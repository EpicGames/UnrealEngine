// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "AudioDefines.h"
#include "SoundAttenuationVisualizer.h"

namespace UE::Audio::Insights
{
	class IDashboardDataViewEntry;

	class FVirtualLoopsDebugDraw
	{
	public:
		FVirtualLoopsDebugDraw();
		~FVirtualLoopsDebugDraw();

		void DebugDraw(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId);

	private:
		void DebugDrawEntries(float InElapsed, const TArray<TSharedPtr<IDashboardDataViewEntry>>& InSelectedItems, ::Audio::FDeviceId InAudioDeviceId) const;

		FSoundAttenuationVisualizer AttenuationVisualizer;
	};
} // namespace UE::Audio::Insights
