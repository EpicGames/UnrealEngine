// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSpectrogram.h"
#include "AudioSpectrumAnalyzer.h"
#include "Engine/DeveloperSettings.h"
#include "LoudnessMeterRackUnitSettings.h"
#include "Settings/AudioEventLogSettings.h"
#include "Settings/SignalFlowSettings.h"
#include "Settings/SoundDashboardSettings.h"

#include "AudioInsightsEditorSettings.generated.h"

#define UE_API AUDIOINSIGHTSEDITOR_API

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UAudioInsightsEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UE_API UAudioInsightsEditorSettings();

	UE_API virtual FName GetCategoryName() const override;
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;

	/** Whether to automatically set the first PIE client in Audio Insights World Filter. */
	UPROPERTY(Config, EditAnywhere, Category="World Filter")
	bool bWorldFilterDefaultsToFirstClient = false;

	/** Disable all non - command line trace channels apart from Audio, Audio Mixer and Bookmark when starting a live monitoring / live trace.This will reduce the file size of snapshots / trace sessions while using Audio Insights. */
	UPROPERTY(Config, EditAnywhere, Category = "Trace Channels")
	bool bOnlyTraceAudioChannels = true;

	/** Settings for analyzer rack spectrogram widget */
	UPROPERTY(EditAnywhere, config, Category = Spectrogram, meta = (ShowOnlyInnerProperties))
	FSpectrogramRackUnitSettings SpectrogramSettings;

	/** Settings for analyzer rack spectrum analyzer widget */
	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer, meta = (ShowOnlyInnerProperties))
	FSpectrumAnalyzerRackUnitSettings SpectrumAnalyzerSettings;

	/** Settings for the Event Log */
	UPROPERTY(EditAnywhere, config, Category = EventLog, meta = (ShowOnlyInnerProperties))
	FAudioEventLogSettings EventLogSettings;

	/** Settings for the Sound Dashboard */
	UPROPERTY(EditAnywhere, config, Category = SoundDashboard, meta = (ShowOnlyInnerProperties))
	FSoundDashboardSettings SoundDashboardSettings;

	/** Settings for the Signal Flow Dashboard */
	UPROPERTY(EditAnywhere, config, Category = SignalFlow, meta = (ShowOnlyInnerProperties))
	FSignalFlowSettings SignalFlowSettings;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOnlyTraceAudioChannelsPropertyChanged, const bool);
	FOnOnlyTraceAudioChannelsPropertyChanged OnOnlyTraceAudioChannelsPropertyChanged;

private:
	struct EditedEvent
	{
		FString* EventName = nullptr;
		int32 LogicalIndex = INDEX_NONE;
	};

	bool VerifyEventLogInput(const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode& ActiveMemberNode, FPropertyChangedChainEvent& PropertyChangedEvent);
	
	void RefreshEventFilter();
	bool VerifyCategoryEdit();
	bool VerifyCustomEventNameEdit(const FPropertyChangedChainEvent& PropertyChangedEvent, const FMapProperty* CategoryToEventMapProperty, const FSetProperty* EventNamesSetProperty);

	FString* GetEditedCategoryKey(const FPropertyChangedChainEvent& PropertyChangedEvent, const FMapProperty* CategoryToEventMapProperty) const;
	EditedEvent GetEditedEvent(const FPropertyChangedChainEvent& PropertyChangedEvent, const FSetProperty* EditedEventsSetProperty, const FAudioEventLogCustomEvents& Events) const;
	bool IsEventNameUnique(const FString& EditedEventName, const FString& EditedFilterCategory, const int32 EditedLogicalIndex) const;

	void ShowNotification(const FText& TitleText, const FText& SubText) const;

	FDelegateHandle OnRequestReadEventLogSettingsHandle;
	FDelegateHandle OnRequestWriteEventLogSettingsHandle;

	FDelegateHandle OnRequestReadSoundDashboardSettingsHandle;
	FDelegateHandle OnRequestWriteSoundDashboardSettingsHandle;

	FDelegateHandle OnRequestReadSignalFlowSettingsHandle;
	FDelegateHandle OnRequestWriteSignalFlowSettingsHandle;

	const TMap<FString, TSet<FString>> InBuiltAudioLogEventTypes;
};

#undef UE_API
