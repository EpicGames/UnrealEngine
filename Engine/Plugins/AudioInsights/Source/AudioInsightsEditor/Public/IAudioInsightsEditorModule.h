// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Framework/Docking/TabManager.h"
#include "IAudioInsightsModuleInterface.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"

class IAudioInsightsTraceModule;
class SDockTab;

namespace UE::Audio::Insights
{
	class IDashboardViewFactory;
	class FTraceProviderBase;
} // namespace UE::Audio::Insights

class IAudioInsightsEditorModule : public IAudioInsightsModuleInterface
{
public:
	virtual void RegisterDashboardViewFactory(TSharedRef<UE::Audio::Insights::IDashboardViewFactory> InDashboardFactory) = 0;
	virtual void UnregisterDashboardViewFactory(FName InName) = 0;

	virtual ::Audio::FDeviceId GetDeviceId() const = 0;

	AUDIOINSIGHTSEDITOR_API virtual IAudioInsightsTraceModule& GetTraceModule() override;

	virtual TSharedRef<SDockTab> CreateDashboardTabWidget(const FSpawnTabArgs& Args) = 0;

	AUDIOINSIGHTSEDITOR_API static bool IsModuleLoaded();

	AUDIOINSIGHTSEDITOR_API static IAudioInsightsEditorModule& GetChecked();
};
